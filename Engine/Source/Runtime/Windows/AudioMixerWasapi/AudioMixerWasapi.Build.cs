// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMixerWasapi : ModuleRules
{
	public AudioMixerWasapi(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");

		if (Target.bCompileAgainstEngine)
        {
            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"AudioPlatformSupportWasapi",
					"SignalProcessing",
					"WindowsMMDeviceEnumeration"
				}
			);
        }

        PrivateDependencyModuleNames.AddRange(
			new string[] {
					"Core",
					"CoreUObject",
					"AudioMixer",
					"AudioMixerCore"
                }
		);

		PrecompileForTargets = PrecompileTargetsType.None;

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}
