// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerWasapi.h"

#include "Async/Async.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "ScopedCom.h"
#include "WasapiAggregateDeviceMgr.h"
#include "WasapiDefaultDeviceMgr.h"
#include "WasapiServiceWatcher.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <mmdeviceapi.h>			// IMMDeviceEnumerator (CoCreateInstance smoke test)
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace Audio
{
#if PLATFORM_WINDOWS
	namespace
	{
		// Enables the SCM-state watcher that recovers audio when Audiosrv / AudioEndpointBuilder
		// come back after a shutdown. When disabled, audio stays parked on the null renderer
		// after the first service outage until process restart - the audio thread's park block
		// in CheckThreadedDeviceSwap keeps re-parking any device that gets installed because
		// nothing flips bAudioServiceAvailable back to true. This is the correct semantic for
		// an emergency kill switch.
		static bool bAudsrvWatcherEnabledCVar = true;
		FAutoConsoleVariableRef CVarAudsrvWatcherEnabled(
			TEXT("au.AudioMixerWasapi.AudsrvWatcher.Enabled"),
			bAudsrvWatcherEnabledCVar,
			TEXT("Enable the Audiosrv / AudioEndpointBuilder service-state watcher. ")
			TEXT("When enabled, the engine recovers audio automatically after the Windows ")
			TEXT("audio service is stopped and restarted. When disabled, audio stays parked ")
			TEXT("on the null renderer after a service outage until process restart - this ")
			TEXT("is an emergency kill switch, not a graceful-degradation mode. ")
			TEXT("Default: 1 (enabled)."),
			ECVF_Default);

		// Smoke-test retries when audsrv reports RUNNING via SCM but isn't yet ready to serve
		// CoCreateInstance calls. Conservative defaults; runs on the watcher thread so blocking
		// briefly is fine.
		static constexpr int32 SmokeTestMaxRetries = 5;
		static constexpr float SmokeTestRetryDelaySeconds = 0.1f;
	}
#endif // PLATFORM_WINDOWS

	FAudioMixerWasapi::FAudioMixerWasapi()
	{
	}

	FAudioMixerWasapi::~FAudioMixerWasapi()
	{
	}

	void FAudioMixerWasapi::CreateDeviceManager(const bool bInUseAggregateDevice, TUniquePtr<IAudioMixerWasapiDeviceManager>& InDeviceManager)
	{
		if (bInUseAggregateDevice)
		{
			InDeviceManager = MakeUnique<FWasapiAggregateDeviceMgr>();
		}
		else
		{
			InDeviceManager = MakeUnique<FWasapiDefaultDeviceMgr>();
		}

		ensure(InDeviceManager);
	}

	bool FAudioMixerWasapi::InitializeHardware()
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_InitializeHardware, FColor::Blue);

		RegisterDeviceChangedListener();

		if (IAudioMixer::ShouldRecycleThreads())
		{
			// Pre-create the null render device thread so we can simply wake it up when needed.
			// Give it nothing to do, with a slow tick as the default, but ask it to wait for a signal to wake up.
			CreateNullDeviceThread([] {}, 1.0f, true);
		}

#if PLATFORM_WINDOWS
		// Start the audio-service watcher so we can recover when Audiosrv / AudioEndpointBuilder
		// come back after a shutdown. Bound to OnAudioServiceRestarted, which runs on the watcher
		// thread - that's fine, RequestDeviceSwap is thread-safe.
		if (bAudsrvWatcherEnabledCVar)
		{
			ServiceWatcher = MakeUnique<FWasapiServiceWatcher>();
			if (!ServiceWatcher->StartWatching([this]() { this->OnAudioServiceRestarted(); }))
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::InitializeHardware: ServiceWatcher failed to start; audsrv shutdown recovery is disabled for this session.");
				ServiceWatcher.Reset();
			}
		}
#endif // PLATFORM_WINDOWS

		return true;
	}

	bool FAudioMixerWasapi::TeardownHardware()
	{
		if (!bIsInitialized)
		{
			// Don't return prematurely here because TeardownHardware can be called when initialization has failed
			// and we want to clean up any state that exists.
			AUDIO_PLATFORM_LOG_ONCE(TEXT("FAudioMixerWasapi::TeardownHardware called when not fully initialized. Check for init errors earlier in the log."), Warning);
		}

#if PLATFORM_WINDOWS
		// Stop the watcher *before* taking DeviceSwapCriticalSection. StopWatching joins the
		// watcher thread; if the watcher were mid-callback into OnAudioServiceRestarted ->
		// RequestDeviceSwap (which takes this same lock), holding the lock here would deadlock.
		// COM session/device-event threads do not reach this lock (publish-only via
		// bAudioServiceAvailable), so the watcher is the only external contributor.
		if (ServiceWatcher.IsValid())
		{
			ServiceWatcher->StopWatching();
			ServiceWatcher.Reset();
		}
#endif // PLATFORM_WINDOWS

		// Lock prior to changing state to avoid race condition if there happens to be an in-flight device swap
		FScopeLock Lock(&DeviceSwapCriticalSection);

		if (DeviceManager.IsValid() && !DeviceManager->TeardownHardware())
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("FAudioMixerWasapi::TeardownHardware DeviceManager->TeardownHardware() failed."), Warning);
		}

		bIsInitialized = false;

		return true;
	}

	bool FAudioMixerWasapi::IsInitialized() const
	{
		return bIsInitialized;
	}

	int32 FAudioMixerWasapi::GetNumFrames(const int32 InNumRequestedFrames)
	{
		if (DeviceManager.IsValid())
		{
			return DeviceManager->GetNumFrames(InNumRequestedFrames);
		}

		return InNumRequestedFrames;
	}

	bool FAudioMixerWasapi::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_GetNumOutputDevices, FColor::Blue);

		OutNumOutputDevices = 0;

		if (const IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
		{
			OutNumOutputDevices = Cache->GetAllActiveOutputDevices().Num();
			return true;
		}
		else
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("FAudioMixerWasapi device cache not initialized"), Warning);
			return false;
		}
	}

	bool FAudioMixerWasapi::GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo)
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_GetOutputDeviceInfo, FColor::Blue);

		if (const IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
		{
			if (InDeviceIndex == AUDIO_MIXER_DEFAULT_DEVICE_INDEX)
			{
				if (TOptional<FAudioPlatformDeviceInfo> Defaults = Cache->FindDefaultOutputDevice())
				{
					OutInfo = *Defaults;
					return true;
				}
			}
			else
			{
				TArray<FAudioPlatformDeviceInfo> ActiveDevices = Cache->GetAllActiveOutputDevices();
				if (ActiveDevices.IsValidIndex(InDeviceIndex))
				{
					OutInfo = ActiveDevices[InDeviceIndex];
					return true;
				}
			}
		}

		return false;
	}

	bool FAudioMixerWasapi::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		OutDefaultDeviceIndex = AUDIO_MIXER_DEFAULT_DEVICE_INDEX;
		return true;
	}

	bool FAudioMixerWasapi::InitStreamParams(const uint32 InDeviceIndex, const int32 InNumBufferFrames, const int32 InNumBuffers, const int32 InSampleRate, TArray<FWasapiRenderStreamParams>& OutParams)
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_InitStreamParams, FColor::Blue);

		FAudioPlatformDeviceInfo DeviceInfo;
		if (!GetOutputDeviceInfo(InDeviceIndex, DeviceInfo))
		{
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::InitStreamParams unable to find default device");
			return false;
		}

		return InitStreamParams(DeviceInfo, InNumBufferFrames, InNumBuffers, InSampleRate, OutParams);
	}
	
	bool FAudioMixerWasapi::InitStreamParams(const FAudioPlatformDeviceInfo& InDeviceInfo, const int32 InNumBufferFrames, const int32 InNumBuffers, const int32 InSampleRate, TArray<FWasapiRenderStreamParams>& OutParams) const
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_InitStreamParams, FColor::Blue);
		check(GetDeviceInfoCache());

		if (GetDeviceInfoCache()->IsAggregateHardwareDeviceId(*InDeviceInfo.DeviceId))
		{
			if (const IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
			{
				const FName DeviceId = *InDeviceInfo.DeviceId;
				// We use the HardwareId as the DeviceId for aggregate devices which is used by
				// GetAggregateDeviceInfo to gather all the logical devices belonging to this aggregate.
				TArray<FAudioPlatformDeviceInfo> AggregateDevice = Cache->GetLogicalAggregateDevices(DeviceId, EDeviceEndpointType::Render);

				for (const FAudioPlatformDeviceInfo& AggregateDeviceInfo : AggregateDevice)
				{
					TComPtr<IMMDevice> MMDevice = GetMMDevice(AggregateDeviceInfo.DeviceId);
					if (!MMDevice)
					{
						UE_LOGF(LogAudioMixer, Error, "FAudioMixerWasapi::InitStreamParams null MMDevice for aggregate device ID: '%ls'", *InDeviceInfo.DeviceId);
						return false;
					}

					OutParams.Emplace(FWasapiRenderStreamParams(MMDevice, AggregateDeviceInfo, InNumBufferFrames, InNumBuffers, InSampleRate));
				}
			}
		}
		else
		{
			const TComPtr<IMMDevice> MMDevice = GetMMDevice(InDeviceInfo.DeviceId);
			if (!MMDevice)
			{
				UE_LOGF(LogAudioMixer, Error, "FAudioMixerWasapi::InitStreamParams null MMDevice for device ID: '%ls'", *InDeviceInfo.DeviceId);
				return false;
			}

			OutParams.Emplace(FWasapiRenderStreamParams(MMDevice, InDeviceInfo, InNumBufferFrames, InNumBuffers, InSampleRate));
		}

		return true;
	}

	bool FAudioMixerWasapi::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_OpenAudioStream, FColor::Green);
		check(GetDeviceInfoCache());

		OpenStreamParams = Params;

		AudioStreamInfo.Reset();

		AudioStreamInfo.OutputDeviceIndex = OpenStreamParams.OutputDeviceIndex;
		AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
		AudioStreamInfo.NumBuffers = OpenStreamParams.NumBuffers;
		AudioStreamInfo.AudioMixer = OpenStreamParams.AudioMixer;

		// If the user has selected a specific audio device (not the system default), then
		// ignore device change events.
		SetIsListeningForDeviceEvents(Params.bUseSystemAudioDevice);

		bool bIsAggregateDevice = false;
		TArray<FWasapiRenderStreamParams> StreamParams;

		// Track whether we successfully opened a hardware device. If any step in the
		// hardware init chain fails, we fall through to null device setup rather than
		// failing the entire audio subsystem. A null device ticks the audio engine
		// (MetaSounds, mixers, etc.) in software but outputs silence and strictly better
		// than crashing every editor path that touches audio.
		bool bUsingHardwareDevice = false;

		if (InitStreamParams(OpenStreamParams.OutputDeviceIndex, OpenStreamParams.NumFrames, OpenStreamParams.NumBuffers, OpenStreamParams.SampleRate, StreamParams))
		{
			// Adopt the first device info. In the case of an aggregate device, all of the sub-devices will
			// be identical because they belong to the same physical device.
			AudioStreamInfo.DeviceInfo = StreamParams[0].HardwareDeviceInfo;

			// Set the current device name
			bIsAggregateDevice = GetDeviceInfoCache()->IsAggregateHardwareDeviceId(*Params.AudioDeviceId);
			if (bIsAggregateDevice)
			{
				CurrentDeviceName = ExtractAggregateDeviceName(AudioStreamInfo.DeviceInfo.Name);
			}
			else
			{
				CurrentDeviceName = AudioStreamInfo.DeviceInfo.Name;
			}

			// Create and initialize the device manager
			CreateDeviceManager(bIsAggregateDevice, DeviceManager);

			// The ReadNextBufferCallback life cycle is tied to this object. It is ultimately bound to a delegate
			// in the render stream object which will be unbound in TeardownHardware, prior to 'this' being deallocated.
			TFunction<void()> ReadNextBufferCallback = [this]() { ReadNextBuffer(); };
			if (!DeviceManager.IsValid() || !DeviceManager->InitializeHardware(StreamParams, ReadNextBufferCallback))
			{
				// Device was found but hardware init failed. e.g. another application has
				// exclusive WASAPI access, COM error, driver issue, etc. Teardown any
				// partial COM state, then reset the device manager so StartAudioStream()
				// will pick up the null device path.
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::OpenAudioStream: Audio device '%ls' was found but "
					"InitializeHardware() failed. Falling back to null device (no audio output). "
					"Check if another application has exclusive access to this device.", *CurrentDeviceName);
				if (DeviceManager.IsValid())
				{
					DeviceManager->TeardownHardware();
				}
				DeviceManager.Reset();
			}
			else
			{
				// Assign the total number of direct out channels based on the device manager. For WasapiDefaultDeviceMgr this will be 0
				// and for WasapiAggregateDeviceMgr this will be the total of all the channels less the main outs (first 8 channels).
				AudioStreamInfo.DeviceInfo.NumDirectOutChannels = DeviceManager->GetNumDirectOutChannels();

				if (!DeviceManager.IsValid() || !DeviceManager->OpenAudioStream(StreamParams))
				{
					// Hardware initialized but the audio stream couldn't be opened. e.g.
					// WASAPI session creation failed, buffer allocation error, format mismatch.
					// Teardown COM resources before discarding the device manager.
					UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::OpenAudioStream: Audio device '%ls' initialized but "
						"OpenAudioStream() failed. Falling back to null device (no audio output).", *CurrentDeviceName);
					DeviceManager->CloseAudioStream();
					DeviceManager->TeardownHardware();
					DeviceManager.Reset();
				}
				else
				{
					// Full hardware path succeeded.
					bUsingHardwareDevice = true;

					// Store the device ID here in case it is removed. We can switch back if the device comes back.
					if (Params.bRestoreIfRemoved)
					{
						SetOriginalAudioDeviceId(AudioStreamInfo.DeviceInfo.DeviceId);
					}
				}
			}
		}

		if (!bUsingHardwareDevice)
		{
			// No hardware device available — either no device was found, or the device
			// was found but couldn't be opened. Set up a stereo null device so the audio
			// engine can still run in software. StartAudioStream() will detect the null
			// DeviceManager and call StartRunningNullDevice().
			AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
			AudioStreamInfo.DeviceInfo.OutputChannelArray = { EAudioMixerChannel::FrontLeft, EAudioMixerChannel::FrontRight };
			AudioStreamInfo.DeviceInfo.NumChannels = 2;
			AudioStreamInfo.DeviceInfo.SampleRate = OpenStreamParams.SampleRate;
			AudioStreamInfo.DeviceInfo.Format = EAudioMixerStreamDataFormat::Float;
			AudioStreamInfo.DeviceInfo.NumDirectOutChannels = 0;
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;
		bIsInitialized = true;

		UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi initialized: SampleRate=%d NumChannels=%d NumDirectOutChannels=%d bIsAggregateDevice=%d StreamParams.Num()=%d", 
			OpenStreamParams.SampleRate, AudioStreamInfo.DeviceInfo.NumChannels, AudioStreamInfo.DeviceInfo.NumDirectOutChannels, bIsAggregateDevice, StreamParams.Num());

		return true;
	}

	bool FAudioMixerWasapi::CloseAudioStream()
	{
		if (!bIsInitialized || AudioStreamInfo.StreamState == EAudioOutputStreamState::Closed)
		{
			return false;
		}

		// Lock prior to changing state to avoid race condition if there happens to be an in-flight device swap
		FScopeLock Lock(&DeviceSwapCriticalSection);

		// If we're closing the stream, we're not interested in the results of the device swap. 
		// Reset the handle to the future.
		ResetActiveDeviceSwapFuture();

		if (DeviceManager.IsValid() && !DeviceManager->CloseAudioStream())
		{
			UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::CloseAudioStream CloseAudioStream failed");
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;

		return true;
	}

	bool FAudioMixerWasapi::StartAudioStream()
	{
		bool bDidStartAudioStream = false;
		
		if (bIsInitialized)
		{
			// Can be called during device swap when AudioRenderEvent can be null
			if (nullptr == AudioRenderEvent)
			{
				// Call BeginGeneratingAudio before StartAudioStream so that the output buffer is set up
				// prior to the device thread starting.
				// This will set AudioStreamInfo.StreamState to EAudioOutputStreamState::Running
				BeginGeneratingAudio();
			}
			else
			{
				AudioStreamInfo.StreamState = EAudioOutputStreamState::Running;
			}

			if (DeviceManager.IsValid() && DeviceManager->IsInitialized())
			{
				bDidStartAudioStream = DeviceManager->StartAudioStream();
			}
			else
			{
				check(!bIsUsingNullDevice);
				StartRunningNullDevice();
				bDidStartAudioStream = true;
			}
		}

		return bDidStartAudioStream;
	}

	bool FAudioMixerWasapi::StopAudioStream()
	{
		if (!bIsInitialized)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("FAudioMixerWasapi::StopAudioStream() not initialized."), Warning);
			return false;
		}

		// Lock prior to changing state to avoid race condition if there happens to be an in-flight device swap
		FScopeLock Lock(&DeviceSwapCriticalSection);

		UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::StopAudioStream() InstanceID=%d, StreamState=%d",
			InstanceID, static_cast<int32>(AudioStreamInfo.StreamState));

		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped && AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			// Shutdown the AudioRenderThread if we're running or mid-device swap
			if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running ||
				AudioStreamInfo.StreamState == EAudioOutputStreamState::SwappingDevice)
			{
				StopGeneratingAudio();
			}

			if (DeviceManager.IsValid())
			{
				DeviceManager->StopAudioStream();
			}

			check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);
		}
		
		if (bIsUsingNullDevice)
		{
			StopRunningNullDevice();
		}

		return true;
	}

	FAudioPlatformDeviceInfo FAudioMixerWasapi::GetPlatformDeviceInfo() const
	{
		return AudioStreamInfo.DeviceInfo;
	}

	void FAudioMixerWasapi::SubmitBuffer(const uint8* InBuffer)
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_SubmitBuffer, FColor::Blue);

		if (DeviceManager.IsValid())
		{
			DeviceManager->SubmitBuffer(InBuffer, OpenStreamParams.NumFrames);
		}
	}

	void FAudioMixerWasapi::SubmitDirectOutBuffer(const int32 InDirectOutIndex, const FAlignedFloatBuffer& InBuffer)
	{
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_SubmitDirectOutBuffer, FColor::Green);
		
		if (DeviceManager.IsValid())
		{
			DeviceManager->SubmitDirectOutBuffer(InDirectOutIndex, InBuffer);
		}
	}

	FString FAudioMixerWasapi::GetDefaultDeviceName()
	{
		return FString();
	}

	FAudioPlatformSettings FAudioMixerWasapi::GetPlatformSettings() const
	{
#if WITH_ENGINE
		return FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
#else
		return FAudioPlatformSettings();
#endif // WITH_ENGINE
	}

	IAudioPlatformDeviceInfoCache* FAudioMixerWasapi::GetDeviceInfoCache() const
	{
		if (ShouldUseDeviceInfoCache())
		{
			return DeviceInfoCache.Get();
		}

		return nullptr;
	}
	
	bool FAudioMixerWasapi::IsDeviceInfoValid(const FAudioPlatformDeviceInfo& InDeviceInfo) const
	{
		// Device enumeration will not return invalid devices. This
		// is more of a sanity check.
		if (InDeviceInfo.NumChannels > 0 && InDeviceInfo.SampleRate > 0)
		{
			return true;
		}

		return false;
	}

	void FAudioMixerWasapi::OnSessionDisconnect(IAudioMixerDeviceChangedListener::EDisconnectReason InReason)
	{
		// Device has disconnected from current session.
		if (InReason == IAudioMixerDeviceChangedListener::EDisconnectReason::FormatChanged)
		{
			// OnFormatChanged, retry again same device.
			RequestDeviceSwap(GetDeviceId(), /*force*/ true, TEXT("FAudioMixerWasapi::OnSessionDisconnect() - FormatChanged"));
		}
		else if (InReason == IAudioMixerDeviceChangedListener::EDisconnectReason::DeviceRemoval)
		{
			// Ignore Device Removal, as this is handle by the Device Removal logic in the Notification Client.
		}
		else if (InReason == IAudioMixerDeviceChangedListener::EDisconnectReason::ServerShutdown)
		{
			// audsrv is going away. Any further WASAPI call (Activate, Initialize, Stop, Release)
			// is a COM RPC to a service that no longer answers, and COM RPCs default to no
			// timeout - they wait forever. Publish the flag; the audio thread observes it in
			// CheckThreadedDeviceSwap and parks on the null renderer. Topology changes stay on
			// the audio thread - we do not touch swap state from a COM notification thread.

			UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::OnSessionDisconnect() - ServerShutdown: audsrv is shutting down, publishing unavailable flag. Audio thread will park on next tick. InstanceID=%d", InstanceID);
			bAudioServiceAvailable.store(false, std::memory_order_release);
		}
		else
		{
			// SessionLogoff, SessionDisconnected, ExclusiveModeOverride
			// audsrv is alive in these cases, so the existing default-swap path is safe.
			RequestDeviceSwap(TEXT(""), /*force*/ true, TEXT("FAudioMixerWasapi::OnSessionDisconnect() - Other"));
		}
	}

	void FAudioMixerWasapi::OnAudioServiceRestarted()
	{
#if PLATFORM_WINDOWS
		// Smoke-test that audsrv is actually ready to serve clients before we try to use it.
		// SCM may report RUNNING the instant the service process starts, but the audio session
		// manager / endpoint discovery layer can take a beat to come back online. CoCreateInstance
		// against MMDeviceEnumerator goes through audsrv and gives us a cheap readiness probe:
		// success means we can safely issue a RequestDeviceSwap; failure means defer.
		TComPtr<IMMDeviceEnumerator> Enumerator;
		HRESULT LastHr = E_FAIL;
		
		FScopedCoInitialize CoInitialize; 
		
		for (int32 Attempt = 0; Attempt < SmokeTestMaxRetries; ++Attempt)
		{
			LastHr = CoCreateInstance(
				__uuidof(MMDeviceEnumerator),
				nullptr,
				CLSCTX_INPROC_SERVER,
				__uuidof(IMMDeviceEnumerator),
				IID_PPV_ARGS_Helper(&Enumerator));

			if (SUCCEEDED(LastHr) && Enumerator.IsValid())
			{
				break;
			}

			UE_LOGF(LogAudioMixer, Display,
				"FAudioMixerWasapi::OnAudioServiceRestarted: smoke-test CoCreateInstance attempt %d/%d returned 0x%08x; retrying.",
				Attempt + 1, SmokeTestMaxRetries, static_cast<uint32>(LastHr));
			FPlatformProcess::Sleep(SmokeTestRetryDelaySeconds);
		}

		if (FAILED(LastHr) || !Enumerator.IsValid())
		{
			UE_LOGF(LogAudioMixer, Warning,
				"FAudioMixerWasapi::OnAudioServiceRestarted: smoke test failed after %d attempts (last HR=0x%08x). Staying parked; will retry on next service-state notification.",
				SmokeTestMaxRetries, static_cast<uint32>(LastHr));
			return;
		}

		// Drop the smoke-test enumerator - the device-swap path creates its own when it
		// rebuilds the device manager. Holding this one would not hurt but it's not needed.
		Enumerator.Reset();

		// Refresh the device cache before resolving the swap target. Entries from before the
		// outage hold IMMDevice proxies that may not survive audsrv recycling; re-enumerate
		// so InitializeDeviceSwapContext's lookup sees current state. Note: the audio thread's
		// bDeviceLost block in CheckThreadedDeviceSwap is gated on IsAudioServiceAvailable(),
		// which is still false at this point, so it will not race us by issuing a swap based
		// on the refreshed cache.
		if (IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
		{
			Cache->Refresh();
		}

		// Prefer the device the player originally selected (stashed in OpenAudioStream when
		// bRestoreIfRemoved is true). An empty TargetId is correct for the "follow system
		// default" config - RequestDeviceSwap resolves it to the current default endpoint.
		// For non-default-device players, an empty swap target lands on whatever audsrv
		// reports as the default during warm-up, which is often a placeholder / non-audible
		// endpoint rather than the player's chosen device.
		const FString TargetId = GetOriginalAudioDeviceId();
		FString SwapTarget = TargetId;

		if (!TargetId.IsEmpty())
		{
			// Verify the original device made it into the refreshed cache. If the player
			// unplugged it during the outage, fall back to a default-device swap rather than
			// handing InitializeDeviceSwapContext an ID it can't resolve.
			bool bOriginalDeviceAvailable = false;
			if (IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
			{
				if (TOptional<FAudioPlatformDeviceInfo> Info = Cache->FindActiveOutputDevice(FName(*TargetId)))
				{
					bOriginalDeviceAvailable = IsDeviceInfoValid(*Info);
				}
			}

			if (!bOriginalDeviceAvailable)
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::OnAudioServiceRestarted: original device '%ls' not present after recovery; falling back to system default. InstanceID=%d", *TargetId, InstanceID);
				SwapTarget = TEXT("");
			}
		}

		// Clear bDeviceLost and flip the service flag atomically under the swap lock so a
		// CheckThreadedDeviceSwap tick can never observe (service==available, device-lost==true)
		// and race us with a duplicate RequestDeviceSwap targeting a different device. The
		// bDeviceLost block in CheckThreadedDeviceSwap takes this same lock and re-checks
		// state under it. Release the lock before issuing RequestDeviceSwap - RequestDeviceSwap
		// acquires DeviceSwapCriticalSection internally; FCriticalSection is recursive on
		// Windows but nesting here is unnecessary and fragile.
		//
		// Note: clearing bDeviceLost here opens a brief window where the WASAPI render thread
		// can pump one buffer through the wedged client before the swap tears it down. The
		// render thread will re-set bDeviceLost on its next failed call; the trade-off is
		// strictly better than letting the audio thread race us with a duplicate swap.
		{
			FScopeLock Lock(&DeviceSwapCriticalSection);
			if (DeviceManager.IsValid())
			{
				DeviceManager->ClearDeviceLost();
			}
			MarkAudioServiceAvailable();
		}

		UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::OnAudioServiceRestarted: smoke test passed, issuing recovery swap to '%ls'. InstanceID=%d", SwapTarget.IsEmpty() ? TEXT("[System Default]") : *SwapTarget, InstanceID);
		RequestDeviceSwap(SwapTarget, /*force*/ true, TEXT("FAudioMixerWasapi::OnAudioServiceRestarted"));
#endif // PLATFORM_WINDOWS
	}

	bool FAudioMixerWasapi::CheckThreadedDeviceSwap()
	{
		bool bDidStopGeneratingAudio = false;
#if PLATFORM_WINDOWS
		// Park on the null renderer if the audio service is unavailable. Set from
		// OnSessionDisconnect(ServerShutdown) and from the device-listener SCM probes.
		// Centralizing the transition here keeps null-renderer changes on the audio
		// thread and out of COM notification threads.
		if (!IsAudioServiceAvailable()
			&& AudioStreamInfo.StreamState == EAudioOutputStreamState::Running
			&& !bIsUsingNullDevice)
		{
			FScopeLock Lock(&DeviceSwapCriticalSection);
			if (!bIsUsingNullDevice)  // re-check under lock; swap machine could race
			{
				// SetDeviceLost is guarded on DeviceManager.IsValid() because
				// PreDeviceSwap can race us between the outer StreamState==Running
				// check and acquiring this lock, leaving DeviceManager moved-out
				// into the swap context. When that happens there is no live WASAPI
				// render callback to neutralize, so the kill-switch invariant
				// (described below) is trivially satisfied with no action needed.
				// We still start the null renderer below to keep the renderer
				// command queue draining while the in-flight swap resolves.
				if (DeviceManager.IsValid())
				{
					// Arm the render-callback kill switch BEFORE starting the null
					// renderer. Without this the WASAPI render thread could pump one
					// or more buffers into ReadNextBuffer concurrently with the null
					// thread, in the window between audsrv going down and WASAPI
					// calls returning failure. bDeviceLost stays set until
					// OnAudioServiceRestarted's swap tears the OldDeviceManager down.
					DeviceManager->SetDeviceLost();
				}
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::CheckThreadedDeviceSwap: audio service unavailable, parking on null renderer. InstanceID=%d", InstanceID);
				StartRunningNullDevice();
			}
		}

		// When the WASAPI render thread sets bDeviceLost (AUDCLNT_E_DEVICE_INVALIDATED
		// or AUDCLNT_E_SERVICE_NOT_RUNNING), trigger a recovery swap immediately rather
		// than spinning on the dead client.
		if (DeviceManager.IsValid()
			&& DeviceManager->IsDeviceLost()
			&& AudioStreamInfo.StreamState == EAudioOutputStreamState::Running)
		{
			// Take the swap lock and re-check state under it. OnAudioServiceRestarted
			// does (ClearDeviceLost + MarkAudioServiceAvailable) atomically under this
			// same lock; without the re-check we could observe (service==available,
			// device-lost==true) and race the watcher's recovery swap with one of our
			// own targeting a different device. Capture the post-decision state here
			// and perform the swap request after releasing the lock - matches
			// OnAudioServiceRestarted's pattern of avoiding nested acquisition.
			bool bShouldRequestSwap = false;
			FString FailedDeviceId;
			{
				FScopeLock Lock(&DeviceSwapCriticalSection);
				if (DeviceManager.IsValid() && DeviceManager->IsDeviceLost())
				{
					if (IsAudioServiceAvailable())
					{
						// Only clear bDeviceLost when a swap will actually fire - while parked,
						// the flag is load-bearing as the kill switch for the WASAPI render
						// callback (see park block above). OnAudioServiceRestarted's swap will
						// tear the OldDeviceManager down and the flag goes away with it.
						DeviceManager->ClearDeviceLost();
						FailedDeviceId = AudioStreamInfo.DeviceInfo.DeviceId;
						bShouldRequestSwap = true;
					}
					// else: service is unavailable - park block above already handled topology.
					// Leave bDeviceLost set so the render callback continues to early-return.
				}
				// else: watcher's recovery already cleared the flag. Nothing to do.
			}

			if (bShouldRequestSwap)
			{
				UpdateCacheFromFailedDevice(FailedDeviceId);
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::CheckThreadedDeviceSwap: device lost detected by render thread, requesting swap. InstanceID=%d", InstanceID);
				RequestDeviceSwap(TEXT(""), /*force*/ true, TEXT("FAudioMixerWasapi::CheckThreadedDeviceSwap - device lost"));
			}
		}

		bDidStopGeneratingAudio = FAudioMixerPlatformSwappable::CheckThreadedDeviceSwap();
#endif //PLATFORM_WINDOWS
		return bDidStopGeneratingAudio;
	}

	bool FAudioMixerWasapi::InitializeDeviceSwapContext(const FString& InRequestedDeviceID, const TCHAR* InReason)
	{
		check(GetDeviceInfoCache());

		// Look up device. Blank name looks up current default.
		const FName NewDeviceId = *InRequestedDeviceID;
		TOptional<FAudioPlatformDeviceInfo> DeviceInfo;
	
		if (TOptional<FAudioPlatformDeviceInfo> TempDeviceInfo = GetDeviceInfoCache()->FindActiveOutputDevice(NewDeviceId))
		{
			if (TempDeviceInfo.IsSet())
			{
				if (IsDeviceInfoValid(*TempDeviceInfo))
				{
					DeviceInfo = MoveTemp(TempDeviceInfo);
				}
				else
				{
					UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::InitializeDeviceSwapContext - Ignoring attempt to switch to device with unsupported params: Channels=%u, SampleRate=%u, Id=%ls, Name=%ls",
						(uint32)TempDeviceInfo->NumChannels, (uint32)TempDeviceInfo->SampleRate, *TempDeviceInfo->DeviceId, *TempDeviceInfo->Name);

					return false;
				}
			}
		}
		
		return InitDeviceSwapContextInternal(InRequestedDeviceID, InReason, DeviceInfo);
	}

	bool FAudioMixerWasapi::InitDeviceSwapContextInternal(const FString& InRequestedDeviceID, const TCHAR* InReason, const TOptional<FAudioPlatformDeviceInfo>& InDeviceInfo)
	{
		check(GetDeviceInfoCache());
		
		// Access to device swap context must be protected by DeviceSwapCriticalSection
		FScopeLock Lock(&DeviceSwapCriticalSection);
		
		if (DeviceSwapContext.IsValid())
		{
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::InitDeviceSwapContextInternal - DeviceSwapContext in-flight, cancelling and starting a new swap.");

			// If there is a previous device manager, this has been slated to be taken offline, so we need to shut the old one down first before resetting it
			// Note, this can happen if a swap request comes in on the hardware device thread in between PreDeviceSwap() is called on the game thread and 
			// PerformDeviceSwap is called on an asynchronous task. 
			if (DeviceSwapContext->OldDeviceManager.IsValid())
			{
				const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId(); // cross-platform thread id
				UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::InitDeviceSwapContextInternal - begin audio device manager shutdown for interrupted swap request. Thread Id %u", ThreadId);

				// Shutdown the current device manager
				DeviceSwapContext->OldDeviceManager->StopAudioStream();
				DeviceSwapContext->OldDeviceManager->CloseAudioStream();
				DeviceSwapContext->OldDeviceManager->TeardownHardware();
				DeviceSwapContext->OldDeviceManager.Reset();
				UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::InitDeviceSwapContextInternal - successfully shut down previous device manager before resetting in interrupted swap");
			}
		}

		// Create the new device swap context
		DeviceSwapContext.Reset();
		DeviceSwapContext = MakeUnique<FWasapiDeviceSwapContext>(InRequestedDeviceID, InReason);
		if (!DeviceSwapContext.IsValid())
		{
			UE_LOGF(LogAudioMixer, Warning, "FMixerPlatformWasapi::CreateDeviceSwapContext - failed to create DeviceSwapContext");
			return false;
		}

		DeviceSwapContext->NewDevice = InDeviceInfo;
		
		const FAudioPlatformSettings EngineSettings = GetPlatformSettings();
		TArray<FWasapiRenderStreamParams> StreamParams;

		if (DeviceSwapContext->NewDevice.IsSet())
		{
			DeviceSwapContext->bIsAggregateDevice = GetDeviceInfoCache()->IsAggregateHardwareDeviceId(*DeviceSwapContext->NewDevice->DeviceId);
			
			if (!InitStreamParams(*DeviceSwapContext->NewDevice, EngineSettings.CallbackBufferFrameSize, EngineSettings.NumBuffers, EngineSettings.SampleRate, StreamParams))
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::InitializeDeviceSwapContext - InitStreamParams() failed");
				DeviceSwapContext->NewDevice.Reset();
				DeviceSwapContext.Reset();

				return false;
			}
		}

		// Initialize remaining fields except for OldDeviceManager which will 
		// happen later in CheckThreadedDeviceSwap from the Game thread.
		DeviceSwapContext->ReadNextBufferCallback = [this](){ ReadNextBuffer(); };
		DeviceSwapContext->StreamParams = StreamParams;
		DeviceSwapContext->PlatformSettings = EngineSettings;

		return true;
	}
	
	void FAudioMixerWasapi::EnqueueAsyncDeviceSwap()
	{
		FScopeLock Lock(&DeviceSwapCriticalSection);
		UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::EnqueueAsyncDeviceSwap - enqueuing async device swap");
		
		TFunction<TUniquePtr<FDeviceSwapResult>()> AsyncDeviceSwap = [this]() mutable -> TUniquePtr<FDeviceSwapResult>
		{
			TUniquePtr<FWasapiDeviceSwapContext> TempContext;
			{
				// Hold DeviceSwapCriticalSection only for the pointer swap, not the heavy COM
				// teardown/init inside PerformDeviceSwap. After Swap(), TempContext is uniquely
				// owned by this lambda - no shared state for PerformDeviceSwap to race on.
				FScopeLock Lock(&DeviceSwapCriticalSection);
				if (AudioStreamInfo.StreamState == EAudioOutputStreamState::SwappingDevice)
				{
					Swap(TempContext, DeviceSwapContext);
				}
			}

			return PerformDeviceSwap(MoveTemp(TempContext));
		};
		SetActiveDeviceSwapFuture(Async(EAsyncExecution::TaskGraph, MoveTemp(AsyncDeviceSwap)));
	}

	void FAudioMixerWasapi::SynchronousDeviceSwap()
	{
		FScopeLock Lock(&DeviceSwapCriticalSection);

		// Transfer ownership of DeviceSwapContext memory to the device swap routine
		TUniquePtr<FDeviceSwapResult> DeviceSwapResult = PerformDeviceSwap(TUniquePtr<FWasapiDeviceSwapContext>(MoveTemp(DeviceSwapContext)));

		// Set the promise and future result to replicate what the async task does
		TPromise<TUniquePtr<FDeviceSwapResult>> Promise;
		
		// It's ok if DeviceSwapResult is null here. It indicates an invalid device which will be handled.
		Promise.SetValue(MoveTemp(DeviceSwapResult));
		SetActiveDeviceSwapFuture(Promise.GetFuture());
	}
	
	TUniquePtr<FDeviceSwapResult> FAudioMixerWasapi::PerformDeviceSwap(TUniquePtr<FWasapiDeviceSwapContext>&& InDeviceContext)
	{
		// Static method enforces no other state sharing occurs with the object that called it. InDeviceContext
		// should have no other references outside of this method so that the device swap operation is isolated.
		SCOPED_NAMED_EVENT(FAudioMixerWasapi_PerformDeviceSwap, FColor::Blue);

		const uint64 StartTimeCycles = FPlatformTime::Cycles64();
		// This runs in an async task whose thread may not have initialized com
		FScopedCoInitialize CoInitialize;

		// No lock held here: InDeviceContext is uniquely owned by this call (moved in from
		// the async lambda's TempContext or from SynchronousDeviceSwap's MoveTemp). All device-
		// manager state lives inside that context; nothing on `this` is touched here except
		// RegisterForSessionEvents, which is independently synchronized via the notification
		// client.
		if (InDeviceContext.IsValid())
		{
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PerformDeviceSwap - Start. Because=%ls", *InDeviceContext->DeviceSwapReason);

			if (InDeviceContext->OldDeviceManager.IsValid())
			{
				// Shutdown the current device manager
				InDeviceContext->OldDeviceManager->StopAudioStream();
				InDeviceContext->OldDeviceManager->CloseAudioStream();
				InDeviceContext->OldDeviceManager->TeardownHardware();
				InDeviceContext->OldDeviceManager.Reset();
				UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PerformDeviceSwap - successfully shut down previous device manager");
			}
			else
			{
				UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PerformDeviceSwap - no device manager running, null renderer must be active");
			}

			// Don't attempt to create a new setup if there's no devices available.
			if (!InDeviceContext->NewDevice.IsSet() || InDeviceContext->StreamParams.IsEmpty())
			{
				UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PerformDeviceSwap - no new device to switch to...will run null device");
				return {};
			}

			TUniquePtr<FWasapiDeviceSwapResult> DeviceSwapResult = MakeUnique<FWasapiDeviceSwapResult>();
			CreateDeviceManager(InDeviceContext->bIsAggregateDevice, DeviceSwapResult->NewDeviceManager);
			
			if (!DeviceSwapResult->NewDeviceManager.IsValid())
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::PerformDeviceSwap - InitializeHardware failed to create new device manager");
				return {};
			}
			
			if (!DeviceSwapResult->NewDeviceManager->InitializeHardware(InDeviceContext->StreamParams, InDeviceContext->ReadNextBufferCallback))
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::PerformDeviceSwap - InitializeHardware failed while attempting to device swap");
				return {};
			}
			
			if (!DeviceSwapResult->NewDeviceManager->OpenAudioStream(InDeviceContext->StreamParams))
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::PerformDeviceSwap - OpenAudioStream failed while attempting to device swap");
				return {};
			}
			
#if PLATFORM_WINDOWS
			RegisterForSessionEvents(InDeviceContext->RequestedDeviceId);
#endif // PLATFORM_WINDOWS

			DeviceSwapResult->SuccessfulDurationMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTimeCycles);
			DeviceSwapResult->DeviceInfo = InDeviceContext->StreamParams[0].HardwareDeviceInfo;
			DeviceSwapResult->bIsAggregateDevice = InDeviceContext->bIsAggregateDevice;
			DeviceSwapResult->SwapReason = InDeviceContext->DeviceSwapReason;
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PerformDeviceSwap - successfully completed device swap");

			return DeviceSwapResult;
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "FAudioMixerWasapi::PerformDeviceSwap - failed due to invalid DeviceSwapContext");
		}

		return {};
	}
	
	bool FAudioMixerWasapi::PreDeviceSwap()
	{
		if (DeviceManager.IsValid())
		{
			// Access to device swap context must be protected by DeviceSwapCriticalSection
			FScopeLock Lock(&DeviceSwapCriticalSection);

			if (DeviceSwapContext.IsValid())
			{
				// Finish initializing the device swap context
				check(!DeviceSwapContext->OldDeviceManager);
				DeviceSwapContext->OldDeviceManager = MoveTemp(DeviceManager);
				UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PreDeviceSwap - Starting swap to [%ls]", DeviceSwapContext->RequestedDeviceId.IsEmpty() ? TEXT("[System Default]") : *DeviceSwapContext->RequestedDeviceId);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::PreDeviceSwap - null device swap context");
				return false;
			}
		}
		else
		{
			// This is not an error because the null renderer could be running
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PreDeviceSwap - no device manager (null renderer must be running)");
		}

		return true;
	}
	
	bool FAudioMixerWasapi::PostDeviceSwap()
	{
		bool bDidSucceed = false;

		FWasapiDeviceSwapResult* DeviceSwapResult = StaticCast<FWasapiDeviceSwapResult*>(GetDeviceSwapResult());
			
		if (DeviceSwapResult && DeviceSwapResult->IsNewDeviceReady())
		{
			SCOPED_NAMED_EVENT(FAudioMixerWasapiFAudioMixerWasapi_CheckThreadedDeviceSwap_EndSwap, FColor::Blue);

			FScopeLock Lock(&DeviceSwapCriticalSection);

			// Copy our new Device Info into our active one.
			AudioStreamInfo.DeviceInfo = DeviceSwapResult->DeviceInfo;

			// Set the current device name
			if (DeviceSwapResult->bIsAggregateDevice)
			{
				CurrentDeviceName = ExtractAggregateDeviceName(AudioStreamInfo.DeviceInfo.Name);
			}
			else
			{
				CurrentDeviceName = AudioStreamInfo.DeviceInfo.Name;
			}

			// Display our new XAudio2 Mastering voice details.
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::PostDeviceSwap - successful Swap new Device is (NumChannels=%u, SampleRate=%u, DeviceID=%ls, Name=%ls), Reason=%ls, InstanceID=%d, DurationMS=%.2f",
				(uint32)AudioStreamInfo.DeviceInfo.NumChannels, (uint32)AudioStreamInfo.DeviceInfo.SampleRate, *AudioStreamInfo.DeviceInfo.DeviceId, *AudioStreamInfo.DeviceInfo.Name,
				*DeviceSwapResult->SwapReason, InstanceID, DeviceSwapResult->SuccessfulDurationMs);

			// Reinitialize the output circular buffer to match the buffer math of the new audio device.
			const int32 NumOutputSamples = AudioStreamInfo.NumOutputFrames * AudioStreamInfo.DeviceInfo.NumChannels;
			if (ensure(NumOutputSamples > 0))
			{
				OutputBuffer.Init(AudioStreamInfo.AudioMixer, NumOutputSamples, NumOutputBuffers, AudioStreamInfo.DeviceInfo.Format);
			}

			check(!DeviceManager);
			DeviceManager = MoveTemp(DeviceSwapResult->NewDeviceManager);

			bDidSucceed = true;
		}
		else
		{
			if (!DeviceSwapResult)
			{
				UE_LOGF(LogAudioMixer, Error, "FAudioMixerWasapi::PostDeviceSwap - null device swap result!)");
			}
			else
			{
				UE_LOGF(LogAudioMixer, Error, "FAudioMixerWasapi::PostDeviceSwap - DeviceSwapResult->IsNewDeviceReady() = %d)", DeviceSwapResult->IsNewDeviceReady());
			}
		}
		
		ResetActiveDeviceSwapFuture();

		return bDidSucceed;
	}
	
	void FAudioMixerWasapi::ResumePlaybackOnNewDevice()
	{
		if (DeviceSwapContext.IsValid())
		{
			// If another device swap was requested, don't bother resuming playback
			AudioStreamInfo.StreamState = EAudioOutputStreamState::SwappingDevice;
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::ResumePlaybackOnNewDevice - device swap requested during previous device swap...not resuming audio");
		}
		else
		{
			FAudioMixerPlatformSwappable::ResumePlaybackOnNewDevice();
		}
	}
}
