// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "AudioMixerWasapiRenderStream.h"
#include "IAudioMixerWasapiDeviceManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#include <atomic>

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <mmdeviceapi.h>			// IMMNotificationClient
#include <audiopolicy.h>			// IAudioSessionEvents
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

namespace Audio
{
	class FWasapiDefaultRenderStream;
	class FWasapiServiceWatcher;

	struct FWasapiDeviceSwapContext : public FDeviceSwapContext
	{
		FWasapiDeviceSwapContext() = delete;
		FWasapiDeviceSwapContext(const FString& InRequestedDeviceID, const FString& InReason) :
			FDeviceSwapContext(InRequestedDeviceID, InReason)
		{}
		
		FAudioPlatformSettings PlatformSettings;
		TArray<FWasapiRenderStreamParams> StreamParams;
		TFunction<void()> ReadNextBufferCallback;
		TUniquePtr<IAudioMixerWasapiDeviceManager> OldDeviceManager;
		bool bIsAggregateDevice = false;
	};

	struct FWasapiDeviceSwapResult : public FDeviceSwapResult
	{
		virtual bool IsNewDeviceReady() const override
		{
			return NewDeviceManager.IsValid();
		}

		TUniquePtr<IAudioMixerWasapiDeviceManager> NewDeviceManager;
		bool bIsAggregateDevice = false;
	};

	/**
	 * FAudioMixerWasapi - Wasapi audio backend for Windows and Xbox
	 */
	class FAudioMixerWasapi : public FAudioMixerPlatformSwappable
	{
	public:

		FAudioMixerWasapi();
		virtual ~FAudioMixerWasapi() override;

		//~ Begin IAudioMixerPlatformInterface
		virtual FString GetPlatformApi() const override { return TEXT("WASAPIMixer"); }
		virtual bool InitializeHardware() override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual int32 GetNumFrames(const int32 InNumRequestedFrames) override;
		virtual bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		virtual bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		virtual bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		virtual void SubmitBuffer(const uint8* Buffer) override;
		virtual void SubmitDirectOutBuffer(const int32 InDirectOutIndex, const FAlignedFloatBuffer& InBuffer) override;
		virtual FString GetDefaultDeviceName() override;
		virtual FAudioPlatformSettings GetPlatformSettings() const override;
		virtual IAudioPlatformDeviceInfoCache* GetDeviceInfoCache() const override;
		virtual bool IsDeviceInfoValid(const FAudioPlatformDeviceInfo& InDeviceInfo) const override;
		virtual bool ShouldUseDeviceInfoCache() const override { return true; }
		virtual void ResumePlaybackOnNewDevice() override;
		//~ End IAudioMixerPlatformInterface

		//~ Begin IAudioMixerDeviceChangedListener
		virtual void RegisterDeviceChangedListener() override;
		virtual void UnregisterDeviceChangedListener() override;
		virtual void OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;
		virtual void OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;
		virtual void OnDeviceAdded(const FString& DeviceId, bool bIsRenderDevice) override;
		virtual void OnDeviceRemoved(const FString& DeviceId, bool bIsRenderDevice) override;
		virtual void OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool bIsRenderDevice) override;
		virtual void OnFormatChanged(const FString& InDeviceId, const FFormatChangedData& InFormat) override;
		virtual void OnSessionDisconnect(Audio::IAudioMixerDeviceChangedListener::EDisconnectReason InReason) override;
		virtual FString GetDeviceId() const override;
		//~ End IAudioMixerDeviceChangedListener

		//~ Begin FAudioMixerPlatformSwappable
		virtual bool InitializeDeviceSwapContext(const FString& InRequestedDeviceID, const TCHAR* InReason) override;
		virtual bool CheckThreadedDeviceSwap() override;
		virtual bool PreDeviceSwap() override;
		virtual void EnqueueAsyncDeviceSwap() override;
		virtual void SynchronousDeviceSwap() override;
		virtual bool PostDeviceSwap() override;
		//~ End FAudioMixerPlatformSwappable

		/** Returns false if we have observed an audsrv shutdown (IAudioSessionEvents::OnSessionDisconnected
		 *  with reason ServerShutdown) and have not yet seen the service come back. While false, no new
		 *  WASAPI calls should be issued: they would block on a dead service. The SCM watcher (future CL)
		 *  is responsible for calling MarkAudioServiceAvailable() once the service is confirmed running. */
		bool IsAudioServiceAvailable() const { return bAudioServiceAvailable.load(std::memory_order_acquire); }

		/** Called when the SCM watcher confirms Audiosrv/AudioEndpointBuilder are RUNNING again
		 *  (or by tests that simulate recovery). Caller is responsible for issuing a fresh
		 *  RequestDeviceSwap to bring real audio back. */
		void MarkAudioServiceAvailable() { bAudioServiceAvailable.store(true, std::memory_order_release); }

		/** Recovery handler invoked by FWasapiServiceWatcher when both Audiosrv and
		 *  AudioEndpointBuilder have returned to SERVICE_RUNNING after either was observed
		 *  non-RUNNING. Smoke-tests audsrv readiness via CoCreateInstance, then marks the
		 *  service available and issues a recovery swap. Runs on the watcher thread;
		 *  RequestDeviceSwap is thread-safe so no explicit marshalling is required. */
		void OnAudioServiceRestarted();

	protected:
		/** Can be used by subclasses to initialize a device swap context by supplying a specific 
		 *  FAudioPlatformDeviceInfo rather than looking it up via the requested device Id.
		 */
		bool InitDeviceSwapContextInternal(const FString& InRequestedDeviceID, const TCHAR* InReason, const TOptional<FAudioPlatformDeviceInfo>& InDeviceInfo);

		/** Returns the device role to be used when swapping to the default device. */
		virtual EAudioDeviceRole GetDefaultDeviceRole() const { return EAudioDeviceRole::Console; }
		
	private:

		/** Cache for holding information about MM audio devices (IMMDevice).  */
		TUniquePtr<IAudioPlatformDeviceInfoCache> DeviceInfoCache;
		
		/** Manages either a single, default device or an aggregate of several devices belonging to the same hardware */
		TUniquePtr<IAudioMixerWasapiDeviceManager> DeviceManager;

		/** Indicates if this object has been successfully initialized. */
		bool bIsInitialized = false;

		/** True while the Windows audio service is believed to be alive.
		 *  Writers (publish-only, never take DeviceSwapCriticalSection):
		 *    - FAudioMixerWasapi::OnSessionDisconnect(ServerShutdown) on COM session-events thread
		 *    - FAudioMixerWasapi::OnDeviceRemoved / OnDeviceStateChanged on COM device-events thread
		 *      (gated on IsAudsrvCurrentlyRunning() SCM probe)
		 *    - MarkAudioServiceAvailable() on the SCM watcher thread (from OnAudioServiceRestarted)
		 *  Consumers:
		 *    - FAudioMixerWasapi::CheckThreadedDeviceSwap on the audio thread - parks on null
		 *      renderer when false, gates recovery swaps when true. */
		std::atomic<bool> bAudioServiceAvailable = true;

		/** Watches Audiosrv / AudioEndpointBuilder for SCM state transitions. Started in
		 *  InitializeHardware (gated on au.AudioMixerWasapi.AudsrvWatcher.Enabled), stopped
		 *  in TeardownHardware. Calls OnAudioServiceRestarted on its thread when both
		 *  services return to RUNNING after either was observed non-RUNNING. */
		TUniquePtr<FWasapiServiceWatcher> ServiceWatcher;

		/** Device swap context which holds necessary data required to perform a device wap. */
		TUniquePtr<FWasapiDeviceSwapContext> DeviceSwapContext;
		
		/** Fetches an IMMDevice with the given ID. May invalidate or refresh the device info cache when the OS reports the ID unresolvable . */
		TComPtr<IMMDevice> GetMMDevice(const FString& InDeviceID) const;

		/** Recovery dispatch when a cached IAudioPlatformDeviceInfoCache entry is out of sync with
		 *  the OS: either GetMMDevice could not resolve the ID, or the render thread observed
		 *  AUDCLNT_E_DEVICE_INVALIDATED / AUDCLNT_E_SERVICE_NOT_RUNNING. Governed by
		 *  au.AudioMixerWasapi.{Refresh,Invalidate}DeviceOnGetFailure. No-op if no cache is in use. */
		void UpdateCacheFromFailedDevice(const FString& InDeviceID) const;

		/** Extracts the hardware device name from a logical device name. The OS
		 *  places the hardware name in parentheses at the end of the string).
		 */
		FString ExtractAggregateDeviceName(const FString& InDeviceID) const;
		
		/** Device manager factory */
		static void CreateDeviceManager(const bool bInUseAggregateDevice, TUniquePtr<IAudioMixerWasapiDeviceManager>& InDeviceManager);
		
		/** Initializes a Wasapi stream parameters struct with the give values. */
		bool InitStreamParams(const uint32 InDeviceIndex, const int32 InNumBufferFrames, const int32 InNumBuffers, const int32 InSampleRate, TArray<FWasapiRenderStreamParams>& OutParams);
		bool InitStreamParams(const FAudioPlatformDeviceInfo& InDeviceInfo, const int32 InNumBufferFrames, const int32 InNumBuffers, const int32 InSampleRate, TArray<FWasapiRenderStreamParams>& OutParams) const;
		
		/** Performs a device swap with the given context. Static method enforces no other state sharing occurs. */
		static TUniquePtr<FDeviceSwapResult> PerformDeviceSwap(TUniquePtr<FWasapiDeviceSwapContext>&& InDeviceContext);

#if PLATFORM_WINDOWS
		/** Register with the Windows MM Notification Client for updates */
		static void RegisterForSessionEvents(const FString& InDeviceId);
		static void UnregisterForSessionEvents();
#endif //PLATFORM_WINDOWS


	};

 }
