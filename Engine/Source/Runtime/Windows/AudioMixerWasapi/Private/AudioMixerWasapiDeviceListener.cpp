// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerWasapi.h"
#include "AudioMixer.h"
#include "AudioDeviceNotificationSubsystem.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeRWLock.h"

#include <atomic>

#include "AudioDeviceManager.h"
#include "Microsoft/COMPointer.h"
#include "ScopedCom.h"					// FScopedComString

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
// Linkage for  Windows GUIDs included by Notification/DeviceInfoCache, otherwise they are extern.
#include <initguid.h>
#if PLATFORM_WINDOWS
#include <winsvc.h>		// OpenSCManagerW / QueryServiceStatusEx
#endif
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "WindowsMMNotificationClient.h"
#include "WindowsMMDeviceInfoCache.h"
#include "WindowsMMStringUtils.h"

// Device invalidation and refresh cvars exist in order to keep the system from retrying to swap
// to a device that exists in the cache but repeatedly fails. The observable symptoms of this are
// generally an OOM preceded by multiple failed attemps to start the same audio device. It is 
// unclear whether this is due to an OS/Wasapi internal issue or whether our device cache has
// simply become stale due to race conditions or logical code errors. Two solutions are added here,
// invalidating a single device OR refreshing the device cache. They are controlled via CVAR and
// should be configured to do the most robust fix. The crashes are rare enough that it is not
// practical to repro locally, and so the best decision for cvar configuration will be determined
// by analysis of the crash logs coming in from the field. 
//
// Recovery dispatch shared by two paths that observe a cached IAudioPlatformDeviceInfoCache entry
// is out of sync with the OS: (1) FAudioMixerWasapi::GetMMDevice when the OS cannot resolve a
// cached device ID, and (2) FAudioMixerWasapi::CheckThreadedDeviceSwap when the render thread
// reports AUDCLNT_E_DEVICE_INVALIDATED / AUDCLNT_E_SERVICE_NOT_RUNNING. Two independent
// kill-switches; Refresh wins when both are enabled because re-enumerating from the OS naturally
// evicts the stale entry and also rebuilds DefaultRenderId / DefaultCaptureId.
//
//   InvalidateDevice  Refresh  Behaviour
//   ----------------  -------  ---------------------------------------------------
//          1             1     Refresh (the bigger hammer wins)
//          0             1     Refresh
//          1             0     Invalidate the failing entry only
//          0             0     Off (do not update cache)
//
static int32 bWasapiInvalidateDeviceOnGetFailureCVar = 1;
FAutoConsoleVariableRef CVarWasapiInvalidateDeviceOnGetFailure(
	TEXT("au.AudioMixerWasapi.InvalidateDeviceOnGetFailure"),
	bWasapiInvalidateDeviceOnGetFailureCVar,
	TEXT("When 1 (default) and RefreshCacheOnGetFailure is 0, FAudioMixerWasapi recovery paths ")
	TEXT("(GetMMDevice and render-thread device-loss detection) flip the failing device's cache entry ")
	TEXT("from Active to NotPresent so the next swap retry skips it. Set to 0 to disable per-device ")
	TEXT("invalidation. Has no effect when RefreshCacheOnGetFailure is 1 (Refresh subsumes invalidation)."),
	ECVF_Default);

static int32 bWasapiRefreshCacheOnGetFailureCVar = 0;
FAutoConsoleVariableRef CVarWasapiRefreshCacheOnGetFailure(
	TEXT("au.AudioMixerWasapi.RefreshCacheOnGetFailure"),
	bWasapiRefreshCacheOnGetFailureCVar,
	TEXT("When 1, recovery paths trigger a full IAudioPlatformDeviceInfoCache::Refresh on device ")
	TEXT("failure. Default 0: the full Refresh holds the cache lock across COM enumeration and can ")
	TEXT("stall the game thread, so recovery uses a cheap per-device re-read instead."),
	ECVF_Default);

namespace Audio
{
#if PLATFORM_WINDOWS
	namespace
	{
		// SCM probe to distinguish audsrv death (taskkill, crash) from user-initiated device
		// changes. Returns true on probe failure so a transient SCM hiccup doesn't park audio.
		bool IsAudsrvCurrentlyRunning()
		{
			SC_HANDLE Scm = OpenSCManagerW(nullptr, nullptr, SC_MANAGER_CONNECT);
			if (!Scm)
			{
				return true;
			}

			SC_HANDLE Service = OpenServiceW(Scm, L"Audiosrv", SERVICE_QUERY_STATUS);
			if (!Service)
			{
				CloseServiceHandle(Scm);
				return true;
			}

			SERVICE_STATUS_PROCESS Status{};
			DWORD BytesNeeded = 0;
			const BOOL bQueried = QueryServiceStatusEx(Service, SC_STATUS_PROCESS_INFO,
				reinterpret_cast<LPBYTE>(&Status), sizeof(Status), &BytesNeeded);

			CloseServiceHandle(Service);
			CloseServiceHandle(Scm);

			return !bQueried || Status.dwCurrentState == SERVICE_RUNNING;
		}
	}
#endif // PLATFORM_WINDOWS

	TSharedPtr<FWindowsMMNotificationClient> WasapiWinNotificationClient;

#if PLATFORM_WINDOWS
	void FAudioMixerWasapi::RegisterForSessionEvents(const FString& InDeviceId)
	{
		if (WasapiWinNotificationClient)
		{
			WasapiWinNotificationClient->RegisterForSessionNotifications(InDeviceId);
		}
	}
	void FAudioMixerWasapi::UnregisterForSessionEvents()
	{
		if (WasapiWinNotificationClient)
		{
			WasapiWinNotificationClient->UnregisterForSessionNotifications();
		}
	}
#endif //PLATFORM_WINDOWS

	void FAudioMixerWasapi::RegisterDeviceChangedListener()
	{
		if (!WasapiWinNotificationClient.IsValid())
		{
			// Shared (This is a COM object, so we don't delete it, just decrement the ref counter).
			WasapiWinNotificationClient = TSharedPtr<FWindowsMMNotificationClient>(
				new FWindowsMMNotificationClient, 
				[](FWindowsMMNotificationClient* InPtr) { InPtr->ReleaseClient(); }
			);
		}
		if (!DeviceInfoCache.IsValid())
		{
			// Wasapi backend supports aggregate devices, provided it is enabled in FAudioDeviceManager
			bool bIsAggregateDeviceSupported = FAudioDeviceManager::IsAggregateDeviceSupportEnabled();

			// Setup device info cache.
			DeviceInfoCache = MakeUnique<FWindowsMMDeviceCache>(bIsAggregateDeviceSupported);
			WasapiWinNotificationClient->RegisterDeviceChangedListener(static_cast<FWindowsMMDeviceCache*>(DeviceInfoCache.Get()));
		}

		WasapiWinNotificationClient->RegisterDeviceChangedListener(this);
	}

	void FAudioMixerWasapi::UnregisterDeviceChangedListener()
	{
		if (WasapiWinNotificationClient.IsValid())
		{
			if (DeviceInfoCache.IsValid())
			{
				// Unregister and kill cache.
				WasapiWinNotificationClient->UnRegisterDeviceDeviceChangedListener(static_cast<FWindowsMMDeviceCache*>(DeviceInfoCache.Get()));
				
				DeviceInfoCache.Reset();
			}
			
			WasapiWinNotificationClient->UnRegisterDeviceDeviceChangedListener(this);
		}
	}

	void FAudioMixerWasapi::OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
	{
		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifySubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifySubsystem->OnDefaultCaptureDeviceChanged(InAudioDeviceRole, DeviceId);
		}
	}

	void FAudioMixerWasapi::OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
	{		
		// There's 3 defaults in windows (communications, console, multimedia). These technically can all be different devices.		
		// However, the Windows UX only allows console+multimedia to be toggle as a pair. This means you get two notifications
		// for default device changing typically. To prevent a trouble trigger we only listen to "Console" here. For more information on 
		// device roles: https://docs.microsoft.com/en-us/windows/win32/coreaudio/device-roles

		// Note: On XBox, headphones plugged into an XBox controller give a device notification on the Communications role. However, you 
		// should not swap to a device for a role other than the default Console, so we re-swap to the Console default here and let
		// the Xbox OS take care of the routing details. 
		
		if (InAudioDeviceRole == EAudioDeviceRole::Console || InAudioDeviceRole == EAudioDeviceRole::Communications)
		{
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi: Changing default audio render device to new device: Role=%ls, DeviceName=%ls, InstanceID=%d", 
				Audio::ToString(InAudioDeviceRole), *WasapiWinNotificationClient->GetFriendlyName(DeviceId), InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.

			FString DeviceIdForSwap;
#if PLATFORM_WINDOWS
			DeviceIdForSwap = DeviceId;
#else
			if (IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
			{
				DeviceIdForSwap = Cache->GetDefaultOutputDevice(GetDefaultDeviceRole()).ToString();
			}
#endif // PLATFORM_WINDOWS
			
			if (GetIsListeningForDeviceEvents())
			{
				RequestDeviceSwap(DeviceIdForSwap, /* force */true, TEXT("FAudioMixerWasapi::OnDefaultRenderDeviceChanged"));
			}
		}

		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifSubsystem->OnDefaultRenderDeviceChanged(InAudioDeviceRole, DeviceId);
		}
	}

	void FAudioMixerWasapi::OnDeviceAdded(const FString& DeviceId, bool bIsRenderDevice)
	{
		// Ignore changes in capture device.
		if (!bIsRenderDevice)
		{
			return;
		}
		
		// If the device that was added is our original device and our current device is NOT our original device, 
		// move our audio stream to this newly added device.
		const FString AudioDeviceId = GetOriginalAudioDeviceId();
		if (AudioStreamInfo.DeviceInfo.DeviceId != AudioDeviceId && DeviceId == AudioDeviceId)
		{
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi: Original audio device re-added. Moving audio back to original audio device: DeviceName=%ls, bRenderDevice=%d, InstanceID=%d", 
				*WasapiWinNotificationClient->GetFriendlyName(*AudioDeviceId), (int32)bIsRenderDevice, InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.
			if (GetIsListeningForDeviceEvents())
			{
				RequestDeviceSwap(AudioDeviceId, /*force */ true, TEXT("FAudioMixerWasapi::OnDeviceAdded"));
			}
		}

		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifSubsystem->OnDeviceAdded(DeviceId, bIsRenderDevice);
		}
	}

	void FAudioMixerWasapi::OnDeviceRemoved(const FString& DeviceId, bool bIsRenderDevice)
	{
		// Ignore changes in capture device.
		if (!bIsRenderDevice)
		{
			return;
		}
		
		// If the device we're currently using was removed... then switch to the new default audio device.
		if (AudioStreamInfo.DeviceInfo.DeviceId == DeviceId)
		{
			UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi: Audio device removed [%ls], falling back to other windows default device. bIsRenderDevice=%d, InstanceID=%d", 
				*WasapiWinNotificationClient->GetFriendlyName(DeviceId), (int32)bIsRenderDevice, InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.
			if (GetIsListeningForDeviceEvents())
			{
				RequestDeviceSwap(TEXT(""), /* force */ true, TEXT("FAudioMixerWasapi::OnDeviceRemoved"));
			}
#if PLATFORM_WINDOWS
			else if (IsAudioServiceAvailable() && !IsAudsrvCurrentlyRunning())
			{
				// taskkill of audsrv can surface as device removal without OnSessionDisconnect; publish
				// here when SCM confirms the service is down. Audio thread parks on next
				// CheckThreadedDeviceSwap tick - we do not touch swap state from a COM notification thread.
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::OnDeviceRemoved: pinned device [%ls] removed and audsrv is not RUNNING; publishing unavailable flag. InstanceID=%d",
					*DeviceId, InstanceID);
				bAudioServiceAvailable.store(false, std::memory_order_release);
			}
#endif // PLATFORM_WINDOWS
		}

		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifySubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifySubsystem->OnDeviceRemoved(DeviceId, bIsRenderDevice);
		}
	}

	void FAudioMixerWasapi::OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool bIsRenderDevice)
	{
		// Ignore changes in capture device.
		if (!bIsRenderDevice)
		{
			return;
		}

		// If the device we're currently using was removed and it's not the system default... then switch to the new default audio device.
		// If it is the system default device, then OnDefaultRenderDeviceChanged() will be called to handle this.
		if (AudioStreamInfo.DeviceInfo.DeviceId == DeviceId && !AudioStreamInfo.DeviceInfo.bIsSystemDefault &&
			(InState == EAudioDeviceState::Disabled || InState == EAudioDeviceState::NotPresent || InState == EAudioDeviceState::Unplugged))
		{
			UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::OnDeviceStateChanged: Audio device not available [%ls], falling back to other windows default device. InState=%d, bIsRenderDevice=%d, InstanceID=%d", 
				*WasapiWinNotificationClient->GetFriendlyName(DeviceId), (int32)InState, (int32)bIsRenderDevice, InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.
			if (GetIsListeningForDeviceEvents())
			{
				FString DefaultDeviceId;
				if (IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
				{
					const FName DeviceIdName = Cache->GetDefaultOutputDevice(GetDefaultDeviceRole());
					if (DeviceIdName != NAME_None)
					{
						DefaultDeviceId = DeviceIdName.ToString();
					}
				}

				RequestDeviceSwap(DefaultDeviceId, /* force */ true, TEXT("FAudioMixerWasapi::OnDeviceStateChanged"));
			}
#if PLATFORM_WINDOWS
			else if (IsAudioServiceAvailable() && !IsAudsrvCurrentlyRunning())
			{
				// taskkill of audsrv produces this notification without OnSessionDisconnect; publish
				// here when SCM confirms the service is down. Audio thread parks on next
				// CheckThreadedDeviceSwap tick - we do not touch swap state from a COM notification thread.
				UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::OnDeviceStateChanged: pinned device [%ls] went unavailable and audsrv is not RUNNING; publishing unavailable flag. InstanceID=%d",
					*DeviceId, InstanceID);
				bAudioServiceAvailable.store(false, std::memory_order_release);
			}
#endif // PLATFORM_WINDOWS
		}
		
		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifySubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifySubsystem->OnDeviceStateChanged(DeviceId, InState, bIsRenderDevice);
		}
	}
	
	void FAudioMixerWasapi::OnFormatChanged(const FString& InDeviceId, const FFormatChangedData& InFormat)
	{
		bool bShouldSwapToThisDevice = false;

		{
			// Access to device swap context must be protected by DeviceSwapCriticalSection
			FScopeLock Lock(&DeviceSwapCriticalSection);

			// If we are currently swapping and the device we are swapping to is changing format, queue up a new swap
			if (AudioStreamInfo.StreamState == EAudioOutputStreamState::SwappingDevice)
			{
				if (DeviceSwapContext.IsValid())
				{
					if (DeviceSwapContext->NewDevice.IsSet())
					{
						if (DeviceSwapContext->NewDevice->DeviceId.Equals(InDeviceId))
						{
							bShouldSwapToThisDevice = true;
						}
					}
				}
			}
			// if we are trying to change the format of the current live device, force a device swap to refresh
			else if (AudioStreamInfo.DeviceInfo.DeviceId == InDeviceId)
			{
				bShouldSwapToThisDevice = true;
			}
		}

		if (bShouldSwapToThisDevice)
		{
			constexpr bool bForceDeviceSwap = true;
			const FString SwapReason = FString::Printf(TEXT("FAudioMixerWasapi - OnFormatChange for live audio device"));

			// Refresh the newly reformatted device
			RequestDeviceSwap(InDeviceId, bForceDeviceSwap, *SwapReason);
		}
	};

	FString FAudioMixerWasapi::GetDeviceId() const
	{
		return AudioStreamInfo.DeviceInfo.DeviceId;
	}

	TComPtr<IMMDevice> FAudioMixerWasapi::GetMMDevice(const FString& InDeviceID) const
	{
		if (WasapiWinNotificationClient)
		{
			if (TComPtr<IMMDevice> Device = WasapiWinNotificationClient->GetDevice(InDeviceID))
			{
				return Device;
			}

			UE_LOGF(LogAudioMixer, Warning, "FAudioMixerWasapi::GetMMDevice: OS cannot resolve DeviceID='%ls'", *InDeviceID);
			UpdateCacheFromFailedDevice(InDeviceID);
		}

		return TComPtr<IMMDevice>();
	}

	void FAudioMixerWasapi::UpdateCacheFromFailedDevice(const FString& InDeviceID) const
	{
		// Full Refresh holds the cache lock across COM enumeration and can stall the game thread, so
		// it is opt-in (RefreshCacheOnGetFailure); default recovery is a cheap per-device re-read.
		if (IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
		{
			if (bWasapiRefreshCacheOnGetFailureCVar)
			{
				Cache->Refresh();
			}
			else if (bWasapiInvalidateDeviceOnGetFailureCVar && !InDeviceID.IsEmpty())
			{
				// Re-read just the failed endpoint off the cache lock (COM in BuildCacheEntry, then a
				// short map write): present -> Active, gone -> stays NotPresent. Reuses the concrete
				// cache's existing OnDeviceAdded, so no header change is needed.
				FWindowsMMDeviceCache* DeviceCache = static_cast<FWindowsMMDeviceCache*>(Cache);
				DeviceCache->InvalidateDevice(FName(*InDeviceID));

				const double RefreshStartTime = FPlatformTime::Seconds();
				DeviceCache->OnDeviceAdded(InDeviceID, /*bIsRender*/ true);
				UE_LOGF(LogAudioMixer, Display, "FAudioMixerWasapi::UpdateCacheFromFailedDevice: targeted re-read of '%ls' took %.2f ms",
					*InDeviceID, (FPlatformTime::Seconds() - RefreshStartTime) * 1000.0);
			}
		}
	}

	FString FAudioMixerWasapi::ExtractAggregateDeviceName(const FString& InName) const
	{
		return FWindowsMMDeviceCache::ExtractAggregateDeviceName(InName);
	}
}

