// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerWasapiDeviceThread.h"
#include "AudioMixer.h"	
#include "AudioMixerWasapiLog.h"
#include "ScopedCom.h"

namespace Audio
{
	std::atomic<uint32> FAudioMixerWasapiDeviceThread::AudioDeviceThreadCounter = 0;

	FAudioMixerWasapiRunnable::FAudioMixerWasapiRunnable(const TFunction<void()>& InDeviceRenderCallback, HANDLE& OutEventHandle) :
		DeviceRenderCallback(InDeviceRenderCallback)
	{
		if (CreateEventHandles())
		{
			OutEventHandle = EventHandles[0];
		}
	}

	FAudioMixerWasapiRunnable::FAudioMixerWasapiRunnable(const TFunction<void()>& InDeviceRenderCallback, TArray<HANDLE>& OutEventHandles, const int32 InNumRequestedHandles) :
		DeviceRenderCallback(InDeviceRenderCallback)
	{
		if (CreateEventHandles(InNumRequestedHandles))
		{
			OutEventHandles = EventHandles;
		}
	}

	bool FAudioMixerWasapiRunnable::CreateEventHandles(const uint32 InNumRequestedHandles)
	{
		uint32 NumHandles = InNumRequestedHandles;
		if (NumHandles > MAXIMUM_WAIT_OBJECTS)
		{
			NumHandles = MAXIMUM_WAIT_OBJECTS;
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRunnable::CreateEventHandles requested %d handles which exceeds max supported. Clamping to %d. ", InNumRequestedHandles, MAXIMUM_WAIT_OBJECTS);
		}

		EventHandles.SetNumZeroed(NumHandles);

		for (uint32 Index = 0; Index < NumHandles; ++Index)
		{
			// Not using FEvent/FEventWin here because we need access to the raw, platform
			// handle (see SetEventHandler() below).
			HANDLE EventHandle = ::CreateEvent(nullptr, 0, 0, nullptr);
			if (EventHandle == nullptr)
			{
				EventHandles.Reset();
				return false;
			}

			EventHandles[Index] = EventHandle;
		}

		return true;
	}

	uint32 FAudioMixerWasapiRunnable::Run()
	{
		bIsRunning = true;

		FScopedCoInitialize ScopedCoInitialize;

		const uint32 NumHandles = EventHandles.Num();
		const HANDLE* Handles = EventHandles.GetData();

		while (bIsRunning.load())
		{
			static constexpr uint32 TimeoutInMs = 1000;

			// WASAPI events for multiple audio devices will all be signaled at the
			// same time when belonging to the same physical device.
			uint32 Result = ::WaitForMultipleObjects(NumHandles, Handles, true, TimeoutInMs);
			if (Result == WAIT_TIMEOUT)
			{
				++OutputStreamTimeoutsDetected;
			}
			else if (Result >= WAIT_OBJECT_0 && Result < (WAIT_OBJECT_0 + NumHandles))
			{
				DeviceRenderCallback();
			}
		}
		
		return 0;
	}

	void FAudioMixerWasapiRunnable::Stop()
	{
		bIsRunning = false;

		// Signal event handles to wake the thread from WaitForMultipleObjects
		// so it can exit promptly instead of waiting for the 1s timeout.
		// This matters when the device is no longer present (e.g. headphones unplugged)
		// and WASAPI won't signal the event handles anymore.
		for (HANDLE Handle : EventHandles)
		{
			if (Handle != nullptr)
			{
				::SetEvent(Handle);
			}
		}

		if (OutputStreamTimeoutsDetected > 0)
		{
			UE_LOGF(LogAudioMixerWasapi, Display, "FAudioMixerWasapiRunnable::Stop render stream reported %d timeouts", OutputStreamTimeoutsDetected);
		}
	}

	FAudioMixerWasapiDeviceThread::FAudioMixerWasapiDeviceThread(const TFunction<void()>& InDeviceRenderCallback, HANDLE& OutEventHandle) :
		DeviceRenderRunnable(MakeUnique<FAudioMixerWasapiRunnable>(InDeviceRenderCallback, OutEventHandle))
	{
	}

	FAudioMixerWasapiDeviceThread::FAudioMixerWasapiDeviceThread(const TFunction<void()>& InDeviceRenderCallback, TArray<HANDLE>& OutEventHandles, const int32 InNumRequestedHandles) :
		DeviceRenderRunnable(MakeUnique<FAudioMixerWasapiRunnable>(InDeviceRenderCallback, OutEventHandles, InNumRequestedHandles))
	{
	}

	bool FAudioMixerWasapiDeviceThread::Start()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Audio::FAudioMixerWasapiDeviceThread::Start);
		check(DeviceRenderThread == nullptr);

		DeviceRenderThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(DeviceRenderRunnable.Get(), *FString::Printf(TEXT("AudioDeviceThread(%d)"), ++AudioDeviceThreadCounter), 0, TPri_TimeCritical));
		return DeviceRenderThread.IsValid();
	}

	void FAudioMixerWasapiDeviceThread::Stop()
	{
		if (DeviceRenderThread.IsValid())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Audio::FAudioMixerWasapiDeviceThread::Stop);

			bool bShouldWait = true;
			DeviceRenderThread->Kill(bShouldWait);
			DeviceRenderThread.Reset();
		}
	}

	void FAudioMixerWasapiDeviceThread::Abort()
	{
		if (DeviceRenderThread.IsValid())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Audio::FAudioMixerWasapiDeviceThread::Abort);

			// Always wait for thread to complete otherwise we can crash if
			// the stream is disposed of mid-callback.
			bool bShouldWait = true;
			DeviceRenderThread->Kill(bShouldWait);
			DeviceRenderThread.Reset();
		}
	}
}
