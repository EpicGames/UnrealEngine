// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IAudioMixerWasapiDeviceManager.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <winnt.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

namespace Audio
{
	/**
	 * FAudioMixerWasapiRunnable - The runnable which executes the main thread loop for FWasapiCaptureThread.
	 */
	class FAudioMixerWasapiRunnable : public FRunnable
	{
	public:
		FAudioMixerWasapiRunnable() = delete;

		/**
		 * FAudioMixerWasapiRunnable
		 */
		FAudioMixerWasapiRunnable(const TFunction<void()>& InDeviceRenderCallback, HANDLE& OutEventHandle);

		/**
		 * FAudioMixerWasapiRunnable
		 */
		FAudioMixerWasapiRunnable(const TFunction<void()>& InDeviceRenderCallback, TArray<HANDLE>& OutEventHandles, const int32 InNumRequestedHandles);

		/** Default destructor */
		virtual ~FAudioMixerWasapiRunnable() = default;

		// Begin FRunnable overrides
		virtual uint32 Run() override;
		virtual void Stop() override;
		// End FRunnable overrides

	private:
		/** The main run loop for this runnable will continue iterating while this flag is true. */
		std::atomic<bool> bIsRunning;
		
		/**
		 * Event handle which our audio thread waits on prior to each callback. WASAPI signals this
		 * object each quanta when a buffer of audio has been rendered and is ready for more data.
		 */
		TArray<HANDLE> EventHandles;

		/**
		 * Accumulates timeouts which occur when the thread event timeout is reached
		 * prior to the event being signaled for new data being available.
		 */
		uint32 OutputStreamTimeoutsDetected = 0;
		
		/** Callback function to be called each time the device signals it is ready for another buffer of audio. */
		TFunction<void()> DeviceRenderCallback;

		bool CreateEventHandles(const uint32 InNumRequestedHandles = 1);
	};

	/**
	 * FAudioMixerWasapiDeviceThread - Manages both the FAudioMixerWasapiRunnable object and the thread whose context it runs in. 
	 */
	class FAudioMixerWasapiDeviceThread
	{
	public:
		FAudioMixerWasapiDeviceThread() = delete;

		/**
		 * FAudioMixerWasapiDeviceThread
		 */
		FAudioMixerWasapiDeviceThread(const TFunction<void()>& InDeviceRenderCallback, HANDLE& OutEventHandle);

		/**
		 * FAudioMixerWasapiDeviceThread
		 */
		FAudioMixerWasapiDeviceThread(const TFunction<void()>& InDeviceRenderCallback, TArray<HANDLE>& OutEventHandles, const int32 InNumRequestedHandles);

		/**
		 * Start - Creates the FRunnableThread object which immediately begins running the FAudioMixerWasapiRunnable member.
		 * 
		 * @return - Boolean indicating of the thread was successfully created.
		 */
		bool Start();

		/**
		 * Stop - Gracefully shuts down the thread.
		 */
		void Stop();
		
		/**
		 * Abort - Performs non-graceful shutdown of thread which will close the underlying thread handle 
		 * without waiting for it to finish.
		 */
		void Abort();

	private:
		/** Tracks number of device threads created. */
		static std::atomic<uint32> AudioDeviceThreadCounter;

		/** The thread which is the context that the runnable executes in. */
		TUniquePtr<FRunnableThread> DeviceRenderThread;

		/** The runnable which manages the run loop for the render stream. */
		TUniquePtr<FAudioMixerWasapiRunnable> DeviceRenderRunnable;
	};
}
