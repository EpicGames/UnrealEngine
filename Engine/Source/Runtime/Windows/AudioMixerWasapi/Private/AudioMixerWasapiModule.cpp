// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerWasapiModule.h"
#include "AudioMixerWasapi.h"
#include "AudioMixerWasapiLog.h"
#include "Features/IModularFeatures.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAudioMixerWasapi);

void FAudioMixerModuleWasapi::StartupModule()
{
	IAudioDeviceModule::StartupModule();

	FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixer"));
	FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
	FModuleManager::Get().LoadModuleChecked(TEXT("WindowsMMDeviceEnumeration"));
}

bool FAudioMixerModuleWasapi::IsAudioMixerModule() const
{
	return true;
}

Audio::IAudioMixerPlatformInterface* FAudioMixerModuleWasapi::CreateAudioMixerPlatformInterface()
{
	return new Audio::FAudioMixerWasapi();
}

#if PLATFORM_WINDOWS
IMPLEMENT_MODULE(FAudioMixerModuleWasapi, AudioMixerWasapi);
#endif //PLATFORM_WINDOWS
