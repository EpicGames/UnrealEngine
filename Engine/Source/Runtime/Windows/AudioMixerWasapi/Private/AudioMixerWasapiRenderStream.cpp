// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerWasapiRenderStream.h"

#include "AudioMixerWasapiLog.h"
#include "HAL/IConsoleManager.h"
#include "WasapiAudioUtils.h"
#include "WindowsMMStringUtils.h"

static int32 UseDefaultQualitySRC_CVar = 0;
FAutoConsoleVariableRef CVarUseDefaultQualitySRC(
	TEXT("au.Wasapi.UseDefaultQualitySRC"),
	UseDefaultQualitySRC_CVar,
	TEXT("Enable Wasapi default SRC quality.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

namespace Audio
{
	uint32 FAudioMixerWasapiRenderStream::GetMinimumBufferSize(const uint32 InSampleRate)
	{
		// Can be called prior to InitializeHardware
		// Makes assumption about minimum buffer size which we verify in InitializeHardware
		return InSampleRate / 100;
	}

	FAudioMixerWasapiRenderStream::FAudioMixerWasapiRenderStream()
	{
	}

	FAudioMixerWasapiRenderStream::~FAudioMixerWasapiRenderStream()
	{
	}

	bool FAudioMixerWasapiRenderStream::InitializeHardware(const FWasapiRenderStreamParams& InParams)
	{
		TComPtr<IMMDevice> MMDevice = InParams.MMDevice;

		if (!MMDevice)
		{
			UE_LOGF(LogAudioMixerWasapi, Error, "InitializeHardware null MMDevice");
			return false;
		}

		TComPtr<IAudioClient2> TempAudioClient;
		HRESULT Result = MMDevice->Activate(__uuidof(IAudioClient2), CLSCTX_INPROC_SERVER, nullptr, IID_PPV_ARGS_Helper(&TempAudioClient));
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioMixerWasapi, Error, "OpenAudioStream failed IMMDevice::Activate %ls", *AudioClientErrorToFString(Result));
			return false;
		}

#if PLATFORM_WINDOWS
		AudioClientProperties ClientProperties = {};
		ClientProperties.cbSize = sizeof(AudioClientProperties);
		ClientProperties.bIsOffload = false;
		ClientProperties.eCategory = AudioCategory_GameEffects;
		ClientProperties.Options = AUDCLNT_STREAMOPTIONS_RAW;

		// Set the stream type to RAW in order to avoid system audio enhancements
		Result = TempAudioClient->SetClientProperties(&ClientProperties);
		if (FAILED(Result))
		{
			// Not all clients support raw mode. Log warning and continue.
			UE_LOGF(LogAudioMixerWasapi, Warning, "InitializeHardware failed SetClientProperties %ls", *AudioClientErrorToFString(Result));
		}
#endif //PLATFORM_WINDOWS
		
		WAVEFORMATEX* MixFormat = nullptr;
		Result = TempAudioClient->GetMixFormat(&MixFormat);
		if (FAILED(Result) || !MixFormat)
		{
			UE_LOGF(LogAudioMixerWasapi, Error, "OpenAudioStream failed IAudioClient::GetMixFormat MixFormat: 0x%llx Result: %ls", (uint64)MixFormat, *AudioClientErrorToFString(Result));
			return false;
		}

		FWasapiAudioFormat StreamFormat(FMath::Min<int32>(MixFormat->nChannels, AUDIO_MIXER_MAX_OUTPUT_CHANNELS), InParams.SampleRate, EWasapiAudioEncoding::FLOATING_POINT_32);

		if (MixFormat)
		{
			::CoTaskMemFree(MixFormat);
			MixFormat = nullptr;
		}

		REFERENCE_TIME DevicePeriodRefTime = 0;
		// The second param to GetDevicePeriod is only valid for exclusive mode
		// Note that GetDevicePeriod returns ref time which is sample rate agnostic
		// In testing, IAudioClient3::GetSharedModeEnginePeriod() appears to return the same value as
		// IAudioClient::GetDevicePeriod() so we use the older API.
		Result = TempAudioClient->GetDevicePeriod(&DevicePeriodRefTime, nullptr);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioMixerWasapi, Error, "OpenAudioStream failed IAudioClient::GetDevicePeriod %ls", *AudioClientErrorToFString(Result));
			return false;
		}

		DefaultDevicePeriod = FWasapiAudioUtils::RefTimeToFrames(DevicePeriodRefTime, InParams.SampleRate);
		if (DefaultDevicePeriod == 0)
		{
			UE_LOGF(LogAudioMixerWasapi, Error, "OpenAudioStream failed DefaultDevicePeriod = %d", DefaultDevicePeriod);
			return false;
		}

		// Determine buffer size to use. 
		bool bRoundUpFramesToRequest = false;
		uint32 BufferFramesToRequest = InParams.NumFrames;

		if (BufferFramesToRequest < DefaultDevicePeriod)
		{
			if (DefaultDevicePeriod % BufferFramesToRequest == 0)
			{
				// For example, DefaultDevicePeriod = 480, BufferFramesToRequest = 240
				// no need to round up, just use the default device period
				BufferFramesToRequest = DefaultDevicePeriod;
			}
			else
			{
				// For example, DefaultDevicePeriod = 512, BufferFramesToRequest = 240
				// non-integral buffer size requested, round up to ensure there are always enough frames buffered
				bRoundUpFramesToRequest = true;
			}
		}
		else if (BufferFramesToRequest > DefaultDevicePeriod && (BufferFramesToRequest % DefaultDevicePeriod != 0))
		{
			// For example, DefaultDevicePeriod = 480, BufferFramesToRequest = 1024
			// non-integral buffer size requested, round up to ensure there are always enough frames buffered
			bRoundUpFramesToRequest = true;
		}
		
		// If the engine buffer size is not an integral multiple of the device period then we must
		// account for buffer phasing by padding the requested buffer size.
		if (bRoundUpFramesToRequest)
		{
			// Round up to nearest multiple of the device period
			uint32 Multiple = FMath::CeilToInt32(static_cast<float>(BufferFramesToRequest + DefaultDevicePeriod - 1) / DefaultDevicePeriod);
			BufferFramesToRequest = DefaultDevicePeriod * Multiple;
		}
		REFERENCE_TIME DesiredBufferDuration = FWasapiAudioUtils::FramesToRefTime(BufferFramesToRequest, InParams.SampleRate);
		
		UE_LOGF(LogAudioMixerWasapi, Log, "FAudioMixerWasapiRenderStream::InitializeHardware DefaultDevicePeriod: %d requesting buffer size: %d", DefaultDevicePeriod, BufferFramesToRequest);

		// For shared mode, this is required to be zero
		constexpr REFERENCE_TIME Periodicity = 0;

		// Audio events will be delivered to us rather than needing to poll
		uint32 Flags = AUDCLNT_STREAMFLAGS_EVENTCALLBACK;

		if (InParams.SampleRate != InParams.HardwareDeviceInfo.SampleRate)
		{
			Flags |= AUDCLNT_STREAMFLAGS_AUTOCONVERTPCM;
			if (UseDefaultQualitySRC_CVar)
			{
				Flags |= AUDCLNT_STREAMFLAGS_SRC_DEFAULT_QUALITY;
			}
			
			UE_LOGF(LogAudioMixerWasapi, Warning, "Sample rate mismatch. Engine sample rate: %d Device sample rate: %d", InParams.SampleRate, InParams.HardwareDeviceInfo.SampleRate);
			UE_LOGF(LogAudioMixerWasapi, Warning, "Device level sample rate conversion will be used.");
		}

		Result = TempAudioClient->Initialize(AUDCLNT_SHAREMODE_SHARED, Flags, DesiredBufferDuration, Periodicity, StreamFormat.GetWaveFormat(), nullptr);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioMixerWasapi, Error, "OpenAudioStream failed IAudioClient::Initialize %ls", *AudioClientErrorToFString(Result));
			return false;
		}

		Result = TempAudioClient->GetBufferSize(&NumFramesPerDeviceBuffer);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioMixerWasapi, Error, "OpenAudioStream failed IAudioClient::GetBufferSize %ls", *AudioClientErrorToFString(Result));
			return false;
		}

		// HardwareDeviceInfo is somewhat of a misnomer, HardwareDeviceInfo.NumChannels is the number of channels
		// that the AudioMixer is requesting. StreamFormat is fetched from the hardware and represents the 
		// actual number of output channels on the hardware device.
		const int32 AudioMixerNumChannels = InParams.HardwareDeviceInfo.NumChannels;
		const int32 RenderStreamHardwareChannels = StreamFormat.GetNumChannels();
		UE_LOGF(LogAudioMixerWasapi, Display, "FAudioMixerWasapiRenderStream::InitializeHardware audio mixer requested num channels: %d, Wasapi render stream hardware supports num channels: %d", AudioMixerNumChannels, RenderStreamHardwareChannels);

		if (AudioMixerNumChannels != RenderStreamHardwareChannels)
		{
			InitializeDownmixBuffers(AudioMixerNumChannels, InParams.NumFrames, StreamFormat);
		}
		
		AudioClient = MoveTemp(TempAudioClient);
		AudioFormat = StreamFormat;
		RenderStreamParams = InParams;

		bIsInitialized.store(true, std::memory_order_release);

		UE_LOGF(LogAudioMixerWasapi, Display, "FAudioMixerWasapiRenderStream::InitializeHardware succeeded with sample rate: %d, buffer period: %d", InParams.SampleRate, InParams.NumFrames);

		return true;
	}

	bool FAudioMixerWasapiRenderStream::TeardownHardware()
	{
		if (!bIsInitialized.load(std::memory_order_acquire))
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::TeardownHardware failed...not initialized. ");
			return false;
		}

		RenderClient.Reset();
		AudioClient.Reset();

		bIsInitialized.store(false, std::memory_order_release);

		UE_LOGF(LogAudioMixerWasapi, Verbose, "FAudioMixerWasapiRenderStream::TeardownHardware succeeded");

		return true;
	}

	bool FAudioMixerWasapiRenderStream::IsInitialized() const
	{
		return bIsInitialized.load(std::memory_order_acquire);
	}

	void FAudioMixerWasapiRenderStream::InitializeDownmixBuffers(const int32 InNumInputChannels, const int32 InNumFrames, const FWasapiAudioFormat& InAudioOutputFormat)
	{
		UE_LOGF(LogAudioMixerWasapi, Display, "FAudioMixerWasapiRenderStream: Initializing mono downmix buffer");

		// The downmixing code assumes float audio buffers
		check(InAudioOutputFormat.GetEncoding() == EWasapiAudioEncoding::FLOATING_POINT_32);

		int32 StreamFormatNumChannels = InAudioOutputFormat.GetNumChannels();

		DownmixScratchBuffer.Reset();
		DownmixScratchBuffer.AddUninitialized(InNumFrames * StreamFormatNumChannels);

		MixdownGainsMap.Reset();

		// There is a special case where the audio mixer is initialized to 2 channels but the OS is configured to use 
		// a headphone spatializer.In this case, the callback requests 8 channels of audio despite originally telling 
		// us that we're 2 channels. So we need to upmix our audio mixer's render to 8 channels.
		if (InNumInputChannels == 2 && StreamFormatNumChannels == 8)
		{
			MixdownGainsMap = { 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 
								0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f };
		}
		else
		{
			// otherwise, just make this a true mono signal
			const float DownmixGain = 1.0f / StreamFormatNumChannels;

			// Gain values used when downmixing from input channel count to mono
			for (int32 Index = 0; Index < InNumInputChannels * StreamFormatNumChannels; ++Index)
			{
				MixdownGainsMap.Add(DownmixGain);
			}
		}
	}

	int32 FAudioMixerWasapiRenderStream::GetNumFrames(const int32 InNumRequestedFrames) const
	{
		return InNumRequestedFrames;
	}

	bool FAudioMixerWasapiRenderStream::OpenAudioStream(const FWasapiRenderStreamParams& InParams, HANDLE InEventHandle)
	{
		EAudioOutputStreamState::Type CurrStreamState = StreamState.load(std::memory_order_acquire);
		if (CurrStreamState != EAudioOutputStreamState::Closed)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::OpenAudioStream stream not in closed sate. CurrStreamState: %d", CurrStreamState);
			return false;
		}
		
		if (InParams.HardwareDeviceInfo.DeviceId != RenderStreamParams.HardwareDeviceInfo.DeviceId)
		{
			if (!InitializeHardware(InParams))
			{
				UE_LOGF(LogAudioMixerWasapi, Warning, "OpenAudioStream failed InitAudioClient");
				return false;
			}
		}

		if (InEventHandle == nullptr)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "OpenAudioStream null EventHandle");
			return false;
		}

		HRESULT Result = AudioClient->SetEventHandle(InEventHandle);
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "OpenAudioStream failed IAudioClient::SetEventHandle %ls", *AudioClientErrorToFString(Result));
			return false;
		}

		TComPtr<IAudioRenderClient> TempRenderClient;
		Result = AudioClient->GetService(__uuidof(IAudioRenderClient), IID_PPV_ARGS_Helper(&TempRenderClient));
		if (FAILED(Result))
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "OpenAudioStream failed IAudioClient::GetService IAudioRenderClient %ls", *AudioClientErrorToFString(Result));
			return false;
		}

		RenderClient = MoveTemp(TempRenderClient);

		// It's the caller's responsibility to maintain thread safety. This class uses compare_exchange() calls to detect any state errors and 
		// bubble them up the chain.
		const EAudioOutputStreamState::Type OriginalState = CurrStreamState;
		if (!ensure(StreamState.compare_exchange_strong(CurrStreamState, EAudioOutputStreamState::Open, std::memory_order_release, std::memory_order_relaxed)))
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::OpenAudioStream StreamState modified during open. OriginalState: %d CurrStreamState: %d", OriginalState, CurrStreamState);
			return false;
		}
		
		UE_LOGF(LogAudioMixerWasapi, Verbose, "FAudioMixerWasapiRenderStream::OpenAudioStream succeeded with SampeRate: %d, NumFrames: %d", InParams.SampleRate, InParams.NumFrames);

		return true;
	}

	bool FAudioMixerWasapiRenderStream::CloseAudioStream()
	{
		EAudioOutputStreamState::Type CurrStreamState = StreamState.load(std::memory_order_acquire);
		if (!bIsInitialized.load(std::memory_order_acquire) || CurrStreamState == EAudioOutputStreamState::Closed)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::CloseAudioStream stream in unexpected state. bIsInitialized: %d CurrStreamState: %d",
				bIsInitialized.load(std::memory_order_relaxed), CurrStreamState);
			return false;
		}

		if (CurrStreamState == EAudioOutputStreamState::Running)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::CloseAudioStream stream appears to be running. StopAudioStream() must be called prior to closing.");
			return false;
		}

		const EAudioOutputStreamState::Type OriginalState = CurrStreamState;
		if (!ensure(StreamState.compare_exchange_strong(CurrStreamState, EAudioOutputStreamState::Closed, std::memory_order_release, std::memory_order_relaxed)))
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::CloseAudioStream StreamState modified during close. OriginalState: %d CurrStreamState: %d", OriginalState, CurrStreamState);
			return false;
		}

		return true;
	}

	bool FAudioMixerWasapiRenderStream::StartAudioStream()
	{
		EAudioOutputStreamState::Type CurrStreamState = StreamState.load(std::memory_order_acquire);
		if (CurrStreamState != EAudioOutputStreamState::Open)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::StartAudioStream stream not in open state. bIsInitialized: %d, CurrStreamState: %d",
				bIsInitialized.load(std::memory_order_relaxed), CurrStreamState);
			return false;
		}

		if (!AudioClient.IsValid())
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "StartAudioStream failed invalid audio client");
			return false;
		}

		AudioClient->Start();
		UE_LOGF(LogAudioMixerWasapi, Verbose, "FAudioMixerWasapiRenderStream::StartAudioStream stream started");

		const EAudioOutputStreamState::Type OriginalState = CurrStreamState;
		if (!ensure(StreamState.compare_exchange_strong(CurrStreamState, EAudioOutputStreamState::Running, std::memory_order_release, std::memory_order_relaxed)))
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::StartAudioStream StreamState modified during start. OriginalState: %d CurrStreamState: %d", OriginalState, CurrStreamState);
			return false;
		}

		return true;
	}

	bool FAudioMixerWasapiRenderStream::StopAudioStream()
	{
		if (!bIsInitialized.load(std::memory_order_acquire))
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::StopAudioStream() not initialized");
			return false;
		}

		EAudioOutputStreamState::Type CurrStreamState = StreamState.load(std::memory_order_acquire);
		if (CurrStreamState != EAudioOutputStreamState::Stopped && CurrStreamState != EAudioOutputStreamState::Closed)
		{
			if (AudioClient.IsValid())
			{
				AudioClient->Stop();
			}
			
			const EAudioOutputStreamState::Type OriginalState = CurrStreamState;
			if (!ensure(StreamState.compare_exchange_strong(CurrStreamState, EAudioOutputStreamState::Stopped, std::memory_order_release, std::memory_order_relaxed)))
			{
				UE_LOGF(LogAudioMixerWasapi, Warning, "FAudioMixerWasapiRenderStream::StopAudioStream StreamState modified during stop. OriginalState: %d CurrStreamState: %d", OriginalState, CurrStreamState);
				return false;
			}
		}

		if (CallbackBufferErrors > 0)
		{
			UE_LOGF(LogAudioMixerWasapi, Display, "FAudioMixerWasapiRenderStream::StopAudioStream render stream reported %d callback buffer errors (can be normal if preceded by device swap).", CallbackBufferErrors);
			CallbackBufferErrors = 0;
		}

		return true;
	}
}
