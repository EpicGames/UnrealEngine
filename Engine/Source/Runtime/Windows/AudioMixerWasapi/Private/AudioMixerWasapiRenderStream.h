// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"	
#include "IAudioMixerWasapiDeviceManager.h"
#include "WasapiAudioFormat.h"

#include <atomic>

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <mmdeviceapi.h>			// IMMNotificationClient
#include <audioclient.h>			// AUDCLNT_E_* error codes
#include <audiopolicy.h>			// IAudioSessionEvents
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

DECLARE_DELEGATE(FAudioMixerReadNextBufferDelegate);

namespace Audio
{
	/** Returns true for HRESULTs that mean the existing IAudioClient is no longer usable
	 *  and the engine should request a device swap to recover. Defined inline in the header
	 *  so both FWasapiDefaultRenderStream and FWasapiAggregateRenderStream can share one
	 *  definition */
	inline bool IsDeviceLostHResult(HRESULT InResult)
	{
#if PLATFORM_WINDOWS
		return InResult == AUDCLNT_E_DEVICE_INVALIDATED
			|| InResult == AUDCLNT_E_SERVICE_NOT_RUNNING;
#else
		(void)InResult;
		return false;
#endif
	}

	/**
	 * FAudioMixerWasapiRenderStream
	 */
	class FAudioMixerWasapiRenderStream : public IDeviceRenderCallback
	{
	public:

		FAudioMixerWasapiRenderStream();
		virtual ~FAudioMixerWasapiRenderStream();

		virtual bool InitializeHardware(const FWasapiRenderStreamParams& InParams);
		virtual bool TeardownHardware();
		virtual bool IsInitialized() const;
		virtual int32 GetNumFrames(const int32 InNumRequestedFrames) const;
		virtual bool OpenAudioStream(const FWasapiRenderStreamParams& InParams, HANDLE InEventHandle);
		virtual bool CloseAudioStream();
		virtual bool StartAudioStream();
		virtual bool StopAudioStream();
		virtual void SubmitBuffer(const uint8* Buffer, const SIZE_T InNumFrames) { }
		virtual void SubmitDirectOutBuffer(const int32 InChannelIndex, const FAlignedFloatBuffer& InBuffer) { }

		/** Returns true if the render thread has observed AUDCLNT_E_DEVICE_INVALIDATED or
		 *  AUDCLNT_E_SERVICE_NOT_RUNNING from a WASAPI call since the last ClearDeviceLost().
		 *  Set on the WASAPI render thread, polled on the audio thread. */
		bool IsDeviceLost() const { return bDeviceLost.load(std::memory_order_acquire); }

		/** Forces the device-lost kill switch on. Used by the audio thread when parking
		 *  the WASAPI render callback before audsrv has had a chance to fail a call.
		 *  Sets the flag without logging; the parking site logs once. */
		void SetDeviceLost() { bDeviceLost.store(true, std::memory_order_release); }

		/** Clears the device-lost flag. Called by the audio thread after a recovery swap is requested. */
		void ClearDeviceLost() { bDeviceLost.store(false, std::memory_order_release); }

		UE_DEPRECATED(5.7, "GetMinimumBufferSize() is deprecated. Please use GetDefaultDevicePeriod() in the future.")
		static uint32 GetMinimumBufferSize(const uint32 InSampleRate);

	protected:
		
		/** Returns the default callback period for as reported by the device. */
		uint32 GetDefaultDevicePeriod() const { return DefaultDevicePeriod; }

		/** COM pointer to the WASAPI audio client object. */
		TComPtr<IAudioClient2> AudioClient;

		/** COM pointer to the WASAPI render client object. */
		TComPtr<IAudioRenderClient> RenderClient;

		/** Holds the audio format configuration for this stream. */
		FWasapiAudioFormat AudioFormat;

		/** Indicates if this object has been successfully initialized. */
		std::atomic<bool> bIsInitialized = false;

		/** The state of the output audio stream. */
		std::atomic<EAudioOutputStreamState::Type> StreamState = EAudioOutputStreamState::Closed;

		/** Render output device info. */
		FWasapiRenderStreamParams RenderStreamParams;

		/** The default callback period for this WASAPI render device. */
		uint32 DefaultDevicePeriod = 0;

		/** Number of frames of audio data which will be used for each audio callback. This value is 
		    determined by the WASAPI audio client and can be equal or greater than the number of frames requested. */
		uint32 NumFramesPerDeviceBuffer = 0;

		/** Accumulates errors that occur in the audio callback. */
		uint32 CallbackBufferErrors = 0;

		/** Set by the WASAPI render thread when a device-invalidation HRESULT is observed
		 *  (AUDCLNT_E_DEVICE_INVALIDATED, AUDCLNT_E_SERVICE_NOT_RUNNING). Polled by the
		 *  audio thread (FAudioMixerWasapi::CheckThreadedDeviceSwap) to trigger a swap. */
		std::atomic<bool> bDeviceLost = false;

		/** Scratch buffer used for downmixing output. */
		FAlignedFloatBuffer DownmixScratchBuffer;
		
		/** Per-channel gains used when downmixing. */
		TArray<float> MixdownGainsMap;

	private:
		
		/** Initializes the necessary data structures needed for downmixing audio buffers. */
		void InitializeDownmixBuffers(const int32 InNumInputChannels, const int32 InNumFrames, const FWasapiAudioFormat& InAudioOutputFormat);
	};
}
