// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"	
#include "Misc/ScopeRWLock.h"
#include "WasapiAudioFormat.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <mmdeviceapi.h>			// IMMNotificationClient
#include <audiopolicy.h>			// IAudioSessionEvents
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

DECLARE_DELEGATE(FAudioMixerReadNextBufferDelegate);

namespace Audio
{
	/** Defines parameters needed for opening a new audio stream to device. */
	struct FWasapiRenderStreamParams
	{
		/** The audio device to open. */
		TComPtr<IMMDevice> MMDevice;

		/** Hardware device configuration info. */
		FAudioPlatformDeviceInfo HardwareDeviceInfo;

		/** The number of desired audio frames in audio callback. */
		uint32 NumFrames = 0;

		/** The number of buffers to keep enqueued. More buffers increases latency, but can compensate for variable compute availability in audio callbacks on some platforms. */
		int32 NumBuffers = 0;

		/** The desired sample rate */
		uint32 SampleRate = 0;

		FWasapiRenderStreamParams() = default;

		FWasapiRenderStreamParams(const TComPtr<IMMDevice>& InMMDevice,
			const FAudioPlatformDeviceInfo& InDeviceInfo,
			const uint32 InNumFrames,
			const uint32 InNumBuffers,
			const uint32 InSampleRate) :
			MMDevice(InMMDevice),
			HardwareDeviceInfo(InDeviceInfo),
			NumFrames(InNumFrames),
			NumBuffers(InNumBuffers),
			SampleRate(InSampleRate)
		{
		}
	};

	/**
	 * IAudioMixerWasapiDeviceManager - classes implementing this interface manage underlying render device streams
	 */
	class IAudioMixerWasapiDeviceManager
	{
	public:

		IAudioMixerWasapiDeviceManager() = default;
		virtual ~IAudioMixerWasapiDeviceManager() = default;

		virtual bool InitializeHardware(const TArray<FWasapiRenderStreamParams>& InParams, const TFunction<void()>& InCallback) = 0;
		virtual bool TeardownHardware() = 0;
		virtual bool IsInitialized() const = 0;
		virtual int32 GetNumDirectOutChannels() const = 0;
		virtual int32 GetNumFrames(const int32 InNumRequestedFrames) const = 0;
		virtual bool OpenAudioStream(const  TArray<FWasapiRenderStreamParams>& InParams) = 0;
		virtual bool CloseAudioStream() = 0;
		virtual bool StartAudioStream() = 0;
		virtual bool StopAudioStream() = 0;
		virtual void SubmitBuffer(const uint8* InBuffer, const SIZE_T InNumFrames) = 0;
		virtual void SubmitDirectOutBuffer(const int32 InDirectOutIndex, const FAlignedFloatBuffer& InBuffer) = 0;

		/** Returns true if the underlying main render stream observed a device-invalidation HRESULT. */
		virtual bool IsDeviceLost() const = 0;

		/** Forces the device-lost flag on. Called by the audio thread when parking the
		 *  WASAPI render callback ahead of audsrv-driven failure. */
		virtual void SetDeviceLost() = 0;

		/** Clears the device-lost flag on the underlying main render stream. */
		virtual void ClearDeviceLost() = 0;
	};

	/**
	 * IDeviceRenderCallback - Interface for providing a callback from the device render thread.
	 */
	
	class IDeviceRenderCallback
	{
	public:
		virtual ~IDeviceRenderCallback() = default;
		
		virtual void DeviceRenderCallback() = 0;
	};

}
