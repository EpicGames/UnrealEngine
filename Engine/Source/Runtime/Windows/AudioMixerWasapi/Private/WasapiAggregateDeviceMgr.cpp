// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiAggregateDeviceMgr.h"

#include "WasapiDefaultRenderStream.h"
#include "WasapiAggregateRenderStream.h"

namespace Audio
{
	constexpr static int32 MaxDeviceCount = 64;

	FWasapiAggregateDeviceMgr::FWasapiAggregateDeviceMgr()
	{
		RenderStreamDevices.Reserve(MaxDeviceCount);
	}

	bool FWasapiAggregateDeviceMgr::InitializeHardware(const TArray<FWasapiRenderStreamParams>& InParams, const TFunction<void()>& InCallback)
	{
		RenderStreamDevices.Reset();

		for (int32 DeviceIndex = 0; DeviceIndex < InParams.Num(); ++DeviceIndex)
		{
			// The first device is our main device. This is used for rendering the bed channels.
			// The other devices (if any) are used for direct outputs (see ADM plugin).
			if (DeviceIndex == 0)
			{
				TUniquePtr<FWasapiDefaultRenderStream> DefaultRenderStream = MakeUnique<FWasapiDefaultRenderStream>();

				if (DefaultRenderStream.IsValid())
				{
					// The first device is always the main device which also drives the callback
					DefaultRenderStream->OnReadNextBuffer().BindLambda(InCallback);

					RenderStreamDevices.Emplace(MoveTemp(DefaultRenderStream));
				}
			}
			else
			{
				RenderStreamDevices.Emplace(MakeUnique<FWasapiAggregateRenderStream>());
			}

			if (RenderStreamDevices.IsEmpty() || !RenderStreamDevices[DeviceIndex].IsValid())
			{
				UE_LOGF(LogAudioMixer, Error, "FWasapiAggregateDeviceMgr::InitializeHardware failed to create RenderStreamDevice: %d", DeviceIndex);
				return false;
			}

			if (!RenderStreamDevices[DeviceIndex]->InitializeHardware(InParams[DeviceIndex]))
			{
				UE_LOGF(LogAudioMixer, Error, "FWasapiAggregateDeviceMgr::InitializeHardware InitializeHardware failed for RenderStreamDevice: %d", DeviceIndex);
				return false;
			}
		}

		NumChannelsPerDevice = InParams[0].HardwareDeviceInfo.NumChannels;
		// Total channel count including main device and all direct out devices
		const int32 NumDevices = RenderStreamDevices.Num();
		NumDirectOutChannels = (NumDevices > 0) ? (NumChannelsPerDevice * (NumDevices - 1)) : 0;

		bIsInitialized = true;

		return true;
	}

	bool FWasapiAggregateDeviceMgr::TeardownHardware()
	{
		if (!RenderStreamDevices.IsEmpty())
		{
			for (TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
			{
				if (Device.IsValid())
				{
					// Teardown the main device which will also unbind our delegate
					Device->TeardownHardware();
					Device.Reset();
				}
			}

			RenderStreamDevices.Reset();

			bIsInitialized = false;

			return true;
		}

		return false;
	}

	bool FWasapiAggregateDeviceMgr::IsInitialized() const
	{
		return bIsInitialized;
	}

	int32 FWasapiAggregateDeviceMgr::GetNumFrames(const int32 InNumRequestedFrames) const
	{
		if (!RenderStreamDevices.IsEmpty())
		{
			return RenderStreamDevices[0]->GetNumFrames(InNumRequestedFrames);
		}

		return InNumRequestedFrames;
	}

	bool FWasapiAggregateDeviceMgr::OpenAudioStream(const TArray<FWasapiRenderStreamParams>& InParams)
	{
		TArray<HANDLE> EventHandles;
		const TFunction<void()> RenderCallback = [this]() { DeviceRenderCallback(); };

		const uint32 NumRenderDevices = RenderStreamDevices.Num();
		if (NumRenderDevices == 0)
		{
			UE_LOGF(LogAudioMixer, Error, "FWasapiAggregateDeviceMgr::OpenAudioStream no render devices found");
			return false;
		}

		RenderDeviceThread = MakeUnique<FAudioMixerWasapiDeviceThread>(RenderCallback, EventHandles, NumRenderDevices);

		if (!RenderDeviceThread.IsValid())
		{
			UE_LOGF(LogAudioMixer, Error, "Unable to create RenderDeviceThread");
			return false;
		}

		if (EventHandles.Num() != NumRenderDevices)
		{
			UE_LOGF(LogAudioMixer, Error, "OpenAudioStream error creating EventHandles");
			return false;
		}
		
		for (uint32 Index = 0; Index < NumRenderDevices; ++Index)
		{
			if (!RenderStreamDevices[Index]->OpenAudioStream(InParams[Index], EventHandles[Index]))
			{
				UE_LOGF(LogAudioMixer, Error, "OpenAudioStream failed to open main audio device");
				return false;
			}
		}

		return true;
	}

	bool FWasapiAggregateDeviceMgr::CloseAudioStream()
	{
		RenderDeviceThread.Reset();
		
		if (!RenderStreamDevices.IsEmpty())
		{
			bool bDidAllClose = true;

			for (TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
			{
				if (Device.IsValid())
				{
					bDidAllClose |= Device->CloseAudioStream();
				}
			}

			return bDidAllClose;
		}

		return false;
	}

	bool FWasapiAggregateDeviceMgr::StartAudioStream()
	{
		if (RenderStreamDevices.IsEmpty())
		{
			UE_LOGF(LogAudioMixer, Error, "FWasapiAggregateDeviceMgr::StartAudioStream no devices available to start");
			return false;
		}

		for (TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
		{
			if (Device.IsValid())
			{
				if (!Device->StartAudioStream())
				{
					UE_LOGF(LogAudioMixer, Error, "FWasapiAggregateDeviceMgr::StartAudioStream unable to start render device");
					return false;
				}
			}
		}

		if (!RenderDeviceThread->Start())
		{
			UE_LOGF(LogAudioMixer, Error, "FWasapiAggregateDeviceMgr::StartAudioStream failed to start device thread");
			return false;
		}

		return true;
	}

	bool FWasapiAggregateDeviceMgr::StopAudioStream()
	{
		if (RenderDeviceThread.IsValid())
		{
			RenderDeviceThread->Stop();
		}

		for (TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
		{
			if (Device.IsValid())
			{
				Device->StopAudioStream();
			}
		}

		return true;
	}

	void FWasapiAggregateDeviceMgr::SubmitBuffer(const uint8* InBuffer, const SIZE_T InNumFrames)
	{
		if (!RenderStreamDevices.IsEmpty())
		{
			RenderStreamDevices[0]->SubmitBuffer(InBuffer, InNumFrames);
		}
	}

	void FWasapiAggregateDeviceMgr::SubmitDirectOutBuffer(const int32 InDirectOutIndex, const FAlignedFloatBuffer& InBuffer)
	{
		const int32 NumDevices = RenderStreamDevices.Num();

		if (NumChannelsPerDevice > 0 && NumDevices > 1 &&
			InDirectOutIndex >= 0 && InDirectOutIndex < NumDirectOutChannels)
		{
			// The first device is reserved for the main audio out (the bed channels)
			// so we add one here.
			const int32 RenderDeviceIndex = (InDirectOutIndex / NumChannelsPerDevice) + 1;
			const int32 ChannelIndex = InDirectOutIndex % NumChannelsPerDevice;

			if (RenderDeviceIndex > 0 && RenderDeviceIndex < NumDevices)
			{
				RenderStreamDevices[RenderDeviceIndex]->SubmitDirectOutBuffer(ChannelIndex, InBuffer);
			}
		}
	}

	void FWasapiAggregateDeviceMgr::DeviceRenderCallback()
	{
		for (TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
		{
			if (Device.IsValid())
			{
				Device->DeviceRenderCallback();
			}
		}
	}

	bool FWasapiAggregateDeviceMgr::IsDeviceLost() const
	{
		// Any stream observing device loss invalidates the aggregate.
		for (const TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
		{
			if (Device.IsValid() && Device->IsDeviceLost())
			{
				return true;
			}
		}
		return false;
	}

	void FWasapiAggregateDeviceMgr::SetDeviceLost()
	{
		for (TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
		{
			if (Device.IsValid())
			{
				Device->SetDeviceLost();
			}
		}
	}

	void FWasapiAggregateDeviceMgr::ClearDeviceLost()
	{
		for (TUniquePtr<FAudioMixerWasapiRenderStream>& Device : RenderStreamDevices)
		{
			if (Device.IsValid())
			{
				Device->ClearDeviceLost();
			}
		}
	}
}
