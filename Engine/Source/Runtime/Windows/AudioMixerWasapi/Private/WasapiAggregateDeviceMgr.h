// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioMixerWasapiDeviceManager.h"
#include "AudioMixerWasapiDeviceThread.h"
#include "AudioMixerWasapiRenderStream.h"

namespace Audio
{
	/**
	 * FWasapiAggregateDeviceMgr
	 * 
	 * Manages the software device streams associated with a single, physical hardware device. 
	 * This provides the ability to address more than 8 channels of a physical device using the
	 * Windows Audio Streaming API (WASAPI).
	 */
	class FWasapiAggregateDeviceMgr : public IAudioMixerWasapiDeviceManager, public IDeviceRenderCallback
	{
	public:

		FWasapiAggregateDeviceMgr();
		virtual ~FWasapiAggregateDeviceMgr() = default;

		//~ Begin IAudioMixerWasapiDeviceManager
		virtual bool InitializeHardware(const TArray<FWasapiRenderStreamParams>& InParams, const TFunction<void()>& InCallback) override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual int32 GetNumDirectOutChannels() const override { return NumDirectOutChannels; }
		virtual int32 GetNumFrames(const int32 InNumRequestedFrames) const override;
		virtual bool OpenAudioStream(const TArray<FWasapiRenderStreamParams>& InParams) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual void SubmitBuffer(const uint8* InBuffer, const SIZE_T InNumFrames) override;
		virtual void SubmitDirectOutBuffer(const int32 InDirectOutIndex, const FAlignedFloatBuffer& InBuffer) override;
		virtual bool IsDeviceLost() const override;
		virtual void SetDeviceLost() override;
		virtual void ClearDeviceLost() override;
		//~ End IAudioMixerWasapiDeviceManager

		//~ Begin IDeviceRenderCallback
		virtual void DeviceRenderCallback() override;
		//~ End IDeviceRenderCallback

	private:

		bool bIsInitialized = false;

		/** The number of channels per WASAPI deivce (max 8 channels) */
		int32 NumChannelsPerDevice = 0;

		/** The number of direct output channels supported by the physical device. The sum 
		    total channels of all WASAPI audio devices belonging to the physical device, minus the 
			number of main out channels (typically the first 8 channels)
		*/
		int32 NumDirectOutChannels = 0;

		/** The render streams associated with this aggregate device */
		TArray<TUniquePtr<FAudioMixerWasapiRenderStream>> RenderStreamDevices;

		/** The thread which provides an execution context during audio playback. */
		TUniquePtr<FAudioMixerWasapiDeviceThread> RenderDeviceThread;
	};
 }
