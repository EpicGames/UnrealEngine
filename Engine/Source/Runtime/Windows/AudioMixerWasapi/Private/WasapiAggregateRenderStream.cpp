// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiAggregateRenderStream.h"

#include "AudioMixerWasapiLog.h"
#include "DSP/FloatArrayMath.h"
#include "WasapiAudioUtils.h"

namespace Audio
{
	bool FWasapiAggregateRenderStream::InitializeHardware(const FWasapiRenderStreamParams& InParams)
	{
		if (FAudioMixerWasapiRenderStream::InitializeHardware(InParams))
		{
			DirectOutBuffers.Reset();
			DirectOutBuffers.SetNum(InParams.HardwareDeviceInfo.NumChannels);
		
			for (TCircularAudioBuffer<float>& Buffer : DirectOutBuffers)
			{
				int32 NumOutputBuffers = FMath::Max(InParams.NumBuffers, 2);
				int32 BufferCapacity = InParams.NumFrames * NumOutputBuffers;

				Buffer.SetCapacity(BufferCapacity);
			}

			InterleaveBuffers.Reset();
			InterleaveBuffers.SetNum(InParams.HardwareDeviceInfo.NumChannels);

			const uint32 MinBufferSize = NumFramesPerDeviceBuffer;
			const uint32 WriteNumFrames = FMath::Max(MinBufferSize, InParams.NumFrames);

			for (FAlignedFloatBuffer& Buffer : InterleaveBuffers)
			{
				Buffer.SetNumZeroed(WriteNumFrames);
			}

			return true;
		}

		return false;
	}

	bool FWasapiAggregateRenderStream::StartAudioStream()
	{
		FAudioMixerWasapiRenderStream::StartAudioStream();

		return true;
	}

	void FWasapiAggregateRenderStream::SubmitDirectOutBuffer(const int32 InChannelIndex, const FAlignedFloatBuffer& InBuffer)
	{
		if (InChannelIndex >= 0 && InChannelIndex < RenderStreamParams.HardwareDeviceInfo.NumChannels)
		{
			TCircularAudioBuffer<float>& DirectOutBuffer = DirectOutBuffers[InChannelIndex];

			if (InBuffer.Num() == RenderStreamParams.NumFrames)
			{
				DirectOutBuffer.Push(InBuffer.GetData(), InBuffer.Num());
			}
		}
	}

	void FWasapiAggregateRenderStream::DeviceRenderCallback()
	{
		SCOPED_NAMED_EVENT(FWasapiAggregateRenderStream_DeviceRenderCallback, FColor::Blue);

		const int32 NumDirectOutChannels = DirectOutBuffers.Num();
		const EAudioOutputStreamState::Type CurrStreamState = StreamState.load(std::memory_order_acquire);

		if (CurrStreamState == EAudioOutputStreamState::Running && NumDirectOutChannels > 0)
		{
#if PLATFORM_WINDOWS
			// Once we've observed device loss, stop calling into the dead client. The audio thread
			// will see bDeviceLost on its next CheckThreadedDeviceSwap and request a recovery swap.
			// Xbox uses the synchronous swap path and does not poll bDeviceLost, so this guard
			// would silently kill audio on Xbox until an external swap fires - keep it Windows-only.
			if (bDeviceLost.load(std::memory_order_acquire))
			{
				return;
			}
#endif

			uint32 NumFramesPadding = 0;
			HRESULT PaddingResult = AudioClient->GetCurrentPadding(&NumFramesPadding);
			if (IsDeviceLostHResult(PaddingResult))
			{
				if (!bDeviceLost.exchange(true, std::memory_order_acq_rel))
				{
					UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiAggregateRenderStream: device lost in GetCurrentPadding (HRESULT=0x%08x). Requesting swap.", static_cast<uint32>(PaddingResult));
				}
				return;
			}
			if (FAILED(PaddingResult))
			{
				++CallbackBufferErrors;
				return;
			}

			// NumFramesPerDeviceBuffer is the buffer size WASAPI allocated. It is guaranteed to
			// be at least the amount requested. For example, if we request a 1024 frame buffer, WASAPI
			// might allocate a 1056 frame buffer. The padding is subtracted from the allocated amount
			// to determine how much space is available currently in the buffer.
			const int32 OutputSpaceAvailable = NumFramesPerDeviceBuffer - NumFramesPadding;
			// The number of frames rendered by the engine currently in the circular buffer
			const int32 EngineFramesAvailable = DirectOutBuffers[0].Num();

			if (const int32 WriteNumFrames = FMath::Min(EngineFramesAvailable, OutputSpaceAvailable))
			{
				uint8* RenderBufferPtr = nullptr;

				HRESULT GetBufferResult = RenderClient->GetBuffer(WriteNumFrames, &RenderBufferPtr);
				if (SUCCEEDED(GetBufferResult))
				{
					InterleaveOutput((float*)RenderBufferPtr, WriteNumFrames);

					HRESULT Result = RenderClient->ReleaseBuffer(WriteNumFrames, 0 /* flags */);
					if (IsDeviceLostHResult(Result))
					{
						if (!bDeviceLost.exchange(true, std::memory_order_acq_rel))
						{
							UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiAggregateRenderStream: device lost in ReleaseBuffer (HRESULT=0x%08x). Requesting swap.", static_cast<uint32>(Result));
						}
					}
					else if (FAILED(Result))
					{
						++CallbackBufferErrors;
					}
				}
				else if (IsDeviceLostHResult(GetBufferResult))
				{
					if (!bDeviceLost.exchange(true, std::memory_order_acq_rel))
					{
						UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiAggregateRenderStream: device lost in GetBuffer (HRESULT=0x%08x). Requesting swap.", static_cast<uint32>(GetBufferResult));
					}
				}
				else
				{
					++CallbackBufferErrors;
				}
			}
		}
	}

	void FWasapiAggregateRenderStream::InterleaveOutput(float* OutRenderBufferPtr, const uint32 InNumFrames)
	{
		const int32 NumChannels = InterleaveBuffers.Num();
		if (NumChannels <= 0)
		{
			return;
		}
		
		const int32 NumFrames = InterleaveBuffers[0].Num();

		if (NumFrames >= static_cast<int32>(InNumFrames))
		{
			for (int32 Index = 0; Index < DirectOutBuffers.Num(); ++Index)
			{
				// Clear out the interleave buffer
				InterleaveBuffers[Index].Reset();
				InterleaveBuffers[Index].SetNumZeroed(NumFrames);

				if (DirectOutBuffers[Index].Num() >= InNumFrames)
				{
					const int32 NumFramesPopped = DirectOutBuffers[Index].Pop(InterleaveBuffers[Index].GetData(), InNumFrames);
				}
			}

			TArray<const float*> BufferPtrArray;
			BufferPtrArray.Reset(NumChannels);

			for (const FAlignedFloatBuffer& Buffer : InterleaveBuffers)
			{
				const float* BufferPtr = Buffer.GetData();
				BufferPtrArray.Add(BufferPtr);
			}

			const float** InterleaveBufferPtr = BufferPtrArray.GetData();

			ArrayInterleave(InterleaveBufferPtr, OutRenderBufferPtr, InNumFrames, NumChannels);
		}
	}
}
