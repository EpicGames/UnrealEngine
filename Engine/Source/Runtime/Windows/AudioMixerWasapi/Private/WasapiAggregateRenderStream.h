// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerWasapiRenderStream.h"

namespace Audio
{
	/**
	 * FWasapiAggregateRenderStream
	 * A render stream which can be sent to direct output channels which 
	 * are channels of an aggregate device above the first 8 channels which are
	 * reserved for the main output of the editor/game.
	 */
	class FWasapiAggregateRenderStream : public FAudioMixerWasapiRenderStream
	{
	public:

		virtual ~FWasapiAggregateRenderStream() = default;

		//~ Begin FAudioMixerWasapiRenderStream
		virtual bool InitializeHardware(const FWasapiRenderStreamParams& InParams) override;
		virtual bool StartAudioStream() override;
		virtual void SubmitDirectOutBuffer(const int32 InChannelIndex, const FAlignedFloatBuffer& InBuffer) override;
		//~ End FAudioMixerWasapiRenderStream

		//~ Begin IDeviceRenderCallback
		virtual void DeviceRenderCallback() override;
		//~ End IDeviceRenderCallback

	private:

		TArray<FAlignedFloatBuffer> InterleaveBuffers;
		TArray<TCircularAudioBuffer<float>> DirectOutBuffers;

		void InterleaveOutput(float* InRenderBufferPtr, const uint32 InNumFrames);
	};
}
