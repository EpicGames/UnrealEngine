// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiDefaultDeviceMgr.h"

namespace Audio
{
	bool FWasapiDefaultDeviceMgr::InitializeHardware(const TArray<FWasapiRenderStreamParams>& InParams, const TFunction<void()>& InCallback)
	{
		MainRenderStreamDevice = MakeUnique<FWasapiDefaultRenderStream>();
		if (ensure(MainRenderStreamDevice.IsValid()) && InParams.Num() > 0)
		{
			MainRenderStreamDevice->OnReadNextBuffer().BindLambda(InCallback);

			return MainRenderStreamDevice->InitializeHardware(InParams[0]);
		}

		return false;
	}

	bool FWasapiDefaultDeviceMgr::TeardownHardware()
	{ 
		if (MainRenderStreamDevice.IsValid())
		{
			// Teardown the main device which will also unbind our delegate
			MainRenderStreamDevice->TeardownHardware();
			MainRenderStreamDevice.Reset();

			return true;
		}

		return false;
	}

	bool FWasapiDefaultDeviceMgr::IsInitialized() const
	{
		return MainRenderStreamDevice.IsValid() && MainRenderStreamDevice->IsInitialized();
	}

	int32 FWasapiDefaultDeviceMgr::GetNumFrames(const int32 InNumRequestedFrames) const
	{ 
		if (MainRenderStreamDevice.IsValid())
		{
			return MainRenderStreamDevice->GetNumFrames(InNumRequestedFrames);
		}

		return InNumRequestedFrames;
	}

	bool FWasapiDefaultDeviceMgr::OpenAudioStream(const TArray<FWasapiRenderStreamParams>& InParams)
	{ 
		HANDLE EventHandle = nullptr;
		const TFunction<void()> RenderCallback = [this]() { MainRenderStreamDevice->DeviceRenderCallback(); };
		RenderDeviceThread = MakeUnique<FAudioMixerWasapiDeviceThread>(RenderCallback, EventHandle);

		if (!RenderDeviceThread.IsValid())
		{
			UE_LOGF(LogAudioMixer, Error, "Unable to create RenderDeviceThread");
			return false;
		}

		if (EventHandle == nullptr)
		{
			UE_LOGF(LogAudioMixer, Error, "OpenAudioStream null EventHandle");
			return false;
		}
		
		if (!MainRenderStreamDevice->OpenAudioStream(InParams[0], EventHandle))
		{
			UE_LOGF(LogAudioMixer, Error, "OpenAudioStream failed to open main audio device");
			return false;
		}

		return true;
	}

	bool FWasapiDefaultDeviceMgr::CloseAudioStream()
	{
		RenderDeviceThread.Reset();
		
		if (MainRenderStreamDevice.IsValid())
		{
			return MainRenderStreamDevice->CloseAudioStream();
		}

		return false;
	}

	bool FWasapiDefaultDeviceMgr::StartAudioStream()
	{ 
		if (!MainRenderStreamDevice->StartAudioStream())
		{
			UE_LOGF(LogAudioMixer, Error, "FWasapiDefaultDeviceMgr::StartAudioStream failed to start main audio device");
			return false;
		}

		if (!RenderDeviceThread->Start())
		{
			UE_LOGF(LogAudioMixer, Error, "FWasapiDefaultDeviceMgr::StartAudioStream failed to start device thread");
			return false;
		}
		
		return true;
	}

	bool FWasapiDefaultDeviceMgr::StopAudioStream()
	{
		if (RenderDeviceThread.IsValid())
		{
			RenderDeviceThread->Stop();
		}

		if (MainRenderStreamDevice.IsValid())
		{
			MainRenderStreamDevice->StopAudioStream();
		}

		return true;
	}

	void FWasapiDefaultDeviceMgr::SubmitBuffer(const uint8* InBuffer, const SIZE_T InNumFrames)
	{
		if (MainRenderStreamDevice.IsValid())
		{
			MainRenderStreamDevice->SubmitBuffer(InBuffer, InNumFrames);
		}
	}

	bool FWasapiDefaultDeviceMgr::IsDeviceLost() const
	{
		return MainRenderStreamDevice.IsValid() && MainRenderStreamDevice->IsDeviceLost();
	}

	void FWasapiDefaultDeviceMgr::SetDeviceLost()
	{
		if (MainRenderStreamDevice.IsValid())
		{
			MainRenderStreamDevice->SetDeviceLost();
		}
	}

	void FWasapiDefaultDeviceMgr::ClearDeviceLost()
	{
		if (MainRenderStreamDevice.IsValid())
		{
			MainRenderStreamDevice->ClearDeviceLost();
		}
	}
}
