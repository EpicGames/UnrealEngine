// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioMixerWasapiDeviceManager.h"
#include "AudioMixerWasapiDeviceThread.h"
#include "WasapiDefaultRenderStream.h"

namespace Audio
{
	/**
	 * FAudioMixerWasapi 
	 * Manages a single audio device which is used for the main (first eight) output channels.
	 */
	class FWasapiDefaultDeviceMgr : public IAudioMixerWasapiDeviceManager
	{
	public:

		FWasapiDefaultDeviceMgr() = default;
		virtual ~FWasapiDefaultDeviceMgr() = default;

		//~ Begin IAudioMixerWasapiDeviceManager
		virtual bool InitializeHardware(const TArray<FWasapiRenderStreamParams>& InParams, const TFunction<void()>& InCallback) override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual int32 GetNumDirectOutChannels() const override { return 0; }
		virtual int32 GetNumFrames(const int32 InNumRequestedFrames) const override;
		virtual bool OpenAudioStream(const TArray<FWasapiRenderStreamParams>& InParams) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual void SubmitBuffer(const uint8* InBuffer, const SIZE_T InNumFrames) override;
		virtual void SubmitDirectOutBuffer(const int32 InDirectOutIndex, const FAlignedFloatBuffer& InBuffer) override { }
		virtual bool IsDeviceLost() const override;
		virtual void SetDeviceLost() override;
		virtual void ClearDeviceLost() override;
		//~ End IAudioMixerWasapiDeviceManager

	private:
	
		/** The main audio device for outputting up to 8 channels. */
		TUniquePtr<FWasapiDefaultRenderStream> MainRenderStreamDevice;

		/** The thread which provides an execution context during audio playback. */
		TUniquePtr<FAudioMixerWasapiDeviceThread> RenderDeviceThread;
	};
 }
