// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiDefaultRenderStream.h"

#include "AudioMixerWasapiLog.h"
#include "WasapiAudioUtils.h"

namespace Audio
{
	bool FWasapiDefaultRenderStream::InitializeHardware(const FWasapiRenderStreamParams& InParams)
	{
		if (FAudioMixerWasapiRenderStream::InitializeHardware(InParams))
		{
			EngineBufferNumFrames = InParams.NumFrames;
			EngineBufferNumBytes = EngineBufferNumFrames * AudioFormat.GetFrameSizeInBytes();

			return true;
		}
		
		return false;
	}
	
	bool FWasapiDefaultRenderStream::TeardownHardware()
	{
		ReadNextBufferDelegate.Unbind();

		return FAudioMixerWasapiRenderStream::TeardownHardware();
	}

	void FWasapiDefaultRenderStream::DeviceRenderCallback()
	{
		SCOPED_NAMED_EVENT(FWasapiDefaultRenderStream_DeviceRenderCallback, FColor::Blue);

		const EAudioOutputStreamState::Type CurrStreamState = StreamState.load(std::memory_order_acquire);
		if (CurrStreamState == EAudioOutputStreamState::Running)
		{
			// Once we've observed device loss, stop calling into the dead client. The audio thread
			// will see bDeviceLost on its next CheckThreadedDeviceSwap and request a recovery swap.
			if (bDeviceLost.load(std::memory_order_acquire))
			{
				return;
			}

			uint32 NumFramesPadding = 0;
			HRESULT PaddingResult = AudioClient->GetCurrentPadding(&NumFramesPadding);
			if (IsDeviceLostHResult(PaddingResult))
			{
				if (!bDeviceLost.exchange(true, std::memory_order_acq_rel))
				{
					UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiDefaultRenderStream: device lost in GetCurrentPadding (HRESULT=0x%08x). Requesting swap.", static_cast<uint32>(PaddingResult));
				}
				return;
			}
			if (FAILED(PaddingResult))
			{
				++CallbackBufferErrors;
				return;
			}

			// NumFramesPerDeviceBuffer is the buffer size WASAPI allocated. It is guaranteed to
			// be at least the amount requested. For example, if we request a 1024 frame buffer, WASAPI
			// might allocate a 1056 frame buffer. The padding is subtracted from the allocated amount
			// to determine how much space is available currently in the buffer.
			const int32 OutputSpaceAvailable = NumFramesPerDeviceBuffer - NumFramesPadding;

			if (OutputSpaceAvailable >= EngineBufferNumFrames)
			{
				check(RenderBufferView.IsEmpty());

				// The number engine buffers that can fit in the space available in the output buffer
				const int32 NumEngineBuffersNeeded = OutputSpaceAvailable / EngineBufferNumFrames;
				// The total number of frames that will be placed in the output buffer
				const int32 TotalFrames = NumEngineBuffersNeeded * EngineBufferNumFrames;
				// The total number of bytes that will be placed in the output buffer
				const int32 TotalBytes = TotalFrames * AudioFormat.GetFrameSizeInBytes();

				uint8* BufferStartPtr = nullptr;
				HRESULT GetBufferResult = RenderClient->GetBuffer(TotalFrames, &BufferStartPtr);
				if (SUCCEEDED(GetBufferResult))
				{
					TArrayView<uint8> BufferStartView(BufferStartPtr, TotalBytes);

					for (int32 i = 0; i < NumEngineBuffersNeeded; ++i)
					{
						const SIZE_T ByteOffset = i * EngineBufferNumBytes;
						RenderBufferView = BufferStartView.Slice(ByteOffset, TotalBytes - ByteOffset);

						if (!ReadNextBufferDelegate.ExecuteIfBound())
						{
							++CallbackBufferErrors;
						}
					}

					HRESULT Result = RenderClient->ReleaseBuffer(TotalFrames, 0 /* flags */);
					if (IsDeviceLostHResult(Result))
					{
						if (!bDeviceLost.exchange(true, std::memory_order_acq_rel))
						{
							UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiDefaultRenderStream: device lost in ReleaseBuffer (HRESULT=0x%08x). Requesting swap.", static_cast<uint32>(Result));
						}
					}
					else if (FAILED(Result))
					{
						++CallbackBufferErrors;
					}

					RenderBufferView = TArrayView<uint8>();
				}
				else if (IsDeviceLostHResult(GetBufferResult))
				{
					if (!bDeviceLost.exchange(true, std::memory_order_acq_rel))
					{
						UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiDefaultRenderStream: device lost in GetBuffer (HRESULT=0x%08x). Requesting swap.", static_cast<uint32>(GetBufferResult));
					}
				}
				else
				{
					++CallbackBufferErrors;
				}
			}
		}
	}

	void FWasapiDefaultRenderStream::SubmitBuffer(const uint8* InBuffer, const SIZE_T InNumFrames)
	{
		const EAudioOutputStreamState::Type CurrStreamState = StreamState.load(std::memory_order_acquire);
		if (CurrStreamState == EAudioOutputStreamState::Running && InBuffer && RenderBufferView.Num() >= InNumFrames)
		{
			checkf(InNumFrames == RenderStreamParams.NumFrames, TEXT("InNumFrames = %zu, RenderStreamParams.NumFrames = %d"), InNumFrames, RenderStreamParams.NumFrames);
			const SIZE_T NumBytes = InNumFrames * AudioFormat.GetFrameSizeInBytes();

			if (uint8* RenderBufferPtr = RenderBufferView.GetData())
			{
				// HardwareDeviceInfo is somewhat of a misnomer, HardwareDeviceInfo.NumChannels is the number of channels
				// that the AudioMixer is requesting. AudioFormat is fetched from the hardware and represents the 
				// actual number of output channels on the hardware device.
				// Another misnomer, "DownmixBuffer" will also upmix if requested.
				const int32 InputNumChannels = RenderStreamParams.HardwareDeviceInfo.NumChannels;
				const int32 OutputNumChannels = AudioFormat.GetNumChannels();

				if (OutputNumChannels != InputNumChannels)
				{
					DownmixBuffer(InputNumChannels, OutputNumChannels, reinterpret_cast<const float*>(InBuffer), DownmixScratchBuffer.GetData(), InNumFrames, MixdownGainsMap.GetData());
					FMemory::Memcpy(RenderBufferPtr, DownmixScratchBuffer.GetData(), NumBytes);
				}
				else
				{
					FMemory::Memcpy(RenderBufferPtr, InBuffer, NumBytes);
				}
			}
		}
	}
}
