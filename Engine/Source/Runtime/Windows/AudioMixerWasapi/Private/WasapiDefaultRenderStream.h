// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerWasapiRenderStream.h"

DECLARE_DELEGATE(FAudioMixerReadNextBufferDelegate);

namespace Audio
{
	/**
	 * FWasapiDefaultRenderStream
	 * The default channel output stream. This is the first eight channels
	 * which are reserved for the main outputs of the editor/game.
	 */
	class FWasapiDefaultRenderStream : public FAudioMixerWasapiRenderStream
	{
	public:

		virtual ~FWasapiDefaultRenderStream() = default;

		//~ Begin FAudioMixerWasapiRenderStream
		virtual bool InitializeHardware(const FWasapiRenderStreamParams& InParams) override;
		virtual bool TeardownHardware() override;
		virtual void SubmitBuffer(const uint8* Buffer, const SIZE_T InNumFrames) override;
		//~ End FAudioMixerWasapiRenderStream

		//~ Begin IDeviceRenderCallback
		virtual void DeviceRenderCallback() override;
		//~ End IDeviceRenderCallback
		
		FAudioMixerReadNextBufferDelegate& OnReadNextBuffer() { return ReadNextBufferDelegate; }

	private:

		/** Number of frames to write every Wasapi period (must be at least the minimum required for current driver) */
		int32 EngineBufferNumFrames = 0;
		
		/** Number of bytes per partial buffer where a partial buffer is used when the engine buffer size
		 *  is less than the Wasapi drivers buffer size. */
		SIZE_T EngineBufferNumBytes = 0;
		
		/** Delegate called each buffer callback to signal the mixer to process the next buffer. */
		FAudioMixerReadNextBufferDelegate ReadNextBufferDelegate;

		/** Pointer to WASAPI render audio buffer filled in each callback by the mixer in SubmitBuffer(). */
		TArrayView<uint8> RenderBufferView;
	};
}
