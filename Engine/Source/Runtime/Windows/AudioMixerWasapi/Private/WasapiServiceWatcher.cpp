// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiServiceWatcher.h"

#if PLATFORM_WINDOWS

#include "AudioMixerWasapiLog.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"

namespace Audio
{
	namespace
	{
		// Mask for state transitions we care about. We register for both ends of the
		// transition (PENDING + final state) so we can log the in-progress transitions
		// and react quickly when the destination state arrives.
		static constexpr DWORD ServiceNotifyMask =
			  SERVICE_NOTIFY_RUNNING
			| SERVICE_NOTIFY_STOPPED
			| SERVICE_NOTIFY_STOP_PENDING
			| SERVICE_NOTIFY_START_PENDING;

		static const TCHAR* ServiceStateToString(DWORD InState)
		{
			switch (InState)
			{
			case SERVICE_STOPPED:           return TEXT("STOPPED");
			case SERVICE_START_PENDING:     return TEXT("START_PENDING");
			case SERVICE_STOP_PENDING:      return TEXT("STOP_PENDING");
			case SERVICE_RUNNING:           return TEXT("RUNNING");
			case SERVICE_CONTINUE_PENDING:  return TEXT("CONTINUE_PENDING");
			case SERVICE_PAUSE_PENDING:     return TEXT("PAUSE_PENDING");
			case SERVICE_PAUSED:            return TEXT("PAUSED");
			default:                        return TEXT("UNKNOWN");
			}
		}

		// QueryServiceStatusEx wrapper that fills in SERVICE_STATUS_PROCESS.
		// Returns the current state (SERVICE_RUNNING etc.), or 0 if the query fails.
		static DWORD ReadServiceState(SC_HANDLE InServiceHandle)
		{
			SERVICE_STATUS_PROCESS Status = {};
			DWORD BytesNeeded = 0;
			const BOOL Success = QueryServiceStatusEx(
				InServiceHandle,
				SC_STATUS_PROCESS_INFO,
				reinterpret_cast<BYTE*>(&Status),
				sizeof(Status),
				&BytesNeeded);
			return Success ? Status.dwCurrentState : 0;
		}

		// No-op APC used only to wake the watcher thread out of its alertable wait when
		// stopping. We rely on the wait returning WAIT_IO_COMPLETION; the loop then
		// re-checks bShouldStop. (The stop event covers most cases; the APC is a
		// belt-and-braces wakeup in case SetEvent races with re-entering the wait.)
		static VOID WINAPI WakeAPC(ULONG_PTR /*InParam*/) {}
	}

	FWasapiServiceWatcher::FWasapiServiceWatcher() = default;

	FWasapiServiceWatcher::~FWasapiServiceWatcher()
	{
		StopWatching();
	}

	bool FWasapiServiceWatcher::StartWatching(FOnServicesRunningCallback InCallback)
	{
		if (Thread.IsValid())
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiServiceWatcher::StartWatching called while already running. Ignoring.");
			return false;
		}

		RecoveryCallback = MoveTemp(InCallback);
		bShouldStop.store(false, std::memory_order_release);

		// Open SCM and service handles synchronously here, not in Run(), so failures
		// surface through the StartWatching return value rather than as a silent
		// thread-exit after the caller already has a thread handle. Handles aren't
		// thread-affined; Run() will use them on the watcher thread.
		ScmHandle = OpenSCManagerW(nullptr, nullptr, SC_MANAGER_CONNECT);
		if (!ScmHandle)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiServiceWatcher::StartWatching OpenSCManagerW failed: %u. Audio-service recovery disabled for this session.", GetLastError());
			RecoveryCallback = nullptr;
			return false;
		}

		AudsrvState.Handle = OpenServiceW(ScmHandle, TEXT("Audiosrv"), SERVICE_QUERY_STATUS);
		AudsrvState.DisplayName = TEXT("Audiosrv");
		AudsrvState.Owner = this;

		AebState.Handle = OpenServiceW(ScmHandle, TEXT("AudioEndpointBuilder"), SERVICE_QUERY_STATUS);
		AebState.DisplayName = TEXT("AudioEndpointBuilder");
		AebState.Owner = this;

		if (!AudsrvState.Handle || !AebState.Handle)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning,
				"FWasapiServiceWatcher::StartWatching OpenServiceW failed (Audiosrv=%p, AEB=%p, LastError=%u). Audio-service recovery disabled for this session.",
				AudsrvState.Handle, AebState.Handle, GetLastError());
			CloseServiceHandles();
			RecoveryCallback = nullptr;
			return false;
		}

		StopEvent = CreateEventW(nullptr, /*bManualReset*/ true, /*bInitialState*/ false, nullptr);
		if (!StopEvent)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiServiceWatcher::StartWatching CreateEventW failed: %u", GetLastError());
			CloseServiceHandles();
			RecoveryCallback = nullptr;
			return false;
		}

		Thread.Reset(FRunnableThread::Create(this, TEXT("WasapiServiceWatcher"), 0, TPri_BelowNormal));
		if (!Thread.IsValid())
		{
			UE_LOGF(LogAudioMixerWasapi, Warning, "FWasapiServiceWatcher::StartWatching failed to create runnable thread");
			CloseHandle(StopEvent);
			StopEvent = nullptr;
			CloseServiceHandles();
			RecoveryCallback = nullptr;
			return false;
		}

		UE_LOGF(LogAudioMixerWasapi, Display, "FWasapiServiceWatcher started.");
		return true;
	}

	void FWasapiServiceWatcher::StopWatching()
	{
		if (!Thread.IsValid())
		{
			return;
		}

		// Signal stop and wake the alertable wait. Stop() (the FRunnable override) also runs
		// during Kill(), but we call it explicitly here for clarity / belt-and-braces.
		Stop();

		Thread->WaitForCompletion();
		Thread.Reset();

		if (StopEvent)
		{
			CloseHandle(StopEvent);
			StopEvent = nullptr;
		}

		// Close handles after the watcher thread has joined - SleepEx drain in Run()
		// reads through them. Pairs with the opens in StartWatching.
		CloseServiceHandles();

		RecoveryCallback = nullptr;
		UE_LOGF(LogAudioMixerWasapi, Display, "FWasapiServiceWatcher stopped.");
	}

	void FWasapiServiceWatcher::Stop()
	{
		bShouldStop.store(true, std::memory_order_release);
		if (StopEvent)
		{
			SetEvent(StopEvent);
		}

		// Belt-and-braces: if the thread is sitting in SleepEx/WaitForSingleObjectEx and the
		// SetEvent above somehow doesn't unblock it (race between signal and wait re-entry),
		// a queued APC will. The thread checks bShouldStop on every wake.
		if (Thread.IsValid())
		{
			HANDLE ThreadHandle = OpenThread(THREAD_SET_CONTEXT, false, Thread->GetThreadID());
			if (ThreadHandle)
			{
				QueueUserAPC(&WakeAPC, ThreadHandle, 0);
				CloseHandle(ThreadHandle);
			}
		}
	}

	uint32 FWasapiServiceWatcher::Run()
	{
		// Handles were opened in StartWatching and validated there - they are
		// guaranteed valid for the lifetime of this thread. StopWatching closes
		// them after joining us, so SleepEx drain below operates on live handles.

		// 1. Read initial state so transition detection has a baseline.
		AudsrvState.LastState = ReadServiceState(AudsrvState.Handle);
		AebState.LastState = ReadServiceState(AebState.Handle);
		bSawNonRunningStatus = AudsrvState.LastState != SERVICE_RUNNING || AebState.LastState != SERVICE_RUNNING;
		UE_LOGF(LogAudioMixerWasapi, Display,
			"FWasapiServiceWatcher initial state: Audiosrv=%ls, AudioEndpointBuilder=%ls",
			ServiceStateToString(AudsrvState.LastState),
			ServiceStateToString(AebState.LastState));

		// 2. Arm initial notifications. ServiceNotifyAPC re-arms after each delivery.
		ArmNotification(AudsrvState);
		ArmNotification(AebState);

		// 3. Loop in alertable wait until Stop is called.
		while (!bShouldStop.load(std::memory_order_acquire))
		{
			const DWORD WaitResult = WaitForSingleObjectEx(StopEvent, INFINITE, /*bAlertable*/ true);

			if (WaitResult == WAIT_OBJECT_0)
			{
				// StopEvent signaled.
				break;
			}
			if (WaitResult == WAIT_IO_COMPLETION)
			{
				// An APC fired and ran. Loop and re-check bShouldStop.
				continue;
			}

			// WAIT_FAILED or other unexpected return; log and exit defensively.
			UE_LOGF(LogAudioMixerWasapi, Warning,
				"FWasapiServiceWatcher: WaitForSingleObjectEx returned 0x%08x (LastError=%u), exiting.",
				WaitResult, GetLastError());
			break;
		}

		// 4. Drain any pending APCs before returning. With bShouldStop set,
		//    ServiceNotifyAPC returns without re-arming, so this terminates promptly.
		SleepEx(0, true);

		return 0;
	}

	bool FWasapiServiceWatcher::ArmNotification(FServiceWatchState& State)
	{
		if (!State.Handle)
		{
			return false;
		}

		// SERVICE_NOTIFY_STATUS_CHANGE = version 2 of the struct; required on Vista+.
		// We re-use State.Notify across re-arms; the OS reads/writes it asynchronously.
		State.Notify.dwVersion = SERVICE_NOTIFY_STATUS_CHANGE;
		State.Notify.pfnNotifyCallback = &FWasapiServiceWatcher::ServiceNotifyAPC;
		State.Notify.pContext = &State;

		const DWORD Result = NotifyServiceStatusChangeW(State.Handle, ServiceNotifyMask, &State.Notify);
		if (Result != ERROR_SUCCESS)
		{
			UE_LOGF(LogAudioMixerWasapi, Warning,
				"FWasapiServiceWatcher: NotifyServiceStatusChangeW failed for %ls: %u",
				State.DisplayName, Result);
			return false;
		}
		return true;
	}

	VOID WINAPI FWasapiServiceWatcher::ServiceNotifyAPC(void* pParam)
	{
		// SCM passes us the SERVICE_NOTIFYW we registered; recover the watch state via pContext.
		SERVICE_NOTIFYW* pNotify = static_cast<SERVICE_NOTIFYW*>(pParam);
		if (!pNotify || !pNotify->pContext)
		{
			return;
		}

		FServiceWatchState* State = static_cast<FServiceWatchState*>(pNotify->pContext);
		FWasapiServiceWatcher* Owner = State->Owner;
		if (!Owner)
		{
			return;
		}

		// If we're shutting down, don't process or re-arm. The wait loop will exit shortly.
		if (Owner->IsStopping())
		{
			return;
		}

		if (pNotify->dwNotificationStatus == ERROR_SUCCESS)
		{
			Owner->OnServiceStateChanged(*State, pNotify->ServiceStatus.dwCurrentState);
		}
		else if (pNotify->dwNotificationStatus == ERROR_SERVICE_MARKED_FOR_DELETE)
		{
			// Service is being uninstalled - extremely unusual for Audiosrv/AEB but log it.
			UE_LOGF(LogAudioMixerWasapi, Warning,
				"FWasapiServiceWatcher: service %ls marked for delete; stopping notifications.",
				State->DisplayName);
			return; // don't re-arm
		}
		else
		{
			UE_LOGF(LogAudioMixerWasapi, Warning,
				"FWasapiServiceWatcher: notification status for %ls: %u",
				State->DisplayName, pNotify->dwNotificationStatus);
		}

		// Re-arm. NotifyServiceStatusChangeW is one-shot per call; we must re-register to
		// keep receiving callbacks. If re-arm fails we just stop watching this service - the
		// other one might still fire, and StopWatching will close handles cleanly.
		Owner->ArmNotification(*State);

		// MSDN-recommended: query after re-arm to catch transitions that landed in the unwatched gap.
		const DWORD ActualState = ReadServiceState(State->Handle);
		if (ActualState != 0 && ActualState != State->LastState)
		{
			Owner->OnServiceStateChanged(*State, ActualState);
		}
	}

	void FWasapiServiceWatcher::OnServiceStateChanged(FServiceWatchState& State, DWORD NewState)
	{
		const DWORD PreviousState = State.LastState;
		State.LastState = NewState;

		if (PreviousState != NewState)
		{
			UE_LOGF(LogAudioMixerWasapi, Display,
				"FWasapiServiceWatcher: %ls state %ls -> %ls",
				State.DisplayName,
				ServiceStateToString(PreviousState),
				ServiceStateToString(NewState));
		}

		// Latch on any non-RUNNING observation so SCM coalescing of stop+start can't swallow the recovery edge.
		if (NewState != SERVICE_RUNNING)
		{
			bSawNonRunningStatus = true;
		}

		if (bSawNonRunningStatus && BothServicesRunning() && RecoveryCallback)
		{
			UE_LOGF(LogAudioMixerWasapi, Display,
				"FWasapiServiceWatcher: both services RUNNING - firing recovery callback.");
			bSawNonRunningStatus = false;
			RecoveryCallback();
		}
	}

	bool FWasapiServiceWatcher::BothServicesRunning() const
	{
		return AudsrvState.LastState == SERVICE_RUNNING
			&& AebState.LastState == SERVICE_RUNNING;
	}

	void FWasapiServiceWatcher::CloseServiceHandles()
	{
		if (AudsrvState.Handle) { CloseServiceHandle(AudsrvState.Handle); AudsrvState.Handle = nullptr; }
		if (AebState.Handle)    { CloseServiceHandle(AebState.Handle);    AebState.Handle = nullptr; }
		if (ScmHandle)          { CloseServiceHandle(ScmHandle);          ScmHandle = nullptr; }
	}
}

#endif // PLATFORM_WINDOWS
