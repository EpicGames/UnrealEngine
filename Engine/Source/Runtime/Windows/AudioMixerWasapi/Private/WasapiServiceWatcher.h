// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/Function.h"

#if PLATFORM_WINDOWS

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "Templates/UniquePtr.h"

#include <atomic>

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
THIRD_PARTY_INCLUDES_START
#include <winsvc.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

class FRunnableThread;

namespace Audio
{
	/**
	 * FWasapiServiceWatcher
	 *
	 * Watches Windows Service Control Manager state for Audiosrv and AudioEndpointBuilder
	 * via NotifyServiceStatusChangeW. Fires the registered callback (on the watcher thread)
	 * when both services transition back to SERVICE_RUNNING after either was observed
	 * non-RUNNING. The handler is responsible for marshalling work to the audio thread if
	 * needed; RequestDeviceSwap is thread-safe so it can be called directly.
	 *
	 * Lifecycle: construct, StartWatching(...), StopWatching(). Safe to destroy after Stop.
	 * Internally owns one dedicated OS thread that sits in an alertable wait - APC delivery
	 * is how SCM gives us notifications, so the wait must be alertable. Stop signaling uses
	 * a manual-reset event plus a no-op QueueUserAPC fallback for clean wakeup.
	 */
	class FWasapiServiceWatcher : public FRunnable
	{
	public:
		using FOnServicesRunningCallback = TFunction<void()>;

		FWasapiServiceWatcher();
		virtual ~FWasapiServiceWatcher();

		/** Start the watcher thread. Returns false if SCM open / notification setup fails. */
		bool StartWatching(FOnServicesRunningCallback InCallback);

		/** Stop the watcher thread and release SCM handles. Blocks until the thread exits. */
		void StopWatching();

		//~ FRunnable
		virtual uint32 Run() override;
		virtual void Stop() override; // Sets stop flag and signals the wait event.
		//~

		/** Used by the static APC callback to decide whether to re-arm. */
		bool IsStopping() const { return bShouldStop.load(std::memory_order_acquire); }

	private:
		// Per-service watch state. Holds the SERVICE_NOTIFYW that NotifyServiceStatusChangeW
		// needs to outlive the call (the OS writes to it asynchronously).
		struct FServiceWatchState
		{
			SERVICE_NOTIFYW Notify = {};
			SC_HANDLE Handle = nullptr;
			const TCHAR* DisplayName = nullptr;
			DWORD LastState = 0; // SERVICE_RUNNING / SERVICE_STOPPED / SERVICE_*_PENDING
			FWasapiServiceWatcher* Owner = nullptr;
		};

		/** Static APC trampoline; recovers the FServiceWatchState from pNotify->pContext. */
		static VOID WINAPI ServiceNotifyAPC(void* pParam);

		/** Called from ServiceNotifyAPC on the watcher thread when a service state arrives. */
		void OnServiceStateChanged(FServiceWatchState& State, DWORD NewState);

		/** Calls NotifyServiceStatusChangeW with our mask and re-arms the callback. */
		bool ArmNotification(FServiceWatchState& State);

		/** True if both services are last-known RUNNING. */
		bool BothServicesRunning() const;

		/** Closes ScmHandle and both per-service handles if open. Null-safe; pairs with
		 *  the synchronous opens in StartWatching. Called from StopWatching after the
		 *  watcher thread has joined, and from StartWatching's failure cleanup paths. */
		void CloseServiceHandles();

		SC_HANDLE ScmHandle = nullptr;
		FServiceWatchState AudsrvState;
		FServiceWatchState AebState;

		// Latched when either service is observed non-RUNNING; cleared when recovery fires.
		bool bSawNonRunningStatus = false;

		FOnServicesRunningCallback RecoveryCallback;
		TUniquePtr<FRunnableThread> Thread;
		HANDLE StopEvent = nullptr; // manual-reset; signaled to wake the alertable wait
		std::atomic<bool> bShouldStop = false;
	};
}

#else // !PLATFORM_WINDOWS

// Non-Windows platforms (Xbox uses synchronous swap and its own audio-stack lifecycle;
// no SCM watcher needed). Declared as an empty inline shim so callers in
// FAudioMixerWasapi don't need PLATFORM_WINDOWS guards around every member access.

namespace Audio
{
	class FWasapiServiceWatcher
	{
	public:
		using FOnServicesRunningCallback = TFunction<void()>;
		bool StartWatching(FOnServicesRunningCallback) { return false; }
		void StopWatching() {}
	};
}

#endif // PLATFORM_WINDOWS
