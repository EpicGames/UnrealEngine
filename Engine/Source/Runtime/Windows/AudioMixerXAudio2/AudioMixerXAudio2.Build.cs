// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMixerXAudio2 : ModuleRules
{
	public AudioMixerXAudio2(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"WindowsMMDeviceEnumeration"
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target,
			"UEOgg",
			"Vorbis",
			"VorbisFile"
			);
		}
		PrivateDependencyModuleNames.AddRange(
			new string[] {
					"Core",
					"CoreUObject",
					"AudioMixer",
					"AudioMixerCore"
				}
		);

		PrecompileForTargets = PrecompileTargetsType.None;

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"XAudio2_9"
		);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}
