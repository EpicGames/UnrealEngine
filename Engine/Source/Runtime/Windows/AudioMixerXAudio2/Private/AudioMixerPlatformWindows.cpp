// Copyright Epic Games, Inc. All Rights Reserved.

/**
	Concrete implementation of FAudioDevice for XAudio2

	See https://msdn.microsoft.com/en-us/library/windows/desktop/hh405049%28v=vs.85%29.aspx
*/

#include "AudioMixerPlatformXAudio2.h"
#include "AudioMixer.h"
#include "AudioDeviceNotificationSubsystem.h"
#include "Misc/ScopeRWLock.h"

#include <atomic>

#if PLATFORM_WINDOWS

#include "Microsoft/COMPointer.h"
#include "ScopedCom.h"					// FScopedComString

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
// Linkage for  Windows GUIDs included by Notification/DeviceInfoCache, otherwise they are extern.
#include <initguid.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "WindowsMMNotificationClient.h"
#include "WindowsMMDeviceInfoCache.h"
#include "WindowsMMStringUtils.h"

namespace Audio
{	
	TSharedPtr<FWindowsMMNotificationClient> WindowsNotificationClient;

	void RegisterForSessionEvents(const FString& InDeviceId)
	{
		if (WindowsNotificationClient)
		{
			WindowsNotificationClient->RegisterForSessionNotifications(InDeviceId);
		}
	}
	void UnregisterForSessionEvents()
	{
		if (WindowsNotificationClient)
		{
			WindowsNotificationClient->UnregisterForSessionNotifications();
		}
	}
		
	void FMixerPlatformXAudio2::RegisterDeviceChangedListener()
	{
		if (!WindowsNotificationClient.IsValid())
		{
			// Shared (This is a COM object, so we don't delete it, just derecement the ref counter).
			WindowsNotificationClient = TSharedPtr<FWindowsMMNotificationClient>(
				new FWindowsMMNotificationClient, 
				[](FWindowsMMNotificationClient* InPtr) { InPtr->ReleaseClient(); }
			);
		}
		if (!DeviceInfoCache.IsValid())
		{
			// Setup device info cache.
			DeviceInfoCache = MakeUnique<FWindowsMMDeviceCache>();
			WindowsNotificationClient->RegisterDeviceChangedListener(static_cast<FWindowsMMDeviceCache*>(DeviceInfoCache.Get()));
		}

		WindowsNotificationClient->RegisterDeviceChangedListener(this);
	}

	void FMixerPlatformXAudio2::UnregisterDeviceChangedListener() 
	{
		if (WindowsNotificationClient.IsValid())
		{
			if (DeviceInfoCache.IsValid())
			{
				// Unregister and kill cache.
				WindowsNotificationClient->UnRegisterDeviceDeviceChangedListener(static_cast<FWindowsMMDeviceCache*>(DeviceInfoCache.Get()));
				
				DeviceInfoCache.Reset();
			}
			
			WindowsNotificationClient->UnRegisterDeviceDeviceChangedListener(this);
		}
	}

	void FMixerPlatformXAudio2::OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
	{
		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifSubsystem->OnDefaultCaptureDeviceChanged(InAudioDeviceRole, DeviceId);
		}
	}

	void FMixerPlatformXAudio2::OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
	{		
		// There's 3 defaults in windows (communications, console, multimedia). These technically can all be different devices.		
		// However the Windows UX only allows console+multimedia to be toggle as a pair. This means you get two notifications
		// for default device changing typically. To prevent a trouble trigger we only listen to "Console" here. For more information on 
		// device roles: https://docs.microsoft.com/en-us/windows/win32/coreaudio/device-roles
		
		if (InAudioDeviceRole == EAudioDeviceRole::Console)
		{
			UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2: Changing default audio render device to new device: Role=%ls, DeviceName=%ls, InstanceID=%d", 
				Audio::ToString(InAudioDeviceRole), *WindowsNotificationClient->GetFriendlyName(DeviceId), InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.
			if (GetIsListeningForDeviceEvents())
			{
				RequestDeviceSwap(DeviceId, /* force */true, TEXT("FMixerPlatformXAudio2::OnDefaultRenderDeviceChanged"));
			}
		}

		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifSubsystem->OnDefaultRenderDeviceChanged(InAudioDeviceRole, DeviceId);
		}
	}

	void FMixerPlatformXAudio2::OnDeviceAdded(const FString& DeviceId, bool bIsRenderDevice)
	{
		// Ignore changes in capture device.
		if (!bIsRenderDevice)
		{
			return;
		}
		
		// If the device that was added is our original device and our current device is NOT our original device, 
		// move our audio stream to this newly added device.
		const FString AudioDeviceId = GetOriginalAudioDeviceId();
		if (AudioStreamInfo.DeviceInfo.DeviceId != AudioDeviceId && DeviceId == AudioDeviceId)
		{
			UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2: Original audio device re-added. Moving audio back to original audio device: DeviceName=%ls, bRenderDevice=%d, InstanceID=%d", 
				*WindowsNotificationClient->GetFriendlyName(*AudioDeviceId), (int32)bIsRenderDevice, InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.
			if (GetIsListeningForDeviceEvents())
			{
				RequestDeviceSwap(AudioDeviceId, /*force */ true, TEXT("FMixerPlatformXAudio2::OnDeviceAdded"));
			}
		}

		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifSubsystem->OnDeviceAdded(DeviceId, bIsRenderDevice);
		}
	}

	void FMixerPlatformXAudio2::OnDeviceRemoved(const FString& DeviceId, bool bIsRenderDevice)
	{
		// Ignore changes in capture device.
		if (!bIsRenderDevice)
		{
			return;
		}
		
		// If the device we're currently using was removed... then switch to the new default audio device.
		if (AudioStreamInfo.DeviceInfo.DeviceId == DeviceId)
		{
			UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2: Audio device removed [%ls], falling back to other windows default device. bIsRenderDevice=%d, InstanceID=%d", 
				*WindowsNotificationClient->GetFriendlyName(DeviceId), (int32)bIsRenderDevice, InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.
			if (GetIsListeningForDeviceEvents())
			{
				RequestDeviceSwap(TEXT(""), /* force */ true, TEXT("FMixerPlatformXAudio2::OnDeviceRemoved"));
			}
		}

		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifSubsystem->OnDeviceRemoved(DeviceId, bIsRenderDevice);
		}
	}

	void FMixerPlatformXAudio2::OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool bIsRenderDevice)
	{
		// Ignore changes in capture device.
		if (!bIsRenderDevice)
		{
			return;
		}
		
		// If the device we're currently using was removed and it's not the system default... then switch to the new default audio device.
		// If it is the system default device, then OnDefaultRenderDeviceChanged() will be called to handle this.
		if (AudioStreamInfo.DeviceInfo.DeviceId == DeviceId && !AudioStreamInfo.DeviceInfo.bIsSystemDefault &&
			(InState == EAudioDeviceState::Disabled || InState == EAudioDeviceState::NotPresent || InState == EAudioDeviceState::Unplugged))
		{
			UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2: Audio device not available [%ls], falling back to other windows default device. InState=%d, bIsRenderDevice=%d, InstanceID=%d", 
				*WindowsNotificationClient->GetFriendlyName(DeviceId), (int32)InState, (int32)bIsRenderDevice, InstanceID);

			// Ignore if not listening for events. This means the user has specified a specific device
			// to use and is not interested in following the current system default device.
			if (GetIsListeningForDeviceEvents())
			{
				RequestDeviceSwap(TEXT(""), /* force */ true, TEXT("FMixerPlatformXAudio2::OnDeviceStateChanged"));
			}
		}

		if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
		{
			AudioDeviceNotifSubsystem->OnDeviceStateChanged(DeviceId, InState, bIsRenderDevice);
		}
	}

	FString FMixerPlatformXAudio2::GetDeviceId() const
	{
		return AudioStreamInfo.DeviceInfo.DeviceId;
	}
}

//#include "Windows/HideWindowsPlatformAtomics.h"
//#include "Windows/HideWindowsPlatformTypes.h"

#else 
// Nothing for XBOXOne
namespace Audio
{
	void FMixerPlatformXAudio2::RegisterDeviceChangedListener() {}
	void FMixerPlatformXAudio2::UnregisterDeviceChangedListener() {}
	void FMixerPlatformXAudio2::OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) {}
	void FMixerPlatformXAudio2::OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) {}
	void FMixerPlatformXAudio2::OnDeviceAdded(const FString& DeviceId, bool bIsRender) {}
	void FMixerPlatformXAudio2::OnDeviceRemoved(const FString& DeviceId, bool bIsRender) {}
	void FMixerPlatformXAudio2::OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool bIsRender){}
	FString FMixerPlatformXAudio2::GetDeviceId() const
	{
		return AudioStreamInfo.DeviceInfo.DeviceId;
	}
}
#endif

