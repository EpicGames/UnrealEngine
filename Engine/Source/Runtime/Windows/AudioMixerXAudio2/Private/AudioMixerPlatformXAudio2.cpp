// Copyright Epic Games, Inc. All Rights Reserved.

/**
	Concrete implementation of FAudioDevice for XAudio2

	See https://msdn.microsoft.com/en-us/library/windows/desktop/hh405049%28v=vs.85%29.aspx
*/

#include "AudioMixerPlatformXAudio2.h"
#include "AudioMixer.h"
#include "AudioDevice.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "Engine/EngineTypes.h"

#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopeLock.h"
#include "HAL/Event.h"
#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "ToStringHelpers.h"
#include "ScopedCom.h"

THIRD_PARTY_INCLUDES_START
// Including initguid.h will define the PKEY symbols below which area used cross-platform
#include <initguid.h>
#include <mmdeviceapi.h>
#include <AudioClient.h>
#if PLATFORM_WINDOWS
#include <FunctionDiscoveryKeys_devpkey.h>
#endif //PLATFORM_WINDOWS
THIRD_PARTY_INCLUDES_END

#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Async/Async.h"

#define XAUDIO2_LOG_AND_HANDLE_ON_FAIL(FunctionName, Result, OnError)\
	if (FAILED(Result))\
	{\
		UE_LOGF(LogAudioMixer, Error, "XAudio2 Error: %ls -> 0x%X: '%ls', called in '%s' (%ls:%d)",\
			TEXT(FunctionName), (uint32)Result, *Audio::ToErrorFString(Result), __func__, __FILEW__, __LINE__);\
		OnError;\
	}
	
#define XAUDIO2_CALL_AND_HANDLE_ERROR(CALL, OnError)\
	{\
		const HRESULT Result = CALL;\
		XAUDIO2_LOG_AND_HANDLE_ON_FAIL(#CALL, Result, OnError);\
	}

static XAUDIO2_PROCESSOR GetXAudio2ProcessorsToUse()
{
	XAUDIO2_PROCESSOR ProcessorsToUse = (XAUDIO2_PROCESSOR)FPlatformAffinity::GetAudioRenderThreadMask();
	// https://docs.microsoft.com/en-us/windows/win32/api/xaudio2/nf-xaudio2-xaudio2create
	// Warning If you specify XAUDIO2_ANY_PROCESSOR, the system will use all of the device's processors and, as noted above, create a worker thread for each processor.
	// We certainly don't want to use all available CPU. XAudio threads are time critical priority and wake up every 10 ms, they may cause lots of unwarranted context switches.
	// In case no specific affinity is specified, let XAudio choose the default processor. It should allocate a single thread and should be enough.
	if (ProcessorsToUse == XAUDIO2_ANY_PROCESSOR)
	{
	#ifdef XAUDIO2_USE_DEFAULT_PROCESSOR
		ProcessorsToUse = XAUDIO2_USE_DEFAULT_PROCESSOR;
	#else
		ProcessorsToUse = XAUDIO2_DEFAULT_PROCESSOR;
	#endif
	}

	return ProcessorsToUse;
}

/*
	Whether or not to enable xaudio2 debugging mode
	To see the debug output, you need to view ETW logs for this application:
	Go to Control Panel, Administrative Tools, Event Viewer.
	View->Show Analytic and Debug Logs.
	Applications and Services Logs / Microsoft / Windows / XAudio2.
	Right click on Microsoft Windows XAudio2 debug logging, Properties, then Enable Logging, and hit OK
*/
#define XAUDIO2_DEBUG_ENABLED 0

namespace Audio
{
	static float GThreadedSwapDebugExtraTimeMs = 0;
	static FAutoConsoleVariableRef GThreadedSwapDebugExtraTimeMsCVar(
		TEXT("au.ThreadedSwapDebugExtraTime"),
		GThreadedSwapDebugExtraTimeMs,
		TEXT("Simulate a slow device swap by adding additional time to the swap task"),
		ECVF_Default);

	void FXAudio2VoiceCallback::OnBufferEnd(void* BufferContext)
	{
		SCOPED_NAMED_EVENT(FXAudio2VoiceCallback_OnBufferEnd, FColor::Blue);
		
		check(BufferContext);
		IAudioMixerPlatformInterface* MixerPlatform = (IAudioMixerPlatformInterface*)BufferContext;
		MixerPlatform->ReadNextBuffer();
	}

	static uint32 ChannelTypeMap[EAudioMixerChannel::ChannelTypeCount] =
	{
		SPEAKER_FRONT_LEFT,
		SPEAKER_FRONT_RIGHT,
		SPEAKER_FRONT_CENTER,
		SPEAKER_LOW_FREQUENCY,
		SPEAKER_BACK_LEFT,
		SPEAKER_BACK_RIGHT,
		SPEAKER_FRONT_LEFT_OF_CENTER,
		SPEAKER_FRONT_RIGHT_OF_CENTER,
		SPEAKER_BACK_CENTER,
		SPEAKER_SIDE_LEFT,
		SPEAKER_SIDE_RIGHT,
		SPEAKER_TOP_CENTER,
		SPEAKER_TOP_FRONT_LEFT,
		SPEAKER_TOP_FRONT_CENTER,
		SPEAKER_TOP_FRONT_RIGHT,
		SPEAKER_TOP_BACK_LEFT,
		SPEAKER_TOP_BACK_CENTER,
		SPEAKER_TOP_BACK_RIGHT,
		SPEAKER_RESERVED,
	};

	FMixerPlatformXAudio2::FMixerPlatformXAudio2()
		: XAudio2System(nullptr)
		, OutputAudioStreamMasteringVoice(nullptr)
		, OutputAudioStreamSourceVoice(nullptr)
		, TimeSinceNullDeviceWasLastChecked(0.0f)		
		, bIsInitialized(false)
		, bIsDeviceOpen(false)
	{
#if PLATFORM_WINDOWS 
		FPlatformMisc::CoInitialize();
#endif // #if PLATFORM_WINDOWS 
	}

	FMixerPlatformXAudio2::~FMixerPlatformXAudio2()
	{
#if PLATFORM_WINDOWS
		FPlatformMisc::CoUninitialize();
#endif // #if PLATFORM_WINDOWS 
	}

#if PLATFORM_WINDOWS
	// Dirty extern for now.
	extern void RegisterForSessionEvents(const FString& InDeviceId);
#endif //PLATFORM_WINDOWS

	bool FMixerPlatformXAudio2::CheckThreadedDeviceSwap()
	{
		bool bDidStopGeneratingAudio = false;
#if PLATFORM_WINDOWS
		bDidStopGeneratingAudio = FAudioMixerPlatformSwappable::CheckThreadedDeviceSwap();
#endif //PLATFORM_WINDOWS
		return bDidStopGeneratingAudio;
	}

	bool FMixerPlatformXAudio2::PreDeviceSwap()
	{
		// Access to device swap context must be protected by DeviceSwapCriticalSection
		FScopeLock Lock(&DeviceSwapCriticalSection);
		
		if (DeviceSwapContext.IsValid())
		{
			// Finish initializing the device swap context
			DeviceSwapContext->PreviousSystem = XAudio2System;
			DeviceSwapContext->PreviousMasteringVoice = OutputAudioStreamMasteringVoice;
			DeviceSwapContext->PreviousSourceVoice = OutputAudioStreamSourceVoice;
			DeviceSwapContext->Callbacks = &OutputVoiceCallback;
			DeviceSwapContext->RenderingSampleRate = OpenStreamParams.SampleRate;

			// NULL our system / master / source voices. (as they are about to be torn down async).
			XAudio2System = nullptr;
			OutputAudioStreamMasteringVoice = nullptr;
			OutputAudioStreamSourceVoice = nullptr;

			UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::PreDeviceSwap - Starting swap to [%ls]", DeviceSwapContext->RequestedDeviceId.IsEmpty() ? TEXT("[System Default]") : *DeviceSwapContext->RequestedDeviceId);

			return true;
		}
		else
		{
			UE_LOGF(LogAudioMixer, Warning, "FMixerPlatformXAudio2::PreDeviceSwap - null device swap context");
			return false;
		}

		return false;
	}
	
	void FMixerPlatformXAudio2::EnqueueAsyncDeviceSwap()
	{
		UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::EnqueueAsyncDeviceSwap - enqueuing async device swap");
		FScopeLock Lock(&DeviceSwapCriticalSection);
		
		TFunction<TUniquePtr<FDeviceSwapResult>()> AsyncDeviceSwap = [this]() mutable -> TUniquePtr<FDeviceSwapResult>
		{
			// Transfer ownership of DeviceSwapContext to the async task.
			TUniquePtr<FXAudio2DeviceSwapContext> TempContext;
			{
				FScopeLock Lock(&DeviceSwapCriticalSection);

				if (AudioStreamInfo.StreamState == EAudioOutputStreamState::SwappingDevice)
				{
					TempContext = MoveTemp(DeviceSwapContext);
				}
			}
			
			return PerformDeviceSwap(MoveTemp(TempContext));
		};
		SetActiveDeviceSwapFuture(Async(EAsyncExecution::TaskGraph, MoveTemp(AsyncDeviceSwap)));
	}
	
	bool FMixerPlatformXAudio2::PostDeviceSwap()
	{
		bool bDidSucceed = false;
		const FXAudio2DeviceSwapResult* DeviceSwapResult = static_cast<const FXAudio2DeviceSwapResult*>(GetDeviceSwapResult());
		
		if (DeviceSwapResult && DeviceSwapResult->IsNewDeviceReady())
		{
			FScopeLock Lock(&DeviceSwapCriticalSection);

			XAudio2System = DeviceSwapResult->NewSystem;
			OutputAudioStreamMasteringVoice = DeviceSwapResult->NewMasteringVoice;
			OutputAudioStreamSourceVoice = DeviceSwapResult->NewSourceVoice;
		
			// Success?
			if (XAudio2System && OutputAudioStreamSourceVoice && OutputAudioStreamMasteringVoice)
			{
				HRESULT Result;
				{
					SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_PostDeviceSwap_StartEngine, FColor::Blue);
					Result = XAudio2System->StartEngine();
				}
				if (SUCCEEDED(Result))
				{
					// Copy our new Device Info into our active one.
					AudioStreamInfo.DeviceInfo = DeviceSwapResult->DeviceInfo;

					// Display our new XAudio2 Mastering voice details.
					UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::PostDeviceSwap - successful Swap new Device is (NumChannels=%u, SampleRate=%u, DeviceID=%ls, Name=%ls), Reason=%ls, InstanceID=%d, DurationMS=%.2f",
						(uint32)AudioStreamInfo.DeviceInfo.NumChannels, (uint32)AudioStreamInfo.DeviceInfo.SampleRate, *AudioStreamInfo.DeviceInfo.DeviceId, *AudioStreamInfo.DeviceInfo.Name,
						*DeviceSwapResult->SwapReason, InstanceID, DeviceSwapResult->SuccessfulDurationMs);

					// Reinitialize the output circular buffer to match the buffer math of the new audio device.
					const int32 NumOutputSamples = AudioStreamInfo.NumOutputFrames * AudioStreamInfo.DeviceInfo.NumChannels;
					if (ensure(NumOutputSamples > 0))
					{
						OutputBuffer.Init(AudioStreamInfo.AudioMixer, NumOutputSamples, NumOutputBuffers, AudioStreamInfo.DeviceInfo.Format);
					}

					bDidSucceed = true;
				}
				else
				{
					XAUDIO2_LOG_AND_HANDLE_ON_FAIL("XAudio2System->StartEngine()", Result, {/* nop */});
				}
			}
			else // We either failed to init or deliberately switched to null device.
			{
				// Null renderer doesn't/shouldn't care about the format, so leave the format as it was before.
			}
		}

		ResetActiveDeviceSwapFuture();
		
		return bDidSucceed;
	}
	
	void FMixerPlatformXAudio2::ResumePlaybackOnNewDevice()
	{
		if (DeviceSwapContext.IsValid())
		{
			// If another device swap was requested, don't bother resuming playback
			AudioStreamInfo.StreamState = EAudioOutputStreamState::SwappingDevice;
			UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::ResumePlaybackOnNewDevice - device swap requested during previous device swap...not resuming audio");
		}
		else
		{
			FAudioMixerPlatformSwappable::ResumePlaybackOnNewDevice();
		}
	}

	void FMixerPlatformXAudio2::SynchronousDeviceSwap()
	{
		// Transfer ownership of DeviceSwapContext memory to the device swap routine
		TUniquePtr<FDeviceSwapResult> DeviceSwapResult = PerformDeviceSwap(TUniquePtr<FXAudio2DeviceSwapContext>(MoveTemp(DeviceSwapContext)));

		// Set the promise and future result to replicate what the async task does
		TPromise<TUniquePtr<FDeviceSwapResult>> Promise;
		
		// It's ok if DeviceSwapResult is null here. It indicates an invalid device which will be handled.
		Promise.SetValue(MoveTemp(DeviceSwapResult));
		SetActiveDeviceSwapFuture(Promise.GetFuture());
	}
	
	TUniquePtr<FDeviceSwapResult> FMixerPlatformXAudio2::PerformDeviceSwap(TUniquePtr<FXAudio2DeviceSwapContext>&& InDeviceContext)
	{
		// Static method enforces no other state sharing occurs with the object that called it. InDeviceContext
		// should have no other references outside of this method so that the device swap operation is isolated.
		SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_PerformDeviceSwap, FColor::Blue);

		const uint64 StartTimeCycles = FPlatformTime::Cycles64();

		// New thread might not have COM setup.
		FScopedCoInitialize ScopedCoInitialize;

		// Critical section not required here as this function is now sole owner of context
		if (!InDeviceContext.IsValid())
		{
			UE_LOGF(LogAudioMixer, Error, "FMixerPlatformXAudio2::PerformDeviceSwap - failed due to invalid DeviceSwapContext");
			return {};
		}

		UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::PerformDeviceSwap - AsyncTask Start. Because=%ls", *InDeviceContext->DeviceSwapReason);

		// Stop old engine running.
		if (InDeviceContext->PreviousSystem)
		{
			SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_StopEngine, FColor::Blue);
			InDeviceContext->PreviousSystem->StopEngine();
		}

		// Kill source voice.
		if (InDeviceContext->PreviousSourceVoice)
		{
			{
				SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_FlushSourceBuffers, FColor::Blue);
				XAUDIO2_CALL_AND_HANDLE_ERROR(InDeviceContext->PreviousSourceVoice->FlushSourceBuffers(), /* nop */ );
			}
			SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_DestroySourceVoice, FColor::Blue);
			InDeviceContext->PreviousSourceVoice->DestroyVoice();
			InDeviceContext->PreviousSourceVoice = nullptr;
		}

		// Now destroy the mastering voice
		if (InDeviceContext->PreviousMasteringVoice)
		{
			SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_DestroyMasterVoice, FColor::Blue);
			InDeviceContext->PreviousMasteringVoice->DestroyVoice();
			InDeviceContext->PreviousMasteringVoice = nullptr;
		}

		// Destroy System
		{
			SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_DestroySystem, FColor::Blue);
			SAFE_RELEASE(InDeviceContext->PreviousSystem);
			InDeviceContext->PreviousSystem = nullptr;
		}

		// Don't attempt to create a new setup if there's no devices available.
		if (!InDeviceContext->NewDevice.IsSet())
		{
			return {};
		}

		TUniquePtr<FXAudio2DeviceSwapResult> DeviceSwapResult = MakeUnique<FXAudio2DeviceSwapResult>();
		
		// Create System.
		{
			SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_CreateSystem, FColor::Blue);
			XAUDIO2_CALL_AND_HANDLE_ERROR(XAudio2Create(&DeviceSwapResult->NewSystem, 0, GetXAudio2ProcessorsToUse()), return {});
		}

		// Create Master
		{
			check(InDeviceContext->NewDevice->NumChannels <= XAUDIO2_MAX_AUDIO_CHANNELS);
			check(InDeviceContext->NewDevice->SampleRate >= XAUDIO2_MIN_SAMPLE_RATE);
			check(InDeviceContext->NewDevice->SampleRate <= XAUDIO2_MAX_SAMPLE_RATE);

			SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_CreateMasterVoice, FColor::Blue);
			DeviceSwapResult->NewMasteringVoice = CreateMasteringVoice(*DeviceSwapResult->NewSystem, *InDeviceContext->NewDevice, InDeviceContext->bUseDefaultDevice);
			if (!DeviceSwapResult->NewMasteringVoice)
			{
				SAFE_RELEASE(DeviceSwapResult->NewSystem);
				return {}; // FAIL.
			}
		}

		// Create Source Voice.
		{
			SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_AsyncDeleteCreate_CreateSourceVoice, FColor::Blue);

			// Setup the format of the output source voice
			WAVEFORMATEX Format = { 0 };
			Format.nChannels = InDeviceContext->NewDevice->NumChannels;
			Format.nSamplesPerSec = InDeviceContext->RenderingSampleRate;		// NOTE: We use the Rendering sample rate here.
			Format.wFormatTag = WAVE_FORMAT_IEEE_FLOAT;
			Format.nAvgBytesPerSec = Format.nSamplesPerSec * sizeof(float) * Format.nChannels;
			Format.nBlockAlign = sizeof(float) * Format.nChannels;
			Format.wBitsPerSample = sizeof(float) * 8;

			// Create the output source voice
			HRESULT Result = DeviceSwapResult->NewSystem->CreateSourceVoice(
				&DeviceSwapResult->NewSourceVoice,
				&Format,
				XAUDIO2_VOICE_NOPITCH,
				XAUDIO2_DEFAULT_FREQ_RATIO,
				InDeviceContext->Callbacks,
				nullptr,
				nullptr
			);

			if (FAILED(Result))
			{
				if (DeviceSwapResult->NewSourceVoice)
				{
					DeviceSwapResult->NewSourceVoice->DestroyVoice();
					DeviceSwapResult->NewSourceVoice = nullptr;
				}

				if (DeviceSwapResult->NewMasteringVoice)
				{
					DeviceSwapResult->NewMasteringVoice->DestroyVoice();
					DeviceSwapResult->NewMasteringVoice = nullptr;
				}
				
				SAFE_RELEASE(DeviceSwapResult->NewSystem);
				XAUDIO2_LOG_AND_HANDLE_ON_FAIL("XAudio2System->CreateSourceVoice", Result, return {});
			}
		}

		// Optionally for testing, sleep for some duration in order to help repro race conditions.
		if (GThreadedSwapDebugExtraTimeMs > 0.f)
		{
			FPlatformProcess::Sleep(GThreadedSwapDebugExtraTimeMs / 1000.f); 
		}

		// Listen session for changes to this device.
#if PLATFORM_WINDOWS
		RegisterForSessionEvents(InDeviceContext->RequestedDeviceId);
#endif
		
		DeviceSwapResult->SuccessfulDurationMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTimeCycles);
		DeviceSwapResult->DeviceInfo = InDeviceContext->NewDevice.Get(FAudioPlatformDeviceInfo());
		DeviceSwapResult->SwapReason = InDeviceContext->DeviceSwapReason;
		
		return DeviceSwapResult;
	}
	
	bool FMixerPlatformXAudio2::ResetXAudio2System()
	{
		SAFE_RELEASE(XAudio2System);

		XAudio2System = nullptr;
		XAUDIO2_CALL_AND_HANDLE_ERROR(
			XAudio2Create(&XAudio2System, GetCreateFlags(), GetXAudio2ProcessorsToUse()),
			return false);
		
		XAUDIO2_CALL_AND_HANDLE_ERROR(XAudio2System->RegisterForCallbacks(this), return false);

		// success.
		return true;
	}

	bool FMixerPlatformXAudio2::InitializeHardware()
	{
		if (bIsInitialized)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("XAudio2 already initialized."), Warning);
			return false;
		}

		const uint32 Flags = GetCreateFlags();
		if (!XAudio2System && FAILED(XAudio2Create(&XAudio2System, Flags, GetXAudio2ProcessorsToUse())))
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Audio", "XAudio2Error", "Failed to initialize audio. This may be an issue with your installation of XAudio 2.7. XAudio2 is available in the DirectX End-User Runtime (June 2010)."));
			return false;
		}
	 
#if XAUDIO2_DEBUG_ENABLED
		XAUDIO2_DEBUG_CONFIGURATION DebugConfiguration = { 0 };
		DebugConfiguration.TraceMask = XAUDIO2_LOG_ERRORS | XAUDIO2_LOG_WARNINGS;
		XAudio2System->SetDebugConfiguration(&DebugConfiguration, 0);
#endif // #if XAUDIO2_DEBUG_ENABLED

		if (FAILED(XAudio2System->RegisterForCallbacks(this)))
		{
			UE_LOGF(LogAudioMixer, Error, "Failed to register for callbacks.");
		}
		
		if(IAudioMixer::ShouldRecycleThreads())
		{
			// Pre-create the null render device thread on XAudio2, so we can simple wake it up when we need it.
			// Give it nothing to do, with a slow tick as the default, but ask it to wait for a signal to wake up.
			CreateNullDeviceThread([] {}, 1.0f, true);
		}

		bIsInitialized = true;

		return true;
	}

	bool FMixerPlatformXAudio2::TeardownHardware()
	{
		if (!bIsInitialized)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("XAudio2 was already tore down."), Warning);
			return false;
		}
		
		// Lock prior to changing state to avoid race condition if there happens to be an in-flight device swap
		FScopeLock Lock(&DeviceSwapCriticalSection);

		if (XAudio2System)
		{
			XAudio2System->UnregisterForCallbacks(this);
		}
		SAFE_RELEASE(XAudio2System);

		bIsInitialized = false;

		return true;
	}

	bool FMixerPlatformXAudio2::IsInitialized() const
	{
		return bIsInitialized;
	}

	bool FMixerPlatformXAudio2::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_GetNumOutputDevices, FColor::Blue);

		// Use Cache if we have it.
		if (IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
		{
			OutNumOutputDevices = Cache->GetAllActiveOutputDevices().Num();
			return true;
		}

		OutNumOutputDevices = 0;

		if (!bIsInitialized)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("XAudio2 was not initialized."), Error);
			return false;
		}

#if  XAUDIO_SUPPORTS_DEVICE_DETAILS

		IMMDeviceEnumerator* DeviceEnumerator = nullptr;
		IMMDeviceCollection* DeviceCollection = nullptr;
		bool bSuccess = false;

		XAUDIO2_CALL_AND_HANDLE_ERROR(CoCreateInstance(__uuidof(MMDeviceEnumerator), NULL, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&DeviceEnumerator)), goto Cleanup);

		XAUDIO2_CALL_AND_HANDLE_ERROR(DeviceEnumerator->EnumAudioEndpoints(eRender, DEVICE_STATE_ACTIVE, &DeviceCollection), goto Cleanup);

		uint32 DeviceCount;
		XAUDIO2_CALL_AND_HANDLE_ERROR(DeviceCollection->GetCount(&DeviceCount), goto Cleanup);

		OutNumOutputDevices = DeviceCount;
		bSuccess = true;

	Cleanup:
		SAFE_RELEASE(DeviceCollection);
		SAFE_RELEASE(DeviceEnumerator);

		return bSuccess;
#else
		OutNumOutputDevices = 1;
		return true;
#endif 
	}


	static bool GetMMDeviceInfo(IMMDevice* MMDevice, FAudioPlatformDeviceInfo& OutInfo)
	{
		SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_GetMMDeviceInfo, FColor::Blue);
		
		check(MMDevice);

		OutInfo.Reset();

		bool bSuccess = false;
		IPropertyStore *PropertyStore = nullptr;
		WAVEFORMATEX* WaveFormatEx = nullptr;
		PROPVARIANT FriendlyName;
		PROPVARIANT DeviceFormat;
		LPWSTR DeviceId;

		check(MMDevice);
		PropVariantInit(&FriendlyName);
		PropVariantInit(&DeviceFormat);

		// Get the device id
		XAUDIO2_CALL_AND_HANDLE_ERROR(MMDevice->GetId(&DeviceId), goto Cleanup);

		// Open up the property store so we can read properties from the device
		XAUDIO2_CALL_AND_HANDLE_ERROR(MMDevice->OpenPropertyStore(STGM_READ, &PropertyStore), goto Cleanup);

#if PLATFORM_WINDOWS
		// Grab the friendly name
		PropVariantInit(&FriendlyName);
		XAUDIO2_CALL_AND_HANDLE_ERROR(PropertyStore->GetValue(PKEY_Device_FriendlyName, &FriendlyName), goto Cleanup);
		OutInfo.Name = FString(FriendlyName.pwszVal);
#endif 

		// Retrieve the DeviceFormat prop variant
		XAUDIO2_CALL_AND_HANDLE_ERROR(PropertyStore->GetValue(PKEY_AudioEngine_DeviceFormat, &DeviceFormat), goto Cleanup);

		// Get the format of the property
		WaveFormatEx = (WAVEFORMATEX *)DeviceFormat.blob.pBlobData;
		if (!WaveFormatEx)
		{
			// Some devices don't provide the Device format, so try the OEMFormat as well.
			XAUDIO2_CALL_AND_HANDLE_ERROR(PropertyStore->GetValue(PKEY_AudioEngine_OEMFormat, &DeviceFormat), goto Cleanup);

			WaveFormatEx = (WAVEFORMATEX*)DeviceFormat.blob.pBlobData;
			if (!ensure(DeviceFormat.blob.pBlobData))
			{
				goto Cleanup;
			}
		}

		// We've succeeded at this point.
		bSuccess = true; 
		
		OutInfo.DeviceId = FString(DeviceId);
		OutInfo.NumChannels = FMath::Clamp((int32)WaveFormatEx->nChannels, 2, 8);
		OutInfo.SampleRate = WaveFormatEx->nSamplesPerSec;

		// XAudio2 automatically converts the audio format to output device us so we don't need to do any format conversions
		OutInfo.Format = EAudioMixerStreamDataFormat::Float;

		OutInfo.OutputChannelArray.Reset();

		// Extensible format supports surround sound so we need to parse the channel configuration to build our channel output array
		if (WaveFormatEx->wFormatTag == WAVE_FORMAT_EXTENSIBLE)
		{
			// Cast to the extensible format to get access to extensible data
			const WAVEFORMATEXTENSIBLE* WaveFormatExtensible = (WAVEFORMATEXTENSIBLE*)WaveFormatEx;

			// Loop through the extensible format channel flags in the standard order and build our output channel array
			// From https://msdn.microsoft.com/en-us/library/windows/hardware/dn653308(v=vs.85).aspx
			// The channels in the interleaved stream corresponding to these spatial positions must appear in the order specified above. This holds true even in the 
			// case of a non-contiguous subset of channels. For example, if a stream contains left, bass enhance and right, then channel 1 is left, channel 2 is right, 
			// and channel 3 is bass enhance. This enables the linkage of multi-channel streams to well-defined multi-speaker configurations.

			uint32 ChanCount = 0;
			for (uint32 ChannelTypeIndex = 0; ChannelTypeIndex < EAudioMixerChannel::ChannelTypeCount && ChanCount < (uint32)OutInfo.NumChannels; ++ChannelTypeIndex)
			{
				if (WaveFormatExtensible->dwChannelMask & ChannelTypeMap[ChannelTypeIndex])
				{
					OutInfo.OutputChannelArray.Add((EAudioMixerChannel::Type)ChannelTypeIndex);
					++ChanCount;
				}
			}

			// We didn't match channel masks for all channels, revert to a default ordering
			if (ChanCount < (uint32)OutInfo.NumChannels)
			{
				UE_LOGF(LogAudioMixer, Warning, "Did not find the channel type flags for audio device '%ls'. Reverting to a default channel ordering.", *OutInfo.Name);

				OutInfo.OutputChannelArray.Reset();

				static EAudioMixerChannel::Type DefaultChannelOrdering[] = {
					EAudioMixerChannel::FrontLeft,
					EAudioMixerChannel::FrontRight,
					EAudioMixerChannel::FrontCenter,
					EAudioMixerChannel::LowFrequency,
					EAudioMixerChannel::SideLeft,
					EAudioMixerChannel::SideRight,
					EAudioMixerChannel::BackLeft,
					EAudioMixerChannel::BackRight,
				};

				EAudioMixerChannel::Type* ChannelOrdering = DefaultChannelOrdering;

				// Override channel ordering for some special cases
				if (OutInfo.NumChannels == 4)
				{
					static EAudioMixerChannel::Type DefaultChannelOrderingQuad[] = {
						EAudioMixerChannel::FrontLeft,
						EAudioMixerChannel::FrontRight,
						EAudioMixerChannel::BackLeft,
						EAudioMixerChannel::BackRight,
					};

					ChannelOrdering = DefaultChannelOrderingQuad;
				}
				else if (OutInfo.NumChannels == 6)
				{
					static EAudioMixerChannel::Type DefaultChannelOrdering51[] = {
						EAudioMixerChannel::FrontLeft,
						EAudioMixerChannel::FrontRight,
						EAudioMixerChannel::FrontCenter,
						EAudioMixerChannel::LowFrequency,
						EAudioMixerChannel::BackLeft,
						EAudioMixerChannel::BackRight,
					};

					ChannelOrdering = DefaultChannelOrdering51;
				}

				check(OutInfo.NumChannels <= 8);
				for (int32 Index = 0; Index < OutInfo.NumChannels; ++Index)
				{
					OutInfo.OutputChannelArray.Add(ChannelOrdering[Index]);
				}
			}
		}
		else
		{
			// Non-extensible formats only support mono or stereo channel output
			OutInfo.OutputChannelArray.Add(EAudioMixerChannel::FrontLeft);
			if (OutInfo.NumChannels == 2)
			{
				OutInfo.OutputChannelArray.Add(EAudioMixerChannel::FrontRight);
			}
		}

	Cleanup:
		PropVariantClear(&FriendlyName);
		PropVariantClear(&DeviceFormat);
		SAFE_RELEASE(PropertyStore);

		return bSuccess;
	}

	bool FMixerPlatformXAudio2::GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo)
	{
		SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_GetOutputDeviceInfo, FColor::Blue);
				
		// Use Cache if we have it. (index is a bad way to find the device, but we do it here).
		if (IAudioPlatformDeviceInfoCache* Cache = GetDeviceInfoCache())
		{
			if (InDeviceIndex == AUDIO_MIXER_DEFAULT_DEVICE_INDEX)
			{
				if (TOptional<FAudioPlatformDeviceInfo> Defaults = Cache->FindDefaultOutputDevice())
				{
					OutInfo = *Defaults;
					return true;
				}
			}
			else
			{
				TArray<FAudioPlatformDeviceInfo> ActiveDevices = Cache->GetAllActiveOutputDevices();
				if (ActiveDevices.IsValidIndex(InDeviceIndex))
				{
					OutInfo = ActiveDevices[InDeviceIndex];
					return true;
				}
			}
			return false;
		}
		
		if (!bIsInitialized)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("XAudio2 was not initialized."), Error);
			return false;
		}

		IMMDeviceEnumerator* DeviceEnumerator = nullptr;
		IMMDeviceCollection* DeviceCollection = nullptr;
		IMMDevice* DefaultDevice = nullptr;
		IMMDevice* Device = nullptr;
		bool bIsDefault = false;
		bool bSucceeded = false;

		XAUDIO2_CALL_AND_HANDLE_ERROR(CoCreateInstance(__uuidof(MMDeviceEnumerator), NULL, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&DeviceEnumerator)), goto Cleanup);

		XAUDIO2_CALL_AND_HANDLE_ERROR(DeviceEnumerator->EnumAudioEndpoints(eRender, DEVICE_STATE_ACTIVE, &DeviceCollection), goto Cleanup);

		uint32 DeviceCount;
		XAUDIO2_CALL_AND_HANDLE_ERROR(DeviceCollection->GetCount(&DeviceCount), goto Cleanup);

		if (DeviceCount == 0)
		{
			UE_LOGF(LogAudioMixer, Warning, "No available audio device");
			goto Cleanup;
		}

		// Get the default device
		XAUDIO2_CALL_AND_HANDLE_ERROR(DeviceEnumerator->GetDefaultAudioEndpoint(eRender, eMultimedia, &DefaultDevice), goto Cleanup);

		// If we are asking to get info on default device
		if (InDeviceIndex == AUDIO_MIXER_DEFAULT_DEVICE_INDEX)
		{
			Device = DefaultDevice;
			bIsDefault = true;
		}
		// Make sure we're not asking for a bad device index
		else if (InDeviceIndex >= DeviceCount)
		{
			UE_LOGF(LogAudioMixer, Error, "Requested device index (%d) is larger than the number of devices available (%d)", InDeviceIndex, DeviceCount);
			goto Cleanup;
		}
		else
		{
			XAUDIO2_CALL_AND_HANDLE_ERROR(DeviceCollection->Item(InDeviceIndex, &Device), goto Cleanup);
		}

		if (ensure(Device))
		{
			bSucceeded = GetMMDeviceInfo(Device, OutInfo);

			// Fix up if this was a default device
			if (bIsDefault)
			{
				OutInfo.bIsSystemDefault = true;
			}
			else if(DefaultDevice)
			{
				FAudioPlatformDeviceInfo DefaultInfo;
				GetMMDeviceInfo(DefaultDevice, DefaultInfo);
				OutInfo.bIsSystemDefault = OutInfo.DeviceId == DefaultInfo.DeviceId;
			}
		}

	Cleanup:
		SAFE_RELEASE(Device);
		SAFE_RELEASE(DeviceCollection);
		SAFE_RELEASE(DeviceEnumerator);

		return bSucceeded;

	}

	bool FMixerPlatformXAudio2::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		OutDefaultDeviceIndex = AUDIO_MIXER_DEFAULT_DEVICE_INDEX;
		return true;
	}

	bool FMixerPlatformXAudio2::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		if (!bIsInitialized)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("XAudio2 was not initialized."), Error);
			return false;
		}

		if (bIsDeviceOpen)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("XAudio2 audio stream already opened."), Warning);
			return false;
		}

		check(XAudio2System);
		check(OutputAudioStreamMasteringVoice == nullptr);

		WAVEFORMATEX Format = { 0 };

		OpenStreamParams = Params;

		AudioStreamInfo.Reset();

		AudioStreamInfo.OutputDeviceIndex = OpenStreamParams.OutputDeviceIndex;
		AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
		AudioStreamInfo.NumBuffers = OpenStreamParams.NumBuffers;
		AudioStreamInfo.AudioMixer = OpenStreamParams.AudioMixer;

		uint32 NumOutputDevices = 0;

		if (GetNumOutputDevices(NumOutputDevices) && NumOutputDevices > 0)
		{
			if (!GetOutputDeviceInfo(AudioStreamInfo.OutputDeviceIndex, AudioStreamInfo.DeviceInfo))
			{
				return false;
			}

			// Store the device ID here in case it is removed. We can switch back if the device comes back.
			if (Params.bRestoreIfRemoved)
			{
				SetOriginalAudioDeviceId(AudioStreamInfo.DeviceInfo.DeviceId);
			}

			// Passing the device-id to CreateMasteringVoice on a non-windows platform, will prevent
			// the creation of virtualized device which handles disconnection state for us, but
			// if we need to handle these errors i.e. OnCriticalError callback, we need to pass the device here.
			OutputAudioStreamMasteringVoice = CreateMasteringVoice(*XAudio2System, AudioStreamInfo.DeviceInfo, ShouldUseDefaultDevice());
			if (!OutputAudioStreamMasteringVoice)
			{
				goto Cleanup;
			}

			// Start the xaudio2 engine running, which will now allow us to start feeding audio to it
			XAUDIO2_CALL_AND_HANDLE_ERROR(XAudio2System->StartEngine(), goto Cleanup);
				
			// Setup the format of the output source voice
			Format.nChannels = AudioStreamInfo.DeviceInfo.NumChannels;
			Format.nSamplesPerSec = Params.SampleRate;
			Format.wFormatTag = WAVE_FORMAT_IEEE_FLOAT;
			Format.nAvgBytesPerSec = Format.nSamplesPerSec * sizeof(float) * Format.nChannels;
			Format.nBlockAlign = sizeof(float) * Format.nChannels;
			Format.wBitsPerSample = sizeof(float) * 8;

			// Create the output source voice
			XAUDIO2_CALL_AND_HANDLE_ERROR(XAudio2System->CreateSourceVoice(&OutputAudioStreamSourceVoice, &Format, XAUDIO2_VOICE_NOPITCH, 2.0f, &OutputVoiceCallback), goto Cleanup);
		}
	Cleanup:

		bool bXAudioOpenSuccessfully = OutputAudioStreamSourceVoice && OutputAudioStreamMasteringVoice;
		if (!bXAudioOpenSuccessfully)
		{
			// Undo anything we created.
			if (OutputAudioStreamSourceVoice)
			{
				OutputAudioStreamSourceVoice->DestroyVoice();
				OutputAudioStreamSourceVoice = nullptr;
			}		
			if (OutputAudioStreamMasteringVoice)
			{
				OutputAudioStreamMasteringVoice->DestroyVoice();
				OutputAudioStreamMasteringVoice = nullptr;
			}

			// Setup for running null device.
			AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
			AudioStreamInfo.DeviceInfo.OutputChannelArray = { EAudioMixerChannel::FrontLeft, EAudioMixerChannel::FrontRight };
			AudioStreamInfo.DeviceInfo.NumChannels = 2;
			AudioStreamInfo.DeviceInfo.SampleRate = OpenStreamParams.SampleRate;
			AudioStreamInfo.DeviceInfo.Format = EAudioMixerStreamDataFormat::Float;
		}

#if PLATFORM_WINDOWS || (1) // Currently all targets do this.
		if(!bXAudioOpenSuccessfully)
		{
			// On Windows where we can have audio devices unplugged/removed/hot-swapped:
			// We must mark ourselves open, even if we failed to open. This will allow the device-swap logic to run.
			// StartAudioStream will happily use the null renderer path if there's no real stream open.
			bXAudioOpenSuccessfully = true;
		}
#endif //PLATFORM_WINDOWS		

		// If we opened, mark the stream as open.
		if(bXAudioOpenSuccessfully)
		{
			AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;
			bIsDeviceOpen = true;
		}	

		return bXAudioOpenSuccessfully;
	}

	FAudioPlatformDeviceInfo FMixerPlatformXAudio2::GetPlatformDeviceInfo() const
	{
		return AudioStreamInfo.DeviceInfo;
	}

	FString FMixerPlatformXAudio2::GetCurrentDeviceName() const
	{
		return AudioStreamInfo.DeviceInfo.Name;
	}

	bool FMixerPlatformXAudio2::CloseAudioStream()
	{
		if (!bIsInitialized || AudioStreamInfo.StreamState == EAudioOutputStreamState::Closed)
		{
			return false;
		}

		// Lock prior to changing state to avoid race condition if there happens to be an in-flight device swap
		FScopeLock Lock(&DeviceSwapCriticalSection);

		// If we're closing the stream, we're not interested in the results of the device swap. 
		// Reset the handle to the future.
		ResetActiveDeviceSwapFuture();
			
		if (bIsDeviceOpen && !StopAudioStream())
		{
			return false;
		}

		if (XAudio2System)
		{
			XAudio2System->StopEngine();
		}

		if (OutputAudioStreamSourceVoice)
		{
			OutputAudioStreamSourceVoice->DestroyVoice();
			OutputAudioStreamSourceVoice = nullptr;
		}

		if (OutputAudioStreamMasteringVoice)
		{
			OutputAudioStreamMasteringVoice->DestroyVoice();
			OutputAudioStreamMasteringVoice = nullptr;
		}

		if (bIsUsingNullDevice)
		{
			StopRunningNullDevice();
		}

		bIsDeviceOpen = false;

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;

		return true;
	}

	bool FMixerPlatformXAudio2::StartAudioStream()
	{
		UE_LOGF(LogAudioMixer, Log, "FMixerPlatformXAudio2::StartAudioStream() called. InstanceID=%d", InstanceID);

		// If we already have a source voice, we can just restart it
		if (OutputAudioStreamSourceVoice)
		{
			OutputAudioStreamSourceVoice->Start();
		}
		else
		{
			check(!bIsUsingNullDevice);
			StartRunningNullDevice();
		}
		
		// Start generating audio with our output source voice
		// Can be called during device swap when AudioRenderEvent can be null
		if (nullptr == AudioRenderEvent)
		{
			// This will set AudioStreamInfo.StreamState to EAudioOutputStreamState::Running
			BeginGeneratingAudio();
		}
		else
		{
			AudioStreamInfo.StreamState = EAudioOutputStreamState::Running;
		}

		return true;
	}

	bool FMixerPlatformXAudio2::StopAudioStream()
	{
		if (!bIsInitialized)
		{
			AUDIO_PLATFORM_LOG_ONCE(TEXT("XAudio2 was not initialized."), Warning);
			return false;
		}

		// Lock prior to changing state to avoid race condition if there happens to be an in-flight device swap
		FScopeLock Lock(&DeviceSwapCriticalSection);

		UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::StopAudioStream() called. InstanceID=%d, StreamState=%d",
			InstanceID, static_cast<int32>(AudioStreamInfo.StreamState));

		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped && AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			// Shutdown the AudioRenderThread if we're running or mid-device swap
			if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running ||
				AudioStreamInfo.StreamState == EAudioOutputStreamState::SwappingDevice)
			{
				StopGeneratingAudio();
			}

			// Signal that the thread that is running the update that we're stopping
			if (OutputAudioStreamSourceVoice)
			{
				OutputAudioStreamSourceVoice->Stop(0, 0); // Don't wait for tails, stop as quick as you can.
			}

			check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);
		}

		return true;
	}

	bool FMixerPlatformXAudio2::CheckAudioDeviceChange()
	{
#if XAUDIO_SUPPORTS_DEVICE_DETAILS
		return FAudioMixerPlatformSwappable::CheckAudioDeviceChange();
#else
		return false;
#endif
	}
	
	bool FMixerPlatformXAudio2::MoveAudioStreamToNewAudioDevice()
	{
		bool bDidStopGeneratingAudio = false;
#if XAUDIO_SUPPORTS_DEVICE_DETAILS
		bDidStopGeneratingAudio = FAudioMixerPlatformSwappable::MoveAudioStreamToNewAudioDevice();
#endif // XAUDIO_SUPPORTS_DEVICE_DETAILS
		return bDidStopGeneratingAudio;
	}
	
	void FMixerPlatformXAudio2::SubmitBuffer(const uint8* Buffer)
	{
		SCOPED_NAMED_EVENT(FMixerPlatformXAudio2_SubmitBuffer, FColor::Blue);

		if (OutputAudioStreamSourceVoice)
		{
			// Create a new xaudio2 buffer submission
			XAUDIO2_BUFFER XAudio2Buffer = { 0 };
			XAudio2Buffer.AudioBytes = OpenStreamParams.NumFrames * AudioStreamInfo.DeviceInfo.NumChannels * sizeof(float);
			XAudio2Buffer.pAudioData = (const BYTE*)Buffer;
			XAudio2Buffer.pContext = this;

			// Submit buffer to the output streaming voice
			OutputAudioStreamSourceVoice->SubmitSourceBuffer(&XAudio2Buffer);

			if(!FirstBufferSubmitted)
			{
				UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::SubmitBuffer() called for the first time. InstanceID=%d", InstanceID);
				FirstBufferSubmitted = true;
			}
		}
	}

	bool FMixerPlatformXAudio2::InitializeDeviceSwapContext(const FString& InRequestedDeviceID, const TCHAR* InReason)
	{
		check(GetDeviceInfoCache());

		// Look up device. Blank name looks up current default.
		const FName NewDeviceName = *InRequestedDeviceID;
		TOptional<FAudioPlatformDeviceInfo> DeviceInfo;
		
		if (TOptional<FAudioPlatformDeviceInfo> TempDeviceInfo = GetDeviceInfoCache()->FindActiveOutputDevice(NewDeviceName))
		{
			if (TempDeviceInfo.IsSet())
			{
				if (IsDeviceInfoValid(*TempDeviceInfo))
				{
					DeviceInfo = MoveTemp(TempDeviceInfo);
				}
				else
				{
					UE_LOGF(LogAudioMixer, Warning, "Ignoring attempt to switch to device with unsupported params: Channels=%u, SampleRate=%u, Id=%ls, Name=%ls",
						(uint32)TempDeviceInfo->NumChannels, (uint32)TempDeviceInfo->SampleRate, *TempDeviceInfo->DeviceId, *TempDeviceInfo->Name);
					return false;
				}
			}
		}
		
		return InitDeviceSwapContextInternal(InRequestedDeviceID, InReason, DeviceInfo);
	}

	bool FMixerPlatformXAudio2::InitDeviceSwapContextInternal(const FString& InRequestedDeviceID, const TCHAR* InReason, const TOptional<FAudioPlatformDeviceInfo>& InDeviceInfo)
	{
		// Access to device swap context must be protected by DeviceSwapCriticalSection
		FScopeLock Lock(&DeviceSwapCriticalSection);

		if (DeviceSwapContext.IsValid())
		{
			UE_LOGF(LogAudioMixer, Display, "FMixerPlatformXAudio2::InitDeviceSwapContextInternal DeviceSwapContext in-flight, ignoring");
			return false;
		}
		
		// Create the device swap context which will be valid over the course of the swap
		DeviceSwapContext = MakeUnique<FXAudio2DeviceSwapContext>(InRequestedDeviceID, InReason);
		if (!DeviceSwapContext.IsValid())
		{
			UE_LOGF(LogAudioMixer, Warning, "FMixerPlatformXAudio2::InitDeviceSwapContextInternal failed to create DeviceSwapContext");
			return false;
		}

		DeviceSwapContext->NewDevice = InDeviceInfo;
		DeviceSwapContext->bUseDefaultDevice = ShouldUseDefaultDevice();

		return true;
	}
	
	FString FMixerPlatformXAudio2::GetDefaultDeviceName()
	{
		return FString();
	}

	FAudioPlatformSettings FMixerPlatformXAudio2::GetPlatformSettings() const
	{
#if WITH_ENGINE
		return FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
#else
		return FAudioPlatformSettings();
#endif // WITH_ENGINE
	}

	void FMixerPlatformXAudio2::OnHardwareUpdate()
	{
	}

	Audio::IAudioPlatformDeviceInfoCache* FMixerPlatformXAudio2::GetDeviceInfoCache() const
	{
		if (ShouldUseDeviceInfoCache())
		{
			return DeviceInfoCache.Get();
		}
		// Disabled.
		return nullptr;
	}

	bool FMixerPlatformXAudio2::IsDeviceInfoValid(const FAudioPlatformDeviceInfo& InDeviceInfo) const
	{
		if (InDeviceInfo.NumChannels <= XAUDIO2_MAX_AUDIO_CHANNELS &&
			InDeviceInfo.SampleRate >= XAUDIO2_MIN_SAMPLE_RATE &&
			InDeviceInfo.SampleRate <= XAUDIO2_MAX_SAMPLE_RATE)
		{
			return true;
		}

		return false;
	}

	void FMixerPlatformXAudio2::OnSessionDisconnect(IAudioMixerDeviceChangedListener::EDisconnectReason InReason)
	{
		// Device has disconnected from current session.
		if (InReason == IAudioMixerDeviceChangedListener::EDisconnectReason::FormatChanged)
		{
			// OnFormatChanged, retry again same device.
			RequestDeviceSwap(GetDeviceId(), /*force*/ true, TEXT("FMixerPlatformXAudio2::OnSessionDisconnect() - FormatChanged"));		
		}
		else if (InReason == IAudioMixerDeviceChangedListener::EDisconnectReason::DeviceRemoval)
		{
			// Ignore Device Removal, as this is handle by the Device Removal logic in the Notification Client.
		}
		else
		{
			// ServerShutdown, SessionLogoff, SessionDisconnected, ExclusiveModeOverride
			// Attempt a default swap, will likely fail, but then we'll switch to a null device.
			RequestDeviceSwap(TEXT(""), /*force*/ true, TEXT("FMixerPlatformXAudio2::OnSessionDisconnect() - Other"));
		}
	}

	IXAudio2MasteringVoice* FMixerPlatformXAudio2::CreateMasteringVoice(IXAudio2& InXAudio2System, const FAudioPlatformDeviceInfo& NewDevice, bool bUseDefaultDevice)
	{
		IXAudio2MasteringVoice* MasteringVoice = nullptr;
		const TCHAR* DeviceId = bUseDefaultDevice ? nullptr : *NewDevice.DeviceId;
		const HRESULT Result = InXAudio2System.CreateMasteringVoice(
			&MasteringVoice,
			NewDevice.NumChannels,
			NewDevice.SampleRate,
			0,
			DeviceId,
			nullptr,
			AudioCategory_GameEffects);
		if (FAILED(Result))
		{
			if (MasteringVoice)
			{
				// Probably unreachable, but just to be safe...
				MasteringVoice->DestroyVoice();
				MasteringVoice = nullptr;
			}

			const TCHAR* DeviceIdDebug = DeviceId ? DeviceId : TEXT("(default)");
			UE_LOGF(LogAudioMixer, Error, "CreateMasteringVoice failed with result 0x%X: %ls (line: %d) with Args (NumChannels=%u, SampleRate=%u, DeviceID=%ls, Name=%ls)",
				Result, *Audio::ToErrorFString(Result), __LINE__, NewDevice.NumChannels, NewDevice.SampleRate, DeviceIdDebug, *NewDevice.Name);
		}

		return MasteringVoice;
	}

	void FMixerPlatformXAudio2::OnProcessingPassStart()
	{
	}

	void FMixerPlatformXAudio2::OnProcessingPassEnd()
	{
	}

	void FMixerPlatformXAudio2::OnCriticalError(HRESULT Error)
	{
		// Windows should handle this via session events, log if we receive one.
		UE_LOGF(LogAudioMixer, Warning, "FMixerPlatformXAudio2::OnCriticalError: 0x%X: %ls", Error, *Audio::ToErrorFString(Error));
	}
	
}

