// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <xaudio2.h>
#include "Windows/HideWindowsPlatformTypes.h"

#include "Async/Future.h"

#ifndef XAUDIO_SUPPORTS_DEVICE_DETAILS
    #define XAUDIO_SUPPORTS_DEVICE_DETAILS		1
#endif	//XAUDIO_SUPPORTS_DEVICE_DETAILS

// Any platform defines
namespace Audio
{
	class FMixerPlatformXAudio2;

	/**
	* FXAudio2VoiceCallback
	* XAudio2 implementation of IXAudio2VoiceCallback
	* This callback class is used to get event notifications on buffer end (when a buffer has finished processing).
	* This is used to signal the I/O thread that it can request another buffer from the user callback.
	*/
	class FXAudio2VoiceCallback final : public IXAudio2VoiceCallback
	{
	public:
		FXAudio2VoiceCallback() {}
		~FXAudio2VoiceCallback() = default;

	private:
		void STDCALL OnVoiceProcessingPassStart(UINT32 BytesRequired) {}
		void STDCALL OnVoiceProcessingPassEnd() {}
		void STDCALL OnStreamEnd() {}
		void STDCALL OnBufferStart(void* BufferContext) {}
		void STDCALL OnLoopEnd(void* BufferContext) {}
		void STDCALL OnVoiceError(void* BufferContext, HRESULT Error) {}

		void STDCALL OnBufferEnd(void* BufferContext);

	};

	struct FXAudio2DeviceSwapContext : public FDeviceSwapContext
	{
		FXAudio2DeviceSwapContext() = delete;
		FXAudio2DeviceSwapContext(const FString& InRequestedDeviceID, const FString& InReason) :
			FDeviceSwapContext(InRequestedDeviceID, InReason)
		{}
		
		bool bUseDefaultDevice = false;
		IXAudio2* PreviousSystem = nullptr;
		IXAudio2MasteringVoice* PreviousMasteringVoice = nullptr;
		IXAudio2SourceVoice* PreviousSourceVoice = nullptr;
		FXAudio2VoiceCallback* Callbacks = nullptr;
		uint32 RenderingSampleRate = 0;
	};

	struct FXAudio2DeviceSwapResult : public FDeviceSwapResult
	{
		virtual bool IsNewDeviceReady() const override
		{
			return NewSystem && NewMasteringVoice && NewSourceVoice;
		}

		IXAudio2* NewSystem = nullptr;
		IXAudio2MasteringVoice* NewMasteringVoice = nullptr;
		IXAudio2SourceVoice* NewSourceVoice = nullptr;
	};

	class FMixerPlatformXAudio2 : public FAudioMixerPlatformSwappable,
								  public IXAudio2EngineCallback
	{

	public:

		FMixerPlatformXAudio2();
		virtual ~FMixerPlatformXAudio2() override;

		//~ Begin IAudioMixerPlatformInterface
		virtual FString GetPlatformApi() const override { return TEXT("XAudio2"); }
		virtual bool InitializeHardware() override;
		virtual bool CheckAudioDeviceChange() override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		virtual bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		virtual FString GetCurrentDeviceName() const override;
		virtual bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual bool MoveAudioStreamToNewAudioDevice() override;
		
		virtual FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		virtual void SubmitBuffer(const uint8* Buffer) override;
		virtual FString GetDefaultDeviceName() override;
		virtual FAudioPlatformSettings GetPlatformSettings() const override;
		virtual void OnHardwareUpdate() override;
		virtual IAudioPlatformDeviceInfoCache* GetDeviceInfoCache() const override;
		virtual bool IsDeviceInfoValid(const FAudioPlatformDeviceInfo& InDeviceInfo) const override;
		virtual bool ShouldUseDeviceInfoCache() const override { return true; }
		virtual void ResumePlaybackOnNewDevice() override;
		//~ End IAudioMixerPlatformInterface

		//~ Begin IAudioMixerDeviceChangedListener
		virtual void RegisterDeviceChangedListener() override;
		virtual void UnregisterDeviceChangedListener() override;
		virtual void OnDefaultCaptureDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;
		virtual void OnDefaultRenderDeviceChanged(const EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;
		virtual void OnDeviceAdded(const FString& DeviceId, bool bIsRenderDevice) override;
		virtual void OnDeviceRemoved(const FString& DeviceId, bool bIsRenderDevice) override;
		virtual void OnDeviceStateChanged(const FString& DeviceId, const EAudioDeviceState InState, bool bIsRenderDevice) override;
		virtual void OnSessionDisconnect(Audio::IAudioMixerDeviceChangedListener::EDisconnectReason InReason) override;
		virtual FString GetDeviceId() const override;		
		//~ End IAudioMixerDeviceChangedListener

		//~ Begin FAudioMixerPlatformSwappable
		virtual bool InitializeDeviceSwapContext(const FString& InRequestedDeviceID, const TCHAR* InReason) override;
		virtual bool CheckThreadedDeviceSwap() override;
		virtual bool PreDeviceSwap() override;
		virtual void EnqueueAsyncDeviceSwap() override;
		virtual void SynchronousDeviceSwap() override;
		virtual bool PostDeviceSwap() override;
		//~ End FAudioMixerPlatformSwappable
	
	protected:
		virtual uint32 GetCreateFlags() const { return 0; }
		virtual bool ShouldUseDefaultDevice() const { return false; }

		static IXAudio2MasteringVoice* CreateMasteringVoice(IXAudio2& InXAudio2System, const FAudioPlatformDeviceInfo& NewDevice, bool bUseDefaultDevice);

		//~ Begin IXAudio2EngineCallback
		virtual void OnCriticalError(HRESULT Error) override;
		virtual void OnProcessingPassStart() override;
		virtual void OnProcessingPassEnd() override;
		//~ End IXAudio2EngineCallback
		
		// Used to teardown and reinitialize XAudio2.
		// This must be done to repopulate the playback device list in XAudio 2.7.
		bool ResetXAudio2System();

		/** Can be used by subclasses to initialize a device swap context by supplying a specific 
		 *  FAudioPlatformDeviceInfo rather than looking it up via the requested device Id.
		 */
		bool InitDeviceSwapContextInternal(const FString& InRequestedDeviceID, const TCHAR* InReason, const TOptional<FAudioPlatformDeviceInfo>& InDeviceInfo);

		// Bool indicating that the default audio device changed
		// And that we need to restart the audio device.
		UE_DEPRECATED(5.6, "bDeviceChanged has been deprecated.")
		FThreadSafeBool bDeviceChanged;

		IXAudio2* XAudio2System;
		IXAudio2MasteringVoice* OutputAudioStreamMasteringVoice;
		IXAudio2SourceVoice* OutputAudioStreamSourceVoice;
		FXAudio2VoiceCallback OutputVoiceCallback;
		
		// When we are running the null device,
		// we check whether a new audio device was connected every second or so.
		float TimeSinceNullDeviceWasLastChecked;

		bool FirstBufferSubmitted{false};

		TUniquePtr<IAudioPlatformDeviceInfoCache> DeviceInfoCache;

		uint32 bIsInitialized : 1;
		uint32 bIsDeviceOpen : 1;

	private:
		/** Context object holding state used during device swap. */
		TUniquePtr<FXAudio2DeviceSwapContext> DeviceSwapContext;

		/** Performs a device swap with the given context. Static method enforces no other state sharing occurs. */
		static TUniquePtr<FDeviceSwapResult> PerformDeviceSwap(TUniquePtr<FXAudio2DeviceSwapContext>&& InDeviceContext);
	};

}

