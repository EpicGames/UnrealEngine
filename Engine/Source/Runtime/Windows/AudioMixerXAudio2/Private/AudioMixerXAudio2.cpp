// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerXAudio2.h"
#include "AudioMixerPlatformXAudio2.h"
#include "AudioMixer.h"
#include "Modules/ModuleManager.h"

void FAudioMixerModuleXAudio2::StartupModule() 
{
	IAudioDeviceModule::StartupModule();

	FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixer"));
	FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
}

bool FAudioMixerModuleXAudio2::IsAudioMixerModule() const
{
	return true;
}

Audio::IAudioMixerPlatformInterface* FAudioMixerModuleXAudio2::CreateAudioMixerPlatformInterface() 
{
	return new Audio::FMixerPlatformXAudio2();
}

#if PLATFORM_WINDOWS
IMPLEMENT_MODULE(FAudioMixerModuleXAudio2, AudioMixerXAudio2);
#endif //PLATFORM_WINDOWS
