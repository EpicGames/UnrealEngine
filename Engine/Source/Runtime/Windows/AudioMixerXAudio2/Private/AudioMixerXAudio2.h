// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "Modules/ModuleManager.h"

class FAudioMixerModuleXAudio2 : public IAudioDeviceModule
{
public:
	virtual void StartupModule() override;
	virtual bool IsAudioMixerModule() const override;
	virtual Audio::IAudioMixerPlatformInterface* CreateAudioMixerPlatformInterface() override;
};
