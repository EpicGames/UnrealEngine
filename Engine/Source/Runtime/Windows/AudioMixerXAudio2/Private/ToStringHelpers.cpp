// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToStringHelpers.h"
#include "WindowsMMStringUtils.h"

namespace Audio
{
	FString ToErrorFString(HRESULT InResult)
	{
#define CASE_AND_STRING(RESULT) case HRESULT(RESULT): return TEXT(#RESULT)

		switch (InResult)
		{
		case HRESULT(XAUDIO2_E_INVALID_CALL):			return TEXT("XAUDIO2_E_INVALID_CALL");
		case HRESULT(XAUDIO2_E_XMA_DECODER_ERROR):		return TEXT("XAUDIO2_E_XMA_DECODER_ERROR");
		case HRESULT(XAUDIO2_E_XAPO_CREATION_FAILED):	return TEXT("XAUDIO2_E_XAPO_CREATION_FAILED");
		case HRESULT(XAUDIO2_E_DEVICE_INVALIDATED):		return TEXT("XAUDIO2_E_DEVICE_INVALIDATED");

		default:
		{
			// Not an XAudio2 error, check for Audio Client errors
			return AudioClientErrorToFString(InResult);
		}
		}
	}
}

