// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

[SupportedPlatformGroups("Windows")]
public class D3D11RHI : ModuleRules
{
	public D3D11RHI(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddAll(
			"Shaders"
		);

		PrivateDependencyModuleNames.AddAll(
			"CoreUObject",
			"Engine",
			"RHICore",
			"RenderCore",
			"TraceLog"
		);

		PublicIncludePathModuleNames.AddAll(
			"DX11"
		);

		PublicDependencyModuleNames.AddAll(
			"Core",
			"RHI"
		);

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			PublicDependencyModuleNames.Add("WindowsD3D");
			PrivateDependencyModuleNames.Add("HeadMountedDisplay");
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "AMD_AGS");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "IntelExtensionsFramework");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAftermath");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAPI");
	}
}
