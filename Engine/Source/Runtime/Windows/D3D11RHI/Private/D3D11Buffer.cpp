// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11IndexBuffer.cpp: D3D Index buffer RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "RHICoreBufferInitializer.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

TAutoConsoleVariable<int32> GCVarUseSharedKeyedMutex(
	TEXT("r.D3D11.UseSharedKeyMutex"),
	0,
	TEXT("If 1, BUF_Shared vertex / index buffer and TexCreate_Shared texture will be created\n")
	TEXT("with the D3D11_RESOURCE_MISC_SHARED_KEYEDMUTEX flag instead of D3D11_RESOURCE_MISC_SHARED (default).\n"),
	ECVF_Default);

FD3D11Buffer::~FD3D11Buffer()
{
	D3D11BufferStats::UpdateBufferStats(*this, false);
}

void FD3D11Buffer::TakeOwnership(FD3D11Buffer& Other)
{
	FRHIBuffer::TakeOwnership(Other);
	Resource = MoveTemp(Other.Resource);
}

void FD3D11Buffer::ReleaseOwnership()
{
	FRHIBuffer::ReleaseOwnership();

	D3D11BufferStats::UpdateBufferStats(*this, false);
	Resource = nullptr;
}

static D3D11_BUFFER_DESC D3D11DescFromBufferDesc(const FRHIBufferDesc& BufferDesc)
{
	D3D11_BUFFER_DESC Desc{};
	Desc.ByteWidth = BufferDesc.Size;

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::AnyDynamic))
	{
		Desc.Usage = D3D11_USAGE_DYNAMIC;
		Desc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE;
	}

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::VertexBuffer))
	{
		Desc.BindFlags |= D3D11_BIND_VERTEX_BUFFER;
	}

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::IndexBuffer))
	{
		Desc.BindFlags |= D3D11_BIND_INDEX_BUFFER;
	}

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::ByteAddressBuffer))
	{
		Desc.MiscFlags |= D3D11_RESOURCE_MISC_BUFFER_ALLOW_RAW_VIEWS;
	}
	else if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::StructuredBuffer))
	{
		Desc.StructureByteStride = BufferDesc.Stride;
		Desc.MiscFlags |= D3D11_RESOURCE_MISC_BUFFER_STRUCTURED;
	}

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::ShaderResource))
	{
		Desc.BindFlags |= D3D11_BIND_SHADER_RESOURCE;
	}

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::UnorderedAccess))
	{
		Desc.BindFlags |= D3D11_BIND_UNORDERED_ACCESS;
	}

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::DrawIndirect))
	{
		Desc.MiscFlags |= D3D11_RESOURCE_MISC_DRAWINDIRECT_ARGS;
	}

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::Shared))
	{
		if (GCVarUseSharedKeyedMutex->GetInt() != 0)
		{
			Desc.MiscFlags |= D3D11_RESOURCE_MISC_SHARED_KEYEDMUTEX;
		}
		else
		{
			Desc.MiscFlags |= D3D11_RESOURCE_MISC_SHARED;
		}
	}

	return Desc;
}

FD3D11Buffer* FD3D11DynamicRHI::BeginCreateBufferInternal(const FRHIBufferCreateDesc& CreateDesc)
{
	// Explicitly check that the size is nonzero before allowing CreateBuffer to opaquely fail.
	checkf(CreateDesc.Size > 0 || CreateDesc.IsNull(), TEXT("Attempt to create buffer '%s' with size 0."), CreateDesc.DebugName ? CreateDesc.DebugName : TEXT("(null)"));

	return new FD3D11Buffer(nullptr, CreateDesc);
}

void FD3D11DynamicRHI::FinalizeCreateBufferInternal(FD3D11Buffer* Buffer, TConstArrayView<uint8> InitialData)
{
	const FRHIBufferDesc& BufferDesc = Buffer->GetDesc();
	const D3D11_BUFFER_DESC Desc = D3D11DescFromBufferDesc(BufferDesc);

	check(!BufferDesc.IsNull());

	// If a resource array was provided for the resource, create the resource pre-populated
	D3D11_SUBRESOURCE_DATA InitData{};
	D3D11_SUBRESOURCE_DATA* pInitData = nullptr;
	if (InitialData.Num())
	{
		check(BufferDesc.Size == InitialData.Num());
		InitData.pSysMem = InitialData.GetData();
		InitData.SysMemPitch = InitialData.Num();

		pInitData = &InitData;
	}

	TRefCountPtr<ID3D11Buffer> BufferResource;
	{
		HRESULT hr = Direct3DDevice->CreateBuffer(&Desc, pInitData, BufferResource.GetInitReference());
		if (FAILED(hr))
		{
			const FName BufferName = Buffer->GetName();

			UE_LOGF(LogD3D11RHI, Error, "Failed to create buffer '%ls' with ByteWidth=%u, Usage=%d, BindFlags=0x%x, CPUAccessFlags=0x%x, MiscFlags=0x%x, StructureByteStride=%u, InitData=0x%p",
				*BufferName.ToString(), Desc.ByteWidth, Desc.Usage, Desc.BindFlags, Desc.CPUAccessFlags, Desc.MiscFlags, Desc.StructureByteStride, pInitData);
			VerifyD3D11Result(hr, "CreateBuffer", __FILE__, __LINE__, Direct3DDevice);
		}
	}

#if RHI_USE_RESOURCE_DEBUG_NAME
	if (const FName BufferName = Buffer->GetName(); BufferName != NAME_None)
	{
		const FAnsiString BufferNameString(BufferName.ToString());
		SetD3D11ObjectName(BufferResource, BufferNameString);
	}
#endif

	Buffer->Resource = BufferResource;

	D3D11BufferStats::UpdateBufferStats(*Buffer, true);
}

FD3D11Buffer* FD3D11DynamicRHI::CreateBufferInternal(const FRHIBufferCreateDesc& CreateDesc, TConstArrayView<uint8> InitialData)
{
	FD3D11Buffer* Buffer = BeginCreateBufferInternal(CreateDesc);
	FinalizeCreateBufferInternal(Buffer, InitialData);
	return Buffer;
}

FRHIBufferInitializer FD3D11DynamicRHI::RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	if (CreateDesc.IsNull())
	{
		FD3D11Buffer* Buffer = BeginCreateBufferInternal(CreateDesc);
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Default)
	{
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, CreateBufferInternal(CreateDesc, {}));
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray)
	{
		check(CreateDesc.InitialData);

		FD3D11Buffer* Buffer = CreateBufferInternal(CreateDesc, CreateDesc.InitialData->GetResourceDataView<uint8>());

		// Discard the resource array's contents.
		CreateDesc.InitialData->Discard();

		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Zeroed)
	{
		// Allocate zeroed memory and upload that on buffer creation. The contents of the buffer are "undefined" if initial data is not passed in, so we actually have to do something here.
		void* UploadMemory = FMemory::MallocZeroed(CreateDesc.Size, 16);
		FD3D11Buffer* Buffer = CreateBufferInternal(CreateDesc, TConstArrayView<uint8>(reinterpret_cast<uint8*>(UploadMemory), CreateDesc.Size));
		FMemory::Free(UploadMemory);

		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Initializer)
	{
		FD3D11Buffer* Buffer = BeginCreateBufferInternal(CreateDesc);

		// Allocate ad-hoc memory for writing.
		void* UploadMemory = FMemory::Malloc(CreateDesc.Size, 16);

		return UE::RHICore::FCustomBufferInitializer(RHICmdList, Buffer, UploadMemory, CreateDesc.Size,
			[this, Buffer = TRefCountPtr<FD3D11Buffer>(Buffer), UploadMemory = UE::RHICore::FInitializerScopedMemory(UploadMemory)](FRHICommandListBase& RHICmdList) mutable
			{
				FinalizeCreateBufferInternal(Buffer, TConstArrayView<uint8>(reinterpret_cast<const uint8*>(UploadMemory.Pointer), Buffer->GetDesc().Size));
				return TRefCountPtr<FRHIBuffer>(MoveTemp(Buffer));
			}
		);
	}

	return UE::RHICore::HandleUnknownBufferInitializerInitAction(RHICmdList, CreateDesc);
}

void* FD3D11DynamicRHI::LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	FD3D11Buffer* Buffer = ResourceCast(BufferRHI);
	// If this resource is bound to the device, unbind it
	ConditionalClearShaderResource(Buffer, true);

	// Determine whether the buffer is dynamic or not.
	D3D11_BUFFER_DESC Desc;
	Buffer->Resource->GetDesc(&Desc);
	const bool bIsDynamic = (Desc.Usage == D3D11_USAGE_DYNAMIC);

	FD3D11LockedKey LockedKey(Buffer->Resource);
	FD3D11LockedData LockedData;

	if(bIsDynamic)
	{
		check(LockMode == RLM_WriteOnly || LockMode == RLM_WriteOnly_NoOverwrite);

		// If the buffer is dynamic, map its memory for writing.
		D3D11_MAPPED_SUBRESOURCE MappedSubresource;

		D3D11_MAP MapType = (LockMode == RLM_WriteOnly || !GRHISupportsMapWriteNoOverwrite) ? D3D11_MAP_WRITE_DISCARD : D3D11_MAP_WRITE_NO_OVERWRITE;
		VERIFYD3D11RESULT_EX(Direct3DDeviceIMContext->Map(Buffer->Resource, 0, MapType, 0, &MappedSubresource), Direct3DDevice);

		LockedData.SetData(MappedSubresource.pData);
		LockedData.Pitch = MappedSubresource.RowPitch;
	}
	else
	{
		if(LockMode == RLM_ReadOnly)
		{
			// If the static buffer is being locked for reading, create a staging buffer.
			D3D11_BUFFER_DESC StagingBufferDesc;
			ZeroMemory( &StagingBufferDesc, sizeof( D3D11_BUFFER_DESC ) );
			StagingBufferDesc.ByteWidth = Size;
			StagingBufferDesc.Usage = D3D11_USAGE_STAGING;
			StagingBufferDesc.BindFlags = 0;
			StagingBufferDesc.CPUAccessFlags = D3D11_CPU_ACCESS_READ;
			StagingBufferDesc.MiscFlags = 0;
			TRefCountPtr<ID3D11Buffer> StagingBuffer;
			VERIFYD3D11RESULT_EX(Direct3DDevice->CreateBuffer(&StagingBufferDesc, NULL, StagingBuffer.GetInitReference()), Direct3DDevice);
			LockedData.StagingResource = StagingBuffer;

			// Copy the contents of the buffer to the staging buffer.
			D3D11_BOX SourceBox;
			SourceBox.left = Offset;
			SourceBox.right = Offset + Size;
			SourceBox.top = SourceBox.front = 0;
			SourceBox.bottom = SourceBox.back = 1;
			Direct3DDeviceIMContext->CopySubresourceRegion(StagingBuffer, 0, 0, 0, 0, Buffer->Resource, 0, &SourceBox);

			// Map the staging buffer's memory for reading.
			D3D11_MAPPED_SUBRESOURCE MappedSubresource;
			VERIFYD3D11RESULT_EX(Direct3DDeviceIMContext->Map(StagingBuffer, 0, D3D11_MAP_READ, 0, &MappedSubresource), Direct3DDevice);
			LockedData.SetData(MappedSubresource.pData);
			LockedData.Pitch = MappedSubresource.RowPitch;
			Offset = 0;
		}
		else
		{
			// If the static buffer is being locked for writing, allocate memory for the contents to be written to.
			LockedData.AllocData(Desc.ByteWidth);
			LockedData.Pitch = Desc.ByteWidth;
		}
	}
	
	// Add the lock to the lock map.
	AddLockedData(LockedKey, LockedData);

	// Return the offset pointer
	return (void*)((uint8*)LockedData.GetData() + Offset);
}

void FD3D11DynamicRHI::UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	FD3D11Buffer* Buffer = ResourceCast(BufferRHI);

	// Determine whether the buffer is dynamic or not.
	D3D11_BUFFER_DESC Desc;
	Buffer->Resource->GetDesc(&Desc);
	const bool bIsDynamic = (Desc.Usage == D3D11_USAGE_DYNAMIC);

	// Find the outstanding lock for this buffer and remove it from the tracker.
	FD3D11LockedData LockedData;
	verifyf(RemoveLockedData(FD3D11LockedKey(Buffer->Resource), LockedData), TEXT("Buffer is not locked"));

	if(bIsDynamic)
	{
		// If the buffer is dynamic, its memory was mapped directly; unmap it.
		Direct3DDeviceIMContext->Unmap(Buffer->Resource, 0);
	}
	else
	{
		// If the static buffer lock involved a staging resource, it was locked for reading.
		if(LockedData.StagingResource)
		{
			// Unmap the staging buffer's memory.
			ID3D11Buffer* StagingBuffer = (ID3D11Buffer*)LockedData.StagingResource.GetReference();
			Direct3DDeviceIMContext->Unmap(StagingBuffer,0);
		}
		else 
		{
			// Copy the contents of the temporary memory buffer allocated for writing into the buffer.
			Direct3DDeviceIMContext->UpdateSubresource(Buffer->Resource, 0, NULL, LockedData.GetData(), LockedData.Pitch, 0);

			// Free the temporary memory buffer.
			LockedData.FreeData();
		}
	}
}

void FD3D11DynamicRHI::RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos)
{
	RHICmdList.EnqueueLambda(TEXT("FD3D11DynamicRHI::RHIReplaceResources"),
		[ReplaceInfos = MoveTemp(ReplaceInfos)](FRHICommandListBase&)
		{
			for (FRHIResourceReplaceInfo const& Info : ReplaceInfos)
			{
				switch (Info.GetType())
				{
				default:
					checkNoEntry();
					break;

				case FRHIResourceReplaceInfo::EType::Buffer:
					{
						FD3D11Buffer* Dst = ResourceCast(Info.GetBuffer().Dst);
						FD3D11Buffer* Src = ResourceCast(Info.GetBuffer().Src);

						if (Src)
						{
							// The source buffer should not have any associated views.
							check(!Src->HasLinkedViews());

							Dst->TakeOwnership(*Src);
						}
						else
						{
							Dst->ReleaseOwnership();
						}

						Dst->UpdateLinkedViews();
					}
					break;
				}
			}
		}
	);

	RHICmdList.RHIThreadFence(true);
}

void FD3D11DynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	check(BufferRHI);

	BufferRHI->SetName(Name);

	SetD3D11ResourceName(ResourceCast(BufferRHI), Name);
#endif
}
