// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Commands.cpp: D3D RHI commands implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "Windows/D3D11RHIPrivateUtil.h"
#include "StaticBoundShaderState.h"
#include "GlobalShader.h"
#include "OneColorShader.h"
#include "RHICommandList.h"
#include "RHIStaticStates.h"
#include "ShaderParameterUtils.h"
#include "SceneUtils.h"
#include "EngineGlobals.h"
#include "RHIShaderParametersShared.h"
#include "RHIUniformBufferUtilities.h"

// For Depth Bounds Test interface
#include "Windows/AllowWindowsPlatformTypes.h"
#if WITH_NVAPI
	#include "nvapi.h"
#endif
#if WITH_AMD_AGS
	#include "amd_ags.h"
#endif
#include "Windows/HideWindowsPlatformTypes.h"

#define DECLARE_ISBOUNDSHADER(ShaderType) inline void ValidateBoundShader(FD3D11StateCache& InStateCache, FRHI##ShaderType* ShaderType##RHI) \
{ \
	ID3D11##ShaderType* CachedShader; \
	InStateCache.Get##ShaderType(&CachedShader); \
	FD3D11##ShaderType* ShaderType = FD3D11DynamicRHI::ResourceCast(ShaderType##RHI); \
	ensureMsgf(CachedShader == ShaderType->Resource, TEXT("Parameters are being set for a %s which is not currently bound"), TEXT( #ShaderType )); \
	if (CachedShader) { CachedShader->Release(); } \
}

DECLARE_ISBOUNDSHADER(VertexShader)
DECLARE_ISBOUNDSHADER(PixelShader)
DECLARE_ISBOUNDSHADER(GeometryShader)
DECLARE_ISBOUNDSHADER(ComputeShader)


#if DO_GUARD_SLOW
#define VALIDATE_BOUND_SHADER(s) ValidateBoundShader(StateCache, s)
#else
#define VALIDATE_BOUND_SHADER(s)
#endif

static int32 GUnbindResourcesBetweenDrawsInDX11 = UE_BUILD_DEBUG;
static FAutoConsoleVariableRef CVarUnbindResourcesBetweenDrawsInDX11(
	TEXT("r.UnbindResourcesBetweenDrawsInDX11"),
	GUnbindResourcesBetweenDrawsInDX11,
	TEXT("Unbind resources between material changes in DX11."),
	ECVF_Default
	);


int32 GDX11ReduceRTVRebinds = 1;
static FAutoConsoleVariableRef CVarDX11ReduceRTVRebinds(
	TEXT("r.DX11.ReduceRTVRebinds"),
	GDX11ReduceRTVRebinds,
	TEXT("Reduce # of SetRenderTargetCalls."),
	ECVF_ReadOnly
);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
int32 GLogDX11RTRebinds = 0;
static FAutoConsoleVariableRef CVarLogDx11RTRebinds(
	TEXT("r.DX11.LogRTRebinds"),
	GLogDX11RTRebinds,
	TEXT("Log # of rebinds of RTs per frame"),
	ECVF_Default
);
FThreadSafeCounter GDX11RTRebind;
FThreadSafeCounter GDX11CommitGraphicsResourceTables;
#endif

static TAutoConsoleVariable<int32> CVarAllowUAVFlushExt(
	TEXT("r.D3D11.AutoFlushUAV"),
	1,
	TEXT("If enabled, use NVAPI (Nvidia), AGS (AMD) or Intel Extensions (Intel) to not flush between dispatches/draw calls")
	TEXT(" 1: on (default)\n")
	TEXT(" 0: off"),
	ECVF_RenderThreadSafe);

// Vertex state.
void FD3D11DynamicRHI::RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBufferRHI, uint32 Offset)
{
	FD3D11Buffer* VertexBuffer = ResourceCast(VertexBufferRHI);

	ID3D11Buffer* D3DBuffer = VertexBuffer ? VertexBuffer->Resource.GetReference() : nullptr;
	TrackResourceBoundAsVB(VertexBuffer, StreamIndex);
	StateCache.SetStreamSource(D3DBuffer, StreamIndex, Offset);
}

// Rasterizer state.
void FD3D11DynamicRHI::RHISetRasterizerState(FRHIRasterizerState* NewStateRHI)
{
	FD3D11RasterizerState* NewState = ResourceCast(NewStateRHI);
	StateCache.SetRasterizerState(NewState->Resource);
}

template<EShaderFrequency ShaderFrequency>
void FD3D11DynamicRHI::BindUniformBuffer(uint32 BufferIndex, FRHIUniformBuffer* BufferRHI)
{
	check(BufferRHI && BufferRHI->GetLayout().GetHash());

	FD3D11UniformBuffer* Buffer = ResourceCast(BufferRHI);

	ID3D11Buffer* ConstantBuffer = Buffer ? Buffer->Resource.GetReference() : nullptr;
	StateCache.SetConstantBuffer<ShaderFrequency>(ConstantBuffer, BufferIndex);

	BoundUniformBuffers[ShaderFrequency][BufferIndex] = BufferRHI;
	DirtyUniformBuffers[ShaderFrequency] |= (1 << BufferIndex);
}

template <typename TRHIShader>
void FD3D11DynamicRHI::ApplyStaticUniformBuffers(TRHIShader* Shader)
{
	if (Shader)
	{
		UE::RHICore::ApplyStaticUniformBuffers(Shader, StaticUniformBuffers,
			[this](int32 BufferIndex, FRHIUniformBuffer* Buffer)
			{
				BindUniformBuffer<static_cast<EShaderFrequency>(TRHIShader::StaticFrequency)>(BufferIndex, Buffer);
			});
	}
}

void FD3D11DynamicRHI::RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState)
{
	FRHIGraphicsPipelineStateFallBack* FallbackGraphicsState = static_cast<FRHIGraphicsPipelineStateFallBack*>(GraphicsState);
	IRHICommandContextPSOFallback::RHISetGraphicsPipelineState(GraphicsState, StencilRef, bApplyAdditionalState);
	const FGraphicsPipelineStateInitializer& PsoInit = FallbackGraphicsState->Initializer;

	if (bApplyAdditionalState)
	{
		ApplyStaticUniformBuffers(static_cast<FD3D11VertexShader*>(PsoInit.BoundShaderState.VertexShaderRHI));
		ApplyStaticUniformBuffers(static_cast<FD3D11GeometryShader*>(PsoInit.BoundShaderState.GetGeometryShader()));
		ApplyStaticUniformBuffers(static_cast<FD3D11PixelShader*>(PsoInit.BoundShaderState.PixelShaderRHI));
	}

	// Store the PSO's primitive (after since IRHICommandContext::RHISetGraphicsPipelineState sets the BSS)
	PrimitiveType = PsoInit.PrimitiveType;
}

void FD3D11DynamicRHI::RHISetComputeShader(FRHIComputeShader* ComputeShaderRHI)
{
	FD3D11ComputeShader* ComputeShader = ResourceCast(ComputeShaderRHI);
	SetCurrentComputeShader(ComputeShaderRHI);

	if (GUnbindResourcesBetweenDrawsInDX11)
	{
		ClearAllShaderResourcesForFrequency<SF_Compute>();
	}

	ApplyStaticUniformBuffers(ComputeShader);
}

void FD3D11DynamicRHI::RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) 
{ 
	RHI_DISPATCH_CALL_INC();

	FRHIComputeShader* ComputeShaderRHI = GetCurrentComputeShader();
	FD3D11ComputeShader* ComputeShader = ResourceCast(ComputeShaderRHI);

	StateCache.SetComputeShader(ComputeShader->Resource);

	if (ComputeShader->bShaderNeedsGlobalConstantBuffer)
	{
		CommitComputeShaderConstants();
	}
	CommitComputeResourceTables(ComputeShader);
	
	Direct3DDeviceIMContext->Dispatch(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);	
	StateCache.SetComputeShader(nullptr);
	EnableUAVOverlap();
}

void FD3D11DynamicRHI::RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{ 
	RHI_DISPATCH_CALL_INC();

	FRHIComputeShader* ComputeShaderRHI = GetCurrentComputeShader();
	FD3D11ComputeShader* ComputeShader = ResourceCast(ComputeShaderRHI);
	FD3D11Buffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);

	StateCache.SetComputeShader(ComputeShader->Resource);
	
	if (ComputeShader->bShaderNeedsGlobalConstantBuffer)
	{
		CommitComputeShaderConstants();
	}
	CommitComputeResourceTables(ComputeShader);

	Direct3DDeviceIMContext->DispatchIndirect(ArgumentBuffer->Resource,ArgumentOffset);
	StateCache.SetComputeShader(nullptr);
	EnableUAVOverlap();
}

void FD3D11DynamicRHI::RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ)
{
	// These are the maximum viewport extents for D3D11. Exceeding them leads to badness.
	check(MinX <= (float)D3D11_VIEWPORT_BOUNDS_MAX);
	check(MinY <= (float)D3D11_VIEWPORT_BOUNDS_MAX);
	check(MaxX <= (float)D3D11_VIEWPORT_BOUNDS_MAX);
	check(MaxY <= (float)D3D11_VIEWPORT_BOUNDS_MAX);

	D3D11_VIEWPORT Viewport = { MinX, MinY, MaxX - MinX, MaxY - MinY, MinZ, MaxZ };
	//avoid setting a 0 extent viewport, which the debug runtime doesn't like
	if (Viewport.Width > 0 && Viewport.Height > 0)
	{
		StateCache.SetViewport(Viewport);
		RHISetScissorRect(true, MinX, MinY, MaxX, MaxY);
	}
}

static void ValidateScissorRect(const D3D11_VIEWPORT& Viewport, const D3D11_RECT& ScissorRect)
{
	ensure(ScissorRect.left >= (LONG)Viewport.TopLeftX);
	ensure(ScissorRect.top >= (LONG)Viewport.TopLeftY);
	ensure(ScissorRect.right <= (LONG)Viewport.TopLeftX + (LONG)Viewport.Width);
	ensure(ScissorRect.bottom <= (LONG)Viewport.TopLeftY + (LONG)Viewport.Height);
	ensure(ScissorRect.left <= ScissorRect.right && ScissorRect.top <= ScissorRect.bottom);
}

void FD3D11DynamicRHI::RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ)
{
	// Set up both viewports
	D3D11_VIEWPORT StereoViewports[2] = {};

	StereoViewports[0].TopLeftX = FMath::FloorToInt(LeftMinX);
	StereoViewports[0].TopLeftY = FMath::FloorToInt(LeftMinY);
	StereoViewports[0].Width = FMath::CeilToInt(LeftMaxX - LeftMinX);
	StereoViewports[0].Height = FMath::CeilToInt(LeftMaxY - LeftMinY);
	StereoViewports[0].MinDepth = MinZ;
	StereoViewports[0].MaxDepth = MaxZ;

	StereoViewports[1].TopLeftX = FMath::FloorToInt(RightMinX);
	StereoViewports[1].TopLeftY = FMath::FloorToInt(RightMinY);
	StereoViewports[1].Width = FMath::CeilToInt(RightMaxX - RightMinX);
	StereoViewports[1].Height = FMath::CeilToInt(RightMaxY - RightMinY);
	StereoViewports[1].MinDepth = MinZ;
	StereoViewports[1].MaxDepth = MaxZ;

	D3D11_RECT ScissorRects[2] =
	{
		{ StereoViewports[0].TopLeftX, StereoViewports[0].TopLeftY, StereoViewports[0].TopLeftX + StereoViewports[0].Width, StereoViewports[0].TopLeftY + StereoViewports[0].Height },
		{ StereoViewports[1].TopLeftX, StereoViewports[1].TopLeftY, StereoViewports[1].TopLeftX + StereoViewports[1].Width, StereoViewports[1].TopLeftY + StereoViewports[1].Height }
	};

	ValidateScissorRect(StereoViewports[0], ScissorRects[0]);
	ValidateScissorRect(StereoViewports[1], ScissorRects[1]);

	StateCache.SetViewports(2, StereoViewports);
	// Set the scissor rect appropriately.
	Direct3DDeviceIMContext->RSSetScissorRects(2, ScissorRects);
}

void FD3D11DynamicRHI::RHISetScissorRect(bool bEnable,uint32 MinX,uint32 MinY,uint32 MaxX,uint32 MaxY)
{
	D3D11_VIEWPORT Viewport;
	StateCache.GetViewport(&Viewport);

	D3D11_RECT ScissorRect;
	if (bEnable)
	{
		ScissorRect.left   = MinX;
		ScissorRect.top    = MinY;
		ScissorRect.right  = MaxX;
		ScissorRect.bottom = MaxY;
	}
	else
	{
		ScissorRect.left   = (LONG) Viewport.TopLeftX;
		ScissorRect.top    = (LONG) Viewport.TopLeftY;
		ScissorRect.right  = (LONG) Viewport.TopLeftX + (LONG) Viewport.Width;
		ScissorRect.bottom = (LONG) Viewport.TopLeftY + (LONG) Viewport.Height;
	}

	ValidateScissorRect(Viewport, ScissorRect);
	Direct3DDeviceIMContext->RSSetScissorRects(1, &ScissorRect);
}

/**
* Set bound shader state. This will set the vertex decl/shader, and pixel shader
* @param BoundShaderState - state resource
*/
void FD3D11DynamicRHI::RHISetBoundShaderState(FRHIBoundShaderState* BoundShaderStateRHI)
{
	FD3D11BoundShaderState* BoundShaderState = ResourceCast(BoundShaderStateRHI);

	StateCache.SetStreamStrides(BoundShaderState->StreamStrides);
	StateCache.SetInputLayout(BoundShaderState->InputLayout);
	StateCache.SetVertexShader(BoundShaderState->VertexShader);
	StateCache.SetPixelShader(BoundShaderState->PixelShader);

	StateCache.SetGeometryShader(BoundShaderState->GeometryShader);

	// @TODO : really should only discard the constants if the shader state has actually changed.
	bDiscardSharedConstants = true;

	// Prevent transient bound shader states from being recreated for each use by keeping a history of the most recently used bound shader states.
	// The history keeps them alive, and the bound shader state cache allows them to am be reused if needed.
	BoundShaderStateHistory.Add(BoundShaderState);

	// Shader changed so all resource tables are dirty
	DirtyUniformBuffers[SF_Vertex] = 0xffff;
	DirtyUniformBuffers[SF_Pixel] = 0xffff;
	DirtyUniformBuffers[SF_Geometry] = 0xffff;

	// Shader changed.  All UB's must be reset by high level code to match other platforms anway.
	// Clear to catch those bugs, and bugs with stale UB's causing layout mismatches.
	// Release references to bound uniform buffers.
	for (int32 Frequency = 0; Frequency < SF_NumStandardFrequencies; ++Frequency)
	{
		for (int32 BindIndex = 0; BindIndex < MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE; ++BindIndex)
		{
			BoundUniformBuffers[Frequency][BindIndex] = nullptr;
		}
	}

	if (GUnbindResourcesBetweenDrawsInDX11 || GRHIGlobals.IsDebugLayerEnabled)
	{
		ClearAllShaderResources();
	}
}

void FD3D11DynamicRHI::RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers)
{
	FMemory::Memzero(StaticUniformBuffers.GetData(), StaticUniformBuffers.Num() * sizeof(FRHIUniformBuffer*));

	for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
	{
		StaticUniformBuffers[InUniformBuffers.GetSlot(Index)] = InUniformBuffers.GetUniformBuffer(Index);
	}
}

void FD3D11DynamicRHI::RHISetStaticUniformBuffer(FUniformBufferStaticSlot InSlot, FRHIUniformBuffer* InBuffer)
{
	StaticUniformBuffers[InSlot] = InBuffer;
}

template<EShaderFrequency ShaderFrequency>
struct FD3D11ResourceBinder
{
	FD3D11DynamicRHI& RHI;

	FD3D11ResourceBinder(FD3D11DynamicRHI& InRHI)
		: RHI(InRHI)
	{
	}

	void SetUAV(FRHIUnorderedAccessView* InUnorderedAccessView, uint8 Index)
	{
		if (ShaderFrequency == SF_Compute)
		{
			RHI.InternalSetUAVCS(Index, FD3D11DynamicRHI::ResourceCast(InUnorderedAccessView));
		}
		else if (ShaderFrequency == SF_Pixel || ShaderFrequency == SF_Vertex)
		{
			RHI.InternalSetUAVVSPS(Index, FD3D11DynamicRHI::ResourceCast(InUnorderedAccessView));
		}
		else
		{
			checkf(false, TEXT("UAVs are only supported in compute, pixel and vertex shaders."));
		}
	}

	void SetSRV(FRHIShaderResourceView* InShaderResourceView, uint8 Index)
	{
		FD3D11ShaderResourceView* D3D11ShaderResourceView = FD3D11DynamicRHI::ResourceCast(InShaderResourceView);
		FD3D11ViewableResource* D3D11ViewableResource = D3D11ShaderResourceView ? D3D11ShaderResourceView->GetBaseResource() : nullptr;
		ID3D11ShaderResourceView* D3D11SRV = D3D11ShaderResourceView ? D3D11ShaderResourceView->View.GetReference() : nullptr;

		RHI.SetShaderResourceView<ShaderFrequency>(
			D3D11ViewableResource,
			D3D11SRV,
			Index
		);
	}

	void SetTexture(FRHITexture* InTexture, uint8 Index)
	{
		FD3D11Texture* D3D11Texture = FD3D11DynamicRHI::ResourceCast(InTexture);
		ID3D11ShaderResourceView* ShaderResourceView = D3D11Texture ? D3D11Texture->GetShaderResourceView() : nullptr;

		RHI.SetShaderResourceView<ShaderFrequency>(
			D3D11Texture,
			ShaderResourceView,
			Index
		);
	}

	void SetSampler(FRHISamplerState* Sampler, uint8 Index)
	{
		RHI.GetStateCache().SetSamplerState<ShaderFrequency>(FD3D11DynamicRHI::ResourceCast(Sampler)->Resource, Index);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Index)
	{
		checkNoEntry();
	}
#endif
};

template<EShaderFrequency ShaderFrequency>
void FD3D11DynamicRHI::SetShaderParametersCommon(FD3D11ConstantBuffer* StageConstantBuffer, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters)
{
	if (InParameters.Num())
	{
		for (const FRHIShaderParameter& Parameter : InParameters)
		{
			check(Parameter.BufferIndex == 0);
			StageConstantBuffer->UpdateConstant(&InParametersData[Parameter.ByteOffset], Parameter.BaseIndex, Parameter.ByteSize);
		}
	}

	FD3D11ResourceBinder<ShaderFrequency> Binder(*this);

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		if (Parameter.Type == FRHIShaderParameterResource::EType::UnorderedAccessView)
		{
			Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Parameter.Resource), Parameter.Index);
		}
	}

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		switch (Parameter.Type)
		{
		case FRHIShaderParameterResource::EType::Texture:
			Binder.SetTexture(static_cast<FRHITexture*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::ResourceView:
			Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UnorderedAccessView:
			break;
		case FRHIShaderParameterResource::EType::Sampler:
			Binder.SetSampler(static_cast<FRHISamplerState*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UniformBuffer:
			BindUniformBuffer<ShaderFrequency>(Parameter.Index, static_cast<FRHIUniformBuffer*>(Parameter.Resource));
			break;
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}
}


void FD3D11DynamicRHI::RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
	SetShaderParametersCommon<SF_Compute>(CSConstantBuffer, InParametersData, InParameters, InResourceParameters);
}

void FD3D11DynamicRHI::RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
	switch (Shader->GetFrequency())
	{
	case SF_Vertex:
		VALIDATE_BOUND_SHADER(static_cast<FRHIVertexShader*>(Shader));
		SetShaderParametersCommon<SF_Vertex>(VSConstantBuffer, InParametersData, InParameters, InResourceParameters);
		break;
	case SF_Geometry:
		VALIDATE_BOUND_SHADER(static_cast<FRHIGeometryShader*>(Shader));
		SetShaderParametersCommon<SF_Geometry>(GSConstantBuffer, InParametersData, InParameters, InResourceParameters);
		break;
	case SF_Pixel:
		VALIDATE_BOUND_SHADER(static_cast<FRHIPixelShader*>(Shader));
		SetShaderParametersCommon<SF_Pixel>(PSConstantBuffer, InParametersData, InParameters, InResourceParameters);
		break;
	default:
		checkf(0, TEXT("Undefined FRHIGraphicsShader Type %d!"), (int32)Shader->GetFrequency());
	}
}

template<EShaderFrequency ShaderFrequency>
void FD3D11DynamicRHI::SetShaderUnbindsCommon(TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	FD3D11ResourceBinder<ShaderFrequency> Binder(*this);

	for (const FRHIShaderParameterUnbind& Unbind : InUnbinds)
	{
		switch (Unbind.Type)
		{
		case FRHIShaderParameterUnbind::EType::ResourceView:
			Binder.SetSRV(nullptr, Unbind.Index);
			break;
		case FRHIShaderParameterUnbind::EType::UnorderedAccessView:
			Binder.SetUAV(nullptr, Unbind.Index);
			break;
		default:
			checkf(false, TEXT("Unhandled unbind resource type?"));
			break;
		}
	}
}

void FD3D11DynamicRHI::RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	SetShaderUnbindsCommon<SF_Compute>(InUnbinds);
}

void FD3D11DynamicRHI::RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	switch (Shader->GetFrequency())
	{
	case SF_Vertex:
		VALIDATE_BOUND_SHADER(static_cast<FRHIVertexShader*>(Shader));
		SetShaderUnbindsCommon<SF_Vertex>(InUnbinds);
		break;
	case SF_Geometry:
		VALIDATE_BOUND_SHADER(static_cast<FRHIGeometryShader*>(Shader));
		SetShaderUnbindsCommon<SF_Geometry>(InUnbinds);
		break;
	case SF_Pixel:
		VALIDATE_BOUND_SHADER(static_cast<FRHIPixelShader*>(Shader));
		SetShaderUnbindsCommon<SF_Pixel>(InUnbinds);
		break;
	default:
		checkf(0, TEXT("Undefined FRHIGraphicsShader Type %d!"), (int32)Shader->GetFrequency());
	}
}

void FD3D11DynamicRHI::ValidateExclusiveDepthStencilAccess(FExclusiveDepthStencil RequestedAccess) const
{
	const bool bSrcDepthWrite = RequestedAccess.IsDepthWrite();
	const bool bSrcStencilWrite = RequestedAccess.IsStencilWrite();

	if (bSrcDepthWrite || bSrcStencilWrite)
	{
		// New Rule: You have to call SetRenderTarget[s]() before
		ensure(CurrentDepthTexture);

		const bool bDstDepthWrite = CurrentDSVAccessType.IsDepthWrite();
		const bool bDstStencilWrite = CurrentDSVAccessType.IsStencilWrite();

		// requested access is not possible, fix SetRenderTarget EExclusiveDepthStencil or request a different one
		ensureMsgf(
			!bSrcDepthWrite || bDstDepthWrite, 
			TEXT("Expected: SrcDepthWrite := false or DstDepthWrite := true. Actual: SrcDepthWrite := %s or DstDepthWrite := %s"),
			(bSrcDepthWrite) ? TEXT("true") : TEXT("false"),
			(bDstDepthWrite) ? TEXT("true") : TEXT("false")
			);

		ensureMsgf(
			!bSrcStencilWrite || bDstStencilWrite,
			TEXT("Expected: SrcStencilWrite := false or DstStencilWrite := true. Actual: SrcStencilWrite := %s or DstStencilWrite := %s"),
			(bSrcStencilWrite) ? TEXT("true") : TEXT("false"),
			(bDstStencilWrite) ? TEXT("true") : TEXT("false")
			);
	}
}

void FD3D11DynamicRHI::RHISetDepthStencilState(FRHIDepthStencilState* NewStateRHI,uint32 StencilRef)
{
	FD3D11DepthStencilState* NewState = ResourceCast(NewStateRHI);

	ValidateExclusiveDepthStencilAccess(NewState->AccessType);

	StateCache.SetDepthStencilState(NewState->Resource, StencilRef);
}

void FD3D11DynamicRHI::RHISetStencilRef(uint32 StencilRef)
{
	StateCache.SetStencilRef(StencilRef);
}

void FD3D11DynamicRHI::RHISetBlendState(FRHIBlendState* NewStateRHI,const FLinearColor& BlendFactor)
{
	FD3D11BlendState* NewState = ResourceCast(NewStateRHI);
	StateCache.SetBlendState(NewState->Resource, (const float*)&BlendFactor, 0xffffffff);
}

void FD3D11DynamicRHI::RHISetBlendFactor(const FLinearColor& BlendFactor)
{
	StateCache.SetBlendFactor((const float*)&BlendFactor, 0xffffffff);
}
void FD3D11DynamicRHI::CommitRenderTargetsAndUAVs()
{
	CommitRenderTargets(false);
	FMemory::Memset(UAVBound, 0); //force to be rebound if any is set
	UAVSChanged = 1;
	CommitUAVs();

}
void FD3D11DynamicRHI::CommitRenderTargets(bool bClearUAVs)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D11RenderTargetCommits);
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	GDX11RTRebind.Increment();
#endif
	ID3D11RenderTargetView* RTArray[D3D11_SIMULTANEOUS_RENDER_TARGET_COUNT];
	for (uint32 RenderTargetIndex = 0; RenderTargetIndex < NumSimultaneousRenderTargets; ++RenderTargetIndex)
	{
		RTArray[RenderTargetIndex] = CurrentRenderTargets[RenderTargetIndex];
	}

	
	Direct3DDeviceIMContext->OMSetRenderTargets(
		NumSimultaneousRenderTargets,
		RTArray,
		CurrentDepthStencilTarget
	);

	if(bClearUAVs)
	{
		for(uint32 i = 0; i < D3D11_PS_CS_UAV_REGISTER_COUNT; ++i)
		{
			CurrentUAVs[i] = nullptr;
			UAVBound[i] = nullptr;
		}
		UAVBindFirst = 0;
		UAVBindCount = 0;
		UAVSChanged = 0;
	}
}

void FD3D11DynamicRHI::InternalSetUAVCS(uint32 BindIndex, FD3D11UnorderedAccessView* UnorderedAccessViewRHI)
{
	if (UnorderedAccessViewRHI)
	{
		ConditionalClearShaderResource(UnorderedAccessViewRHI->GetBaseResource(), true);
	}

	ID3D11UnorderedAccessView* D3D11UAV = UnorderedAccessViewRHI ? UnorderedAccessViewRHI->View.GetReference() : nullptr;

	uint32 InitialCount = -1;
	Direct3DDeviceIMContext->CSSetUnorderedAccessViews(BindIndex, 1, &D3D11UAV, &InitialCount);
}

void FD3D11DynamicRHI::InternalSetUAVVSPS(uint32 BindIndex, FD3D11UnorderedAccessView* UnorderedAccessViewRHI)
{
	check(BindIndex < D3D11_PS_CS_UAV_REGISTER_COUNT);
	if (CurrentUAVs[BindIndex] != UnorderedAccessViewRHI)
	{
		CurrentUAVs[BindIndex] = UnorderedAccessViewRHI;
		UAVSChanged = 1;
	}
	if (UnorderedAccessViewRHI)
	{
		ConditionalClearShaderResource(UnorderedAccessViewRHI->GetBaseResource(), true);
		for (uint32 i = 0; i < D3D11_PS_CS_UAV_REGISTER_COUNT; i++)
		{
			if (i != BindIndex && CurrentUAVs[i] == UnorderedAccessViewRHI)
			{
				CurrentUAVs[i] = nullptr;
			}
		}
	}
}

void FD3D11DynamicRHI::CommitUAVs()
{
	if (!UAVSChanged)
	{
		return;
	}
	int32 First = -1;
	int32 Count = 0;
	for (int32 i = 0; i < D3D11_PS_CS_UAV_REGISTER_COUNT; ++i)
	{
		if (CurrentUAVs[i] != nullptr)
		{
			First = i;
			break;
		}
	}

	if (First != -1)
	{
		FD3D11UnorderedAccessView* RHIUAVs[D3D11_PS_CS_UAV_REGISTER_COUNT];
		ID3D11UnorderedAccessView* UAVs[D3D11_PS_CS_UAV_REGISTER_COUNT];
		FMemory::Memset(UAVs, 0);

		for (int32 i = First; i < D3D11_PS_CS_UAV_REGISTER_COUNT; ++i)
		{
			if (CurrentUAVs[i] == nullptr)
				break;
			RHIUAVs[i] = CurrentUAVs[i].GetReference();
			UAVs[i] = RHIUAVs[i]->View;
			Count++;
		}

		if (First != UAVBindFirst || Count != UAVBindCount || 0 != FMemory::Memcmp(&UAVs[First], &UAVBound[First], sizeof(UAVs[0]) * Count))
		{
			SCOPE_CYCLE_COUNTER(STAT_D3D11RenderTargetCommitsUAV);
			for (int32 i = First; i < First + Count; ++i)
			{
				if (UAVs[i] != UAVBound[i])
				{
					FD3D11UnorderedAccessView* RHIUAV = RHIUAVs[i];
					ID3D11UnorderedAccessView* UAV = UAVs[i];

					// Unbind any shader views of the UAV's resource.
					ConditionalClearShaderResource(RHIUAV->GetBaseResource(), true);
					UAVBound[i] = UAV;
				}
			}
			static const uint32 UAVInitialCountArray[D3D11_PS_CS_UAV_REGISTER_COUNT] = { ~0u, ~0u, ~0u, ~0u, ~0u, ~0u, ~0u, ~0u };
			Direct3DDeviceIMContext->OMSetRenderTargetsAndUnorderedAccessViews(D3D11_KEEP_RENDER_TARGETS_AND_DEPTH_STENCIL, 0, 0, First, Count, &UAVs[First], &UAVInitialCountArray[0]);
		}

	}
	else
	{
		if (First != UAVBindFirst)
		{
			Direct3DDeviceIMContext->OMSetRenderTargetsAndUnorderedAccessViews(D3D11_KEEP_RENDER_TARGETS_AND_DEPTH_STENCIL, 0, 0, 0, 0, nullptr, nullptr);
		}
	}

	UAVBindFirst = First;
	UAVBindCount = Count;
	UAVSChanged = 0;
}

struct FRTVDesc
{
	uint32 Width;
	uint32 Height;
	DXGI_SAMPLE_DESC SampleDesc;
};

// Return an FRTVDesc structure whose
// Width and height dimensions are adjusted for the RTV's miplevel.
FRTVDesc GetRenderTargetViewDesc(ID3D11RenderTargetView* RenderTargetView)
{
	D3D11_RENDER_TARGET_VIEW_DESC TargetDesc;
	RenderTargetView->GetDesc(&TargetDesc);

	TRefCountPtr<ID3D11Resource> BaseResource;
	RenderTargetView->GetResource((ID3D11Resource**)BaseResource.GetInitReference());
	uint32 MipIndex = 0;
	FRTVDesc ret;
	memset(&ret, 0, sizeof(ret));

	switch (TargetDesc.ViewDimension)
	{
		case D3D11_RTV_DIMENSION_TEXTURE2D:
		case D3D11_RTV_DIMENSION_TEXTURE2DMS:
		case D3D11_RTV_DIMENSION_TEXTURE2DARRAY:
		case D3D11_RTV_DIMENSION_TEXTURE2DMSARRAY:
		{
			D3D11_TEXTURE2D_DESC Desc;
			((ID3D11Texture2D*)(BaseResource.GetReference()))->GetDesc(&Desc);
			ret.Width = Desc.Width;
			ret.Height = Desc.Height;
			ret.SampleDesc = Desc.SampleDesc;
			if (TargetDesc.ViewDimension == D3D11_RTV_DIMENSION_TEXTURE2D || TargetDesc.ViewDimension == D3D11_RTV_DIMENSION_TEXTURE2DARRAY)
			{
				// All the non-multisampled texture types have their mip-slice in the same position.
				MipIndex = TargetDesc.Texture2D.MipSlice;
			}
			break;
		}
		case D3D11_RTV_DIMENSION_TEXTURE3D:
		{
			D3D11_TEXTURE3D_DESC Desc;
			((ID3D11Texture3D*)(BaseResource.GetReference()))->GetDesc(&Desc);
			ret.Width = Desc.Width;
			ret.Height = Desc.Height;
			ret.SampleDesc.Count = 1;
			ret.SampleDesc.Quality = 0;
			MipIndex = TargetDesc.Texture3D.MipSlice;
			break;
		}
		default:
		{
			// not expecting 1D targets.
			checkNoEntry();
		}
	}
	ret.Width >>= MipIndex;
	ret.Height >>= MipIndex;
	return ret;
}

void FD3D11DynamicRHI::SetRenderTargets(
	uint32 NewNumSimultaneousRenderTargets,
	const FRHIRenderTargetView* NewRenderTargetsRHI,
	const FRHIDepthRenderTargetView* NewDepthStencilTargetRHI)
{
	FD3D11Texture* NewDepthStencilTarget = ResourceCast(NewDepthStencilTargetRHI ? NewDepthStencilTargetRHI->Texture : nullptr);

	check(NewNumSimultaneousRenderTargets <= MaxSimultaneousRenderTargets);

	bool bTargetChanged = false;

	// Set the appropriate depth stencil view depending on whether depth writes are enabled or not
	ID3D11DepthStencilView* DepthStencilView = NULL;
	if(NewDepthStencilTarget)
	{
		check(NewDepthStencilTargetRHI);
		CurrentDSVAccessType = NewDepthStencilTargetRHI->GetDepthStencilAccess();
		DepthStencilView = NewDepthStencilTarget->GetDepthStencilView(CurrentDSVAccessType);

		// Unbind any shader views of the depth stencil target that are bound.
		ConditionalClearShaderResource(NewDepthStencilTarget, false);
	}

	// Check if the depth stencil target is different from the old state.
	if(CurrentDepthStencilTarget != DepthStencilView)
	{
		CurrentDepthTexture = NewDepthStencilTarget;
		CurrentDepthStencilTarget = DepthStencilView;
		bTargetChanged = true;
	}

	// Gather the render target views for the new render targets.
	ID3D11RenderTargetView* NewRenderTargetViews[MaxSimultaneousRenderTargets];
	for(uint32 RenderTargetIndex = 0;RenderTargetIndex < MaxSimultaneousRenderTargets;++RenderTargetIndex)
	{
		ID3D11RenderTargetView* RenderTargetView = NULL;
		if(RenderTargetIndex < NewNumSimultaneousRenderTargets && NewRenderTargetsRHI[RenderTargetIndex].Texture != nullptr) //-V522
		{
			int32 RTMipIndex = NewRenderTargetsRHI[RenderTargetIndex].MipIndex;
			int32 RTSliceIndex = NewRenderTargetsRHI[RenderTargetIndex].ArraySliceIndex;
			
			FD3D11Texture* NewRenderTarget = ResourceCast(NewRenderTargetsRHI[RenderTargetIndex].Texture);
			RenderTargetView = NewRenderTarget ? NewRenderTarget->GetRenderTargetView(RTMipIndex, RTSliceIndex) : nullptr;

			ensureMsgf(RenderTargetView, TEXT("Texture being set as render target has no RTV"));
			
			// Unbind any shader views of the render target that are bound.
			ConditionalClearShaderResource(NewRenderTarget, false);

#if UE_BUILD_DEBUG	
			// A check to allow you to pinpoint what is using mismatching targets
			// We filter our d3ddebug spew that checks for this as the d3d runtime's check is wrong.
			// For filter code, see D3D11Device.cpp look for "OMSETRENDERTARGETS_INVALIDVIEW"
			if(RenderTargetView && DepthStencilView)
			{
				FRTVDesc RTTDesc = GetRenderTargetViewDesc(RenderTargetView);

				TRefCountPtr<ID3D11Texture2D> DepthTargetTexture;
				DepthStencilView->GetResource((ID3D11Resource**)DepthTargetTexture.GetInitReference());

				D3D11_TEXTURE2D_DESC DTTDesc;
				DepthTargetTexture->GetDesc(&DTTDesc);

				// enforce color target is <= depth and MSAA settings match
				if(RTTDesc.Width > DTTDesc.Width || RTTDesc.Height > DTTDesc.Height || 
					RTTDesc.SampleDesc.Count != DTTDesc.SampleDesc.Count || 
					RTTDesc.SampleDesc.Quality != DTTDesc.SampleDesc.Quality)
				{
					UE_LOGF(LogD3D11RHI, Fatal,"RTV(%i,%i c=%i,q=%i) and DSV(%i,%i c=%i,q=%i) have mismatching dimensions and/or MSAA levels!",
						RTTDesc.Width,RTTDesc.Height,RTTDesc.SampleDesc.Count,RTTDesc.SampleDesc.Quality,
						DTTDesc.Width,DTTDesc.Height,DTTDesc.SampleDesc.Count,DTTDesc.SampleDesc.Quality);
				}
			}
#endif
		}

		NewRenderTargetViews[RenderTargetIndex] = RenderTargetView;

		// Check if the render target is different from the old state.
		if(CurrentRenderTargets[RenderTargetIndex] != RenderTargetView)
		{
			CurrentRenderTargets[RenderTargetIndex] = RenderTargetView;
			bTargetChanged = true;
		}
	}
	if(NumSimultaneousRenderTargets != NewNumSimultaneousRenderTargets)
	{
		NumSimultaneousRenderTargets = NewNumSimultaneousRenderTargets;
		uint32 Bit = 1;
		uint32 Mask = 0;
		for (uint32 Index = 0; Index < NumSimultaneousRenderTargets; ++Index)
		{
			Mask |= Bit;
			Bit <<= 1;
		}
		CurrentRTVOverlapMask = Mask;
		bTargetChanged = true;
	}

	// Only make the D3D call to change render targets if something actually changed.
	if(bTargetChanged)
	{
		CommitRenderTargets(true);
		CurrentUAVMask = 0;
	}

	// Set the viewport to the full size of render target 0.
	if (NewRenderTargetViews[0])
	{
		// check target 0 is valid
		check(0 < NewNumSimultaneousRenderTargets && NewRenderTargetsRHI[0].Texture != nullptr);
		FRTVDesc RTTDesc = GetRenderTargetViewDesc(NewRenderTargetViews[0]);
		RHISetViewport(0.0f, 0.0f, 0.0f, (float)RTTDesc.Width, (float)RTTDesc.Height, 1.0f);
	}
	else if( DepthStencilView )
	{
		TRefCountPtr<ID3D11Texture2D> DepthTargetTexture;
		DepthStencilView->GetResource((ID3D11Resource**)DepthTargetTexture.GetInitReference());

		D3D11_TEXTURE2D_DESC DTTDesc;
		DepthTargetTexture->GetDesc(&DTTDesc);
		RHISetViewport(0.0f, 0.0f, 0.0f, (float)DTTDesc.Width, (float)DTTDesc.Height, 1.0f);
	}
}

void FD3D11DynamicRHI::SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo)
{
	this->SetRenderTargets(RenderTargetsInfo.NumColorRenderTargets,
		RenderTargetsInfo.ColorRenderTarget,
		&RenderTargetsInfo.DepthStencilRenderTarget);
	
	if (RenderTargetsInfo.bClearColor || RenderTargetsInfo.bClearStencil || RenderTargetsInfo.bClearDepth)
	{
		FLinearColor ClearColors[MaxSimultaneousRenderTargets];
		bool bClearColorArray[MaxSimultaneousRenderTargets];
		float DepthClear = 0.0;
		uint32 StencilClear = 0;

		if (RenderTargetsInfo.bClearColor)
		{
			for (int32 i = 0; i < RenderTargetsInfo.NumColorRenderTargets; ++i)
			{
				bClearColorArray[i] = RenderTargetsInfo.ColorRenderTarget[i].LoadAction == ERenderTargetLoadAction::EClear;

				if (bClearColorArray[i] && RenderTargetsInfo.ColorRenderTarget[i].Texture != nullptr)
				{
					const FClearValueBinding& ClearValue = RenderTargetsInfo.ColorRenderTarget[i].Texture->GetClearBinding();
					checkf(ClearValue.ColorBinding == EClearBinding::EColorBound, TEXT("Texture: %s does not have a color bound for fast clears"), *RenderTargetsInfo.ColorRenderTarget[i].Texture->GetName().GetPlainNameString());
					ClearColors[i] = ClearValue.GetClearColor();
				}
			}
		}
		if (RenderTargetsInfo.bClearDepth || RenderTargetsInfo.bClearStencil)
		{
			const FClearValueBinding& ClearValue = RenderTargetsInfo.DepthStencilRenderTarget.Texture->GetClearBinding();
			checkf(ClearValue.ColorBinding == EClearBinding::EDepthStencilBound, TEXT("Texture: %s does not have a DS value bound for fast clears"), *RenderTargetsInfo.DepthStencilRenderTarget.Texture->GetName().GetPlainNameString());
			ClearValue.GetDepthStencil(DepthClear, StencilClear);
	}

		this->RHIClearMRTImpl(RenderTargetsInfo.bClearColor ? bClearColorArray : nullptr, RenderTargetsInfo.NumColorRenderTargets, ClearColors, RenderTargetsInfo.bClearDepth, DepthClear, RenderTargetsInfo.bClearStencil, StencilClear);
	}
}

// Primitive drawing.

static D3D11_PRIMITIVE_TOPOLOGY GetD3D11PrimitiveType(EPrimitiveType PrimitiveType)
{
	switch(PrimitiveType)
	{
	case PT_TriangleList: return D3D11_PRIMITIVE_TOPOLOGY_TRIANGLELIST;
	case PT_TriangleStrip: return D3D11_PRIMITIVE_TOPOLOGY_TRIANGLESTRIP;
	case PT_LineList: return D3D11_PRIMITIVE_TOPOLOGY_LINELIST;
	case PT_PointList: return D3D11_PRIMITIVE_TOPOLOGY_POINTLIST;

	default: UE_LOGF(LogD3D11RHI, Fatal,"Unknown primitive type: %u",PrimitiveType);
	};

	return D3D11_PRIMITIVE_TOPOLOGY_TRIANGLELIST;
}

namespace FD3DRHIUtil
{
	template <EShaderFrequency ShaderFrequencyT>
	inline void CommitConstants(FD3D11ConstantBuffer* InConstantBuffer, FD3D11StateCache& StateCache, bool bDiscardSharedConstants)
	{
		FWinD3D11ConstantBuffer* ConstantBuffer = static_cast<FWinD3D11ConstantBuffer*>(InConstantBuffer);
		// Array may contain NULL entries to pad out to proper 
		if (ConstantBuffer && ConstantBuffer->CommitConstantsToDevice(bDiscardSharedConstants))
		{
			ID3D11Buffer* DeviceBuffer = ConstantBuffer->GetConstantBuffer();
			StateCache.SetConstantBuffer<ShaderFrequencyT>(DeviceBuffer, GLOBAL_CONSTANT_BUFFER_INDEX);
		}
	}
};

void FD3D11DynamicRHI::CommitNonComputeShaderConstants()
{
	FD3D11BoundShaderState* CurrentBoundShaderState = (FD3D11BoundShaderState*)BoundShaderStateHistory.GetLast();
	check(CurrentBoundShaderState);

	// Only set the constant buffer if this shader needs the global constant buffer bound
	// Otherwise we will overwrite a different constant buffer
	if (CurrentBoundShaderState->bShaderNeedsGlobalConstantBuffer[SF_Vertex])
	{
		// Commit and bind vertex shader constants
		FD3DRHIUtil::CommitConstants<SF_Vertex>(VSConstantBuffer, StateCache, bDiscardSharedConstants);
	}

	if (CurrentBoundShaderState->bShaderNeedsGlobalConstantBuffer[SF_Geometry])
	{
		// Commit and bind geometry shader constants
		FD3DRHIUtil::CommitConstants<SF_Geometry>(GSConstantBuffer, StateCache, bDiscardSharedConstants);
	}

	if (CurrentBoundShaderState->bShaderNeedsGlobalConstantBuffer[SF_Pixel])
	{
		// Commit and bind pixel shader constants
		FD3DRHIUtil::CommitConstants<SF_Pixel>(PSConstantBuffer, StateCache, bDiscardSharedConstants);
	}

	bDiscardSharedConstants = false;
}

void FD3D11DynamicRHI::CommitComputeShaderConstants()
{
	// Commit and bind compute shader constants
	FD3DRHIUtil::CommitConstants<SF_Compute>(CSConstantBuffer, StateCache, bDiscardSharedConstants);
}

template <class ShaderType>
void FD3D11DynamicRHI::SetResourcesFromTables(const ShaderType* RESTRICT Shader)
{
	checkSlow(Shader);
	static constexpr EShaderFrequency Frequency = static_cast<EShaderFrequency>(ShaderType::StaticFrequency);

	UE::RHI::Private::SetUniformBufferResourcesFromTables(
		  FD3D11ResourceBinder<Frequency> { *this }
		, *Shader
		, DirtyUniformBuffers[Frequency]
		, BoundUniformBuffers[Frequency]
#if ENABLE_RHI_VALIDATION
		, Tracker
#endif
	);
}

void FD3D11DynamicRHI::CommitGraphicsResourceTables()
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	GDX11CommitGraphicsResourceTables.Increment();
#endif

	FD3D11BoundShaderState* RESTRICT CurrentBoundShaderState = (FD3D11BoundShaderState*)BoundShaderStateHistory.GetLast();
	check(CurrentBoundShaderState);

	bool bRTVInvalidate = false;
	uint32 UAVMask = 0;

	if (auto* Shader = CurrentBoundShaderState->GetPixelShader())
	{
		UAVMask |= Shader->UAVMask & CurrentRTVOverlapMask;
		SetResourcesFromTables(Shader);
	}

	if (auto* Shader = CurrentBoundShaderState->GetVertexShader())
	{
		UAVMask |= Shader->UAVMask & CurrentRTVOverlapMask;
		SetResourcesFromTables(Shader);
	}
	if (auto* Shader = CurrentBoundShaderState->GetGeometryShader())
	{
		UAVMask |= Shader->UAVMask & CurrentRTVOverlapMask;
		SetResourcesFromTables(Shader);
	}

	// Because d3d11 binding uses the same slots for UAVs and RTVs, we have to rebind when two shaders with different sets of rendertargets are bound,
	// as they can potentially be used by UAVs, which can cause them to unbind RTVs used by subsequent shaders.
	if (GDX11ReduceRTVRebinds &&
		(0 != ((~CurrentUAVMask) & UAVMask) && CurrentUAVMask == (CurrentUAVMask & UAVMask)))
	{
		//if the mask only -adds- uav binds, no RTs will be missing so we just grow the mask
		CurrentUAVMask = UAVMask;
	}
	else if (CurrentUAVMask != UAVMask)
	{
		bRTVInvalidate = true;
		CurrentUAVMask = UAVMask;
	}

	if (bRTVInvalidate)
	{
		CommitRenderTargets(true);
		DirtyUniformBuffers[SF_Pixel] = -1;
		DirtyUniformBuffers[SF_Vertex] = -1;
		DirtyUniformBuffers[SF_Geometry] = -1;
	}

	if (UAVSChanged)
	{
		CommitUAVs();
	}
}

void FD3D11DynamicRHI::CommitComputeResourceTables(FD3D11ComputeShader* InComputeShader)
{
	FD3D11ComputeShader* RESTRICT ComputeShader = InComputeShader;
	check(ComputeShader);
	SetResourcesFromTables(ComputeShader);
}

void FD3D11DynamicRHI::RHIDrawPrimitive(uint32 BaseVertexIndex,uint32 NumPrimitives,uint32 NumInstances)
{
	CommitGraphicsResourceTables();
	CommitNonComputeShaderConstants();

	uint32 VertexCount = GetVertexCountForPrimitiveCount(NumPrimitives, PrimitiveType);

	RHI_DRAW_CALL_STATS(PrimitiveType, VertexCount, NumPrimitives, NumInstances);

	StateCache.SetPrimitiveTopology(GetD3D11PrimitiveType(PrimitiveType));
	if(NumInstances > 1)
	{
		Direct3DDeviceIMContext->DrawInstanced(VertexCount,NumInstances,BaseVertexIndex,0);
	}
	else
	{
		Direct3DDeviceIMContext->Draw(VertexCount,BaseVertexIndex);
	}

	EnableUAVOverlap();
}

void FD3D11DynamicRHI::RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	FD3D11Buffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);

	RHI_DRAW_CALL_INC();

	CommitGraphicsResourceTables();
	CommitNonComputeShaderConstants();

	StateCache.SetPrimitiveTopology(GetD3D11PrimitiveType(PrimitiveType));
	Direct3DDeviceIMContext->DrawInstancedIndirect(ArgumentBuffer->Resource,ArgumentOffset);

	EnableUAVOverlap();
}

void FD3D11DynamicRHI::RHIDrawIndexedPrimitive(FRHIBuffer* IndexBufferRHI, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances)
{
	RHI_DRAW_CALL_STATS(PrimitiveType, NumVertices, NumPrimitives, NumInstances);

	FD3D11Buffer* IndexBuffer = ResourceCast(IndexBufferRHI);

	// called should make sure the input is valid, this avoid hidden bugs
	ensure(NumPrimitives > 0);

	CommitGraphicsResourceTables();
	CommitNonComputeShaderConstants();

	// determine 16bit vs 32bit indices
	const DXGI_FORMAT Format = (IndexBuffer->GetStride() == sizeof(uint16) ? DXGI_FORMAT_R16_UINT : DXGI_FORMAT_R32_UINT);

	uint32 IndexCount = GetVertexCountForPrimitiveCount(NumPrimitives,PrimitiveType);

	// Verify that we are not trying to read outside the index buffer range
	// test is an optimized version of: StartIndex + IndexCount <= IndexBuffer->GetSize() / IndexBuffer->GetStride() 
	checkf((StartIndex + IndexCount) * IndexBuffer->GetStride() <= IndexBuffer->GetSize(), 		
		TEXT("Start %u, Count %u, Type %u, Buffer Size %u, Buffer stride %u"), StartIndex, IndexCount, PrimitiveType, IndexBuffer->GetSize(), IndexBuffer->GetStride());

	TrackResourceBoundAsIB(IndexBuffer);
	StateCache.SetIndexBuffer(IndexBuffer->Resource, Format, 0);
	StateCache.SetPrimitiveTopology(GetD3D11PrimitiveType(PrimitiveType));

	if (NumInstances > 1 || FirstInstance != 0)
	{
		const uint64 TotalIndexCount = (uint64)NumInstances * (uint64)IndexCount + (uint64)StartIndex;
		checkf(TotalIndexCount <= (uint64)0xFFFFFFFF, TEXT("Instanced Index Draw exceeds maximum d3d11 limit: Total: %llu, NumInstances: %u, IndexCount: %u, StartIndex: %u, FirstInstance: %u"), TotalIndexCount, NumInstances, IndexCount, StartIndex, FirstInstance);
		Direct3DDeviceIMContext->DrawIndexedInstanced(IndexCount, NumInstances, StartIndex, BaseVertexIndex, FirstInstance);
	}
	else
	{
		Direct3DDeviceIMContext->DrawIndexed(IndexCount,StartIndex,BaseVertexIndex);
	}

	EnableUAVOverlap();
}

void FD3D11DynamicRHI::RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBufferRHI, FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	FD3D11Buffer* IndexBuffer = ResourceCast(IndexBufferRHI);
	FD3D11Buffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);

	RHI_DRAW_CALL_INC();
	
	CommitGraphicsResourceTables();
	CommitNonComputeShaderConstants();
	
	// Set the index buffer.
	const uint32 SizeFormat = sizeof(DXGI_FORMAT);
	const DXGI_FORMAT Format = (IndexBuffer->GetStride() == sizeof(uint16) ? DXGI_FORMAT_R16_UINT : DXGI_FORMAT_R32_UINT);
	TrackResourceBoundAsIB(IndexBuffer);
	StateCache.SetIndexBuffer(IndexBuffer->Resource, Format, 0);
	StateCache.SetPrimitiveTopology(GetD3D11PrimitiveType(PrimitiveType));
	Direct3DDeviceIMContext->DrawIndexedInstancedIndirect(ArgumentBuffer->Resource,ArgumentOffset);

	EnableUAVOverlap();
}

void FD3D11DynamicRHI::RHIClearMRTImpl(const bool* bClearColorArray, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil)
{
	FD3D11BoundRenderTargets BoundRenderTargets(Direct3DDeviceIMContext);

	// Must specify enough clear colors for all active RTs
	check(!bClearColorArray || NumClearColors >= BoundRenderTargets.GetNumActiveTargets());

	// If we're clearing depth or stencil and we have a readonly depth/stencil view bound, we need to use a writable depth/stencil view
	if (CurrentDepthTexture)
	{
		FExclusiveDepthStencil RequestedAccess;
		
		RequestedAccess.SetDepthStencilWrite(bClearDepth, bClearStencil);

		ensure(RequestedAccess.IsValid(CurrentDSVAccessType));
	}

	ID3D11DepthStencilView* DepthStencilView = BoundRenderTargets.GetDepthStencilView();

	if (bClearColorArray && BoundRenderTargets.GetNumActiveTargets() > 0)
	{
		for (int32 TargetIndex = 0; TargetIndex < BoundRenderTargets.GetNumActiveTargets(); TargetIndex++)
		{
			if (bClearColorArray[TargetIndex])
			{
				ID3D11RenderTargetView* RenderTargetView = BoundRenderTargets.GetRenderTargetView(TargetIndex);
				if (RenderTargetView != nullptr)
				{
					Direct3DDeviceIMContext->ClearRenderTargetView(RenderTargetView, (float*)&ClearColorArray[TargetIndex]);
				}
			}
		}
	}

	if ((bClearDepth || bClearStencil) && DepthStencilView)
	{
		uint32 ClearFlags = 0;
		if (bClearDepth)
		{
			ClearFlags |= D3D11_CLEAR_DEPTH;
		}
		if (bClearStencil)
		{
			ClearFlags |= D3D11_CLEAR_STENCIL;
		}
		Direct3DDeviceIMContext->ClearDepthStencilView(DepthStencilView,ClearFlags,Depth,Stencil);
	}
}

// Blocks the CPU until the GPU catches up and goes idle.
void FD3D11DynamicRHI::RHIBlockUntilGPUIdle()
{
	D3D11_QUERY_DESC Desc = {};
	Desc.Query = D3D11_QUERY_EVENT;

	TRefCountPtr<ID3D11Query> Query;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateQuery(&Desc, Query.GetInitReference()), Direct3DDevice);
	
	Direct3DDeviceIMContext->End(Query.GetReference());
	Direct3DDeviceIMContext->Flush();

	for(;;)
	{
		BOOL EventComplete = false;
		Direct3DDeviceIMContext->GetData(Query.GetReference(), &EventComplete, sizeof(EventComplete), 0);
		if (EventComplete)
		{
			break;
		}
		else
		{
			FPlatformProcess::Sleep(0.005f);
		}
	}
}

// NVIDIA Depth Bounds Test interface
void FD3D11DynamicRHI::EnableDepthBoundsTest(bool bEnable,float MinDepth,float MaxDepth)
{
#if PLATFORM_DESKTOP
	if(MinDepth > MaxDepth)
	{
		UE_LOGF(LogD3D11RHI, Error,"RHIEnableDepthBoundsTest(%i,%f, %f) MinDepth > MaxDepth, cannot set DBT.",bEnable,MinDepth,MaxDepth);
		return;
	}

	if( MinDepth < 0.f || MaxDepth > 1.f)
	{
		UE_LOGF(LogD3D11RHI, Verbose,"RHIEnableDepthBoundsTest(%i,%f, %f) depths out of range, will clamp.",bEnable,MinDepth,MaxDepth);
	}

	MinDepth = FMath::Clamp(MinDepth, 0.0f, 1.0f);
	MaxDepth = FMath::Clamp(MaxDepth, 0.0f, 1.0f);

#if WITH_NVAPI
	if (IsRHIDeviceNVIDIA())
	{
		auto Result = NvAPI_D3D11_SetDepthBoundsTest( Direct3DDevice, bEnable, MinDepth, MaxDepth );
		if (Result != NVAPI_OK)
		{
			static bool bOnce = false;
			if (!bOnce)
			{
				bOnce = true;
				if (bRenderDoc)
				{
					if (FApp::IsUnattended())
					{
						UE_LOGF(LogD3D11RHI, Display, "NvAPI is not available under RenderDoc");
					}
					else
					{
						UE_LOGF(LogD3D11RHI, Warning, "NvAPI is not available under RenderDoc");
					}
				}
				else
				{
					UE_LOGF(LogD3D11RHI, Error, "NvAPI_D3D11_SetDepthBoundsTest(%i,%f, %f) returned error code %i. **********PLEASE UPDATE YOUR VIDEO DRIVERS*********", bEnable, MinDepth, MaxDepth, (unsigned int)Result);
				}
			}
		}
	}
#endif
#if WITH_AMD_AGS
	if (IsRHIDeviceAMD())
	{
		auto Result = agsDriverExtensionsDX11_SetDepthBounds(AmdAgsContext, Direct3DDeviceIMContext, bEnable, MinDepth, MaxDepth);
		if(Result != AGS_SUCCESS)
		{
			static bool bOnce = false;
			if (!bOnce)
			{
				bOnce = true;
				if (bRenderDoc)
				{
					if (FApp::IsUnattended())
					{
						UE_LOGF(LogD3D11RHI, Display, "AGS is not available under RenderDoc");
					}
					else
					{
						UE_LOGF(LogD3D11RHI, Warning, "AGS is not available under RenderDoc");
					}
				}
				else
				{
					UE_LOGF(LogD3D11RHI, Error, "agsDriverExtensionsDX11_SetDepthBounds(%i,%f, %f) returned error code %i. **********PLEASE UPDATE YOUR VIDEO DRIVERS*********", bEnable, MinDepth, MaxDepth, (unsigned int)Result);
				}
			}
		}
	}
#endif
#endif

	StateCache.bDepthBoundsEnabled = bEnable;
	StateCache.DepthBoundsMin = MinDepth;
	StateCache.DepthBoundsMax = MaxDepth;
}

IRHICommandContext* FD3D11DynamicRHI::RHIGetDefaultContext()
{
	return this;
}

IRHIComputeContext* FD3D11DynamicRHI::RHIGetCommandContext(ERHIPipeline Pipeline)
{
	UE_LOGF(LogRHI, Fatal, "FD3D11DynamicRHI::RHIGetCommandContext should never be called. D3D11 RHI does not implement parallel command list execution.");
	return nullptr;
}

struct FD3D11PlatformCommandList : public IRHIPlatformCommandList
{
	virtual ~FD3D11PlatformCommandList() = default;
};

FDynamicRHI::FRHIFinalizeContextsResult FD3D11DynamicRHI::RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args)
{
	checkf(Args.UploadContext == nullptr, TEXT("D3D11 RHI does not implement upload contexts."));

	FlushProfilerStats();

	FRHIFinalizeContextsResult Result;

	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		checkf(Pipeline == ERHIPipeline::Graphics || !Args.Contexts[Pipeline], TEXT("Only graphics pipeline contexts are supported in D3D11 RHI."));

		if (IRHIComputeContext* Context = Args.Contexts[Pipeline])
		{
			// "Context" will always be the default context, since we don't implement parallel execution.
			check(Context == this);

		#if WITH_RHI_BREADCRUMBS
			// We need platform command lists to contain the breadcrumb allocators
			Result.CommandLists[Pipeline] = new FD3D11PlatformCommandList;
		#endif
		}
	}

	// Reset some context state
	for (int32 Frequency = 0; Frequency < SF_NumStandardFrequencies; ++Frequency)
	{
		DirtyUniformBuffers[Frequency] = 0;

		for (int32 BindIndex = 0; BindIndex < MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE; ++BindIndex)
		{
			BoundUniformBuffers[Frequency][BindIndex] = nullptr;
		}
	}

	return Result;
}

void FD3D11DynamicRHI::RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args)
{
	// Attempt to readback completed queries and fences
	PollQueryResults();
	FD3D11Sync::PollSyncs();

#if WITH_RHI_BREADCRUMBS
	for (IRHIPlatformCommandList* CmdList : Args.CommandLists)
	{
		FD3D11PlatformCommandList* D3DCmdList = static_cast<FD3D11PlatformCommandList*>(CmdList);

		// Preserve the breadcrumb allocators in the profiler frame
		// so they are kept alive until the frame's data is processed.
		for (auto const& Allocator : D3DCmdList->BreadcrumbAllocators)
		{
			Profiler.Current.BreadcrumbAllocators.AddUnique(&Allocator.Get());
		}

		delete D3DCmdList;
	}
#endif
}

void FD3D11DynamicRHI::EnableUAVOverlap()
{
	// This function is called after every draw or dispatch to turn overlap back on if it was turned off by a UAV barrier. This way, the next
	// draw/dispatch after the barrier executes after everything before it has completed and the caches were flushed, and any subsequent
	// submissions are allowed to overlap until the next UAV barrier.

	if (bUAVOverlapEnabled || !CVarAllowUAVFlushExt.GetValueOnRenderThread())
	{
		return;
	}

	bUAVOverlapEnabled = true;

	if (IsRHIDeviceNVIDIA())
	{
#if WITH_NVAPI
		NvAPI_D3D11_BeginUAVOverlap(Direct3DDevice);
#endif
	}
	else if (IsRHIDeviceAMD())
	{
#if WITH_AMD_AGS
		agsDriverExtensionsDX11_BeginUAVOverlap(AmdAgsContext, Direct3DDeviceIMContext);
#endif
	}
	else if (IsRHIDeviceIntel())
	{
#if INTEL_EXTENSIONS
		if (bIntelSupportsUAVOverlap)
		{
			INTC_D3D11_BeginUAVOverlap(IntelExtensionContext);
		}
#endif
	}
}

void FD3D11DynamicRHI::DisableUAVOverlap()
{
	// This is called when a transition to UAVCompute or UAVGraphics is executed. It disables overlapping for the next draw/dispatch, so we get the same
	// behavior as with a UAV barrier in APIs with explicit barriers. Overlapping will be turned back on automatically after the draw/dispatch.
	if (!bUAVOverlapEnabled)
	{
		return;
	}

	if (IsRHIDeviceNVIDIA())
	{
#if WITH_NVAPI
		NvAPI_D3D11_EndUAVOverlap(Direct3DDevice);
#endif
	}
	else if (IsRHIDeviceAMD())
	{
#if WITH_AMD_AGS
		agsDriverExtensionsDX11_EndUAVOverlap(AmdAgsContext, Direct3DDeviceIMContext);
#endif
	}
	else if (IsRHIDeviceIntel())
	{
#if INTEL_EXTENSIONS
		if (bIntelSupportsUAVOverlap)
		{
			INTC_D3D11_EndUAVOverlap(IntelExtensionContext);
		}
#endif
	}

	bUAVOverlapEnabled = false;
}

void FD3D11DynamicRHI::RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo)
{
	checkf(FMath::IsPowerOfTwo(uint32(CreateInfo.SrcPipelines)) && FMath::IsPowerOfTwo(uint32(CreateInfo.DstPipelines)), TEXT("Support for multi-pipe resources is not yet implemented."));

	FD3D11TransitionData* Data = new (Transition->GetPrivateData<FD3D11TransitionData>()) FD3D11TransitionData;
	Data->bUAVBarrier = false;

	// If we have any transitions to UAVCompute or UAVGraphics, we need to break up the current overlap group.
	for (const FRHITransitionInfo& Info : CreateInfo.TransitionInfos)
	{
		if (Info.Resource && EnumHasAnyFlags(Info.AccessAfter, ERHIAccess::UAVMask))
		{
			Data->bUAVBarrier = true;
			break;
		}
	}
}

void FD3D11DynamicRHI::RHIReleaseTransition(FRHITransition* Transition)
{
	Transition->GetPrivateData<FD3D11TransitionData>()->~FD3D11TransitionData();
}

void FD3D11DynamicRHI::RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions)
{
}

void FD3D11DynamicRHI::RHIEndTransitions(TArrayView<const FRHITransition*> Transitions)
{
	// The only thing we care about in D3D11 is breaking up the current overlap group if we have a UAV barrier. If overlap is already off, there's nothing to do.
	if (!bUAVOverlapEnabled)
	{
		return;
	}

	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D11TransitionData* Data = Transition->GetPrivateData<FD3D11TransitionData>();
		if (Data->bUAVBarrier)
		{
			DisableUAVOverlap();
			break;
		}
	}
}

//*********************** StagingBuffer Implementation ***********************//

FStagingBufferRHIRef FD3D11DynamicRHI::RHICreateStagingBuffer()
{
	return new FD3D11StagingBuffer();
}

FD3D11StagingBuffer::~FD3D11StagingBuffer()
{
	if (StagedRead)
	{
		StagedRead.SafeRelease();
	}
}

void* FD3D11StagingBuffer::Lock(uint32 Offset, uint32 NumBytes)
{
	check(!bIsLocked);
	bIsLocked = true;
	if (StagedRead)
	{
		// Map the staging buffer's memory for reading.
		D3D11_MAPPED_SUBRESOURCE MappedSubresource;
		VERIFYD3D11RESULT(Context->Map(StagedRead, 0, D3D11_MAP_READ, 0, &MappedSubresource));

		return (void*)((uint8*)MappedSubresource.pData + Offset);
	}
	else
	{
		return nullptr;
	}
}

void FD3D11StagingBuffer::Unlock()
{
	check(bIsLocked);
	bIsLocked = false;
	if (StagedRead)
	{
		Context->Unmap(StagedRead, 0);
	}
}

void FD3D11DynamicRHI::RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* StagingBufferRHI, uint32 Offset, uint32 NumBytes)
{
	FD3D11Buffer* SourceBuffer = ResourceCast(SourceBufferRHI);
	FD3D11StagingBuffer* StagingBuffer = ResourceCast(StagingBufferRHI);
	if (StagingBuffer)
	{
		ensureMsgf(!StagingBuffer->bIsLocked, TEXT("Attempting to Copy to a locked staging buffer. This may have undefined behavior"));
		if (SourceBuffer)
		{
			if (!StagingBuffer->StagedRead || StagingBuffer->ShadowBufferSize < NumBytes)
			{
				// Free previously allocated buffer.
				if (StagingBuffer->StagedRead)
				{
					StagingBuffer->StagedRead.SafeRelease();
				}

				// Allocate a new one with enough space.
				// @todo-mattc I feel like we should allocate more than NumBytes to handle small reads without blowing tons of space. Need to pool this.
				D3D11_BUFFER_DESC StagedReadDesc;
				ZeroMemory(&StagedReadDesc, sizeof(D3D11_BUFFER_DESC));
				StagedReadDesc.ByteWidth = NumBytes;
				StagedReadDesc.Usage = D3D11_USAGE_STAGING;
				StagedReadDesc.BindFlags = 0;
				StagedReadDesc.CPUAccessFlags = D3D11_CPU_ACCESS_READ;
				StagedReadDesc.MiscFlags = 0;
				TRefCountPtr<ID3D11Buffer> StagingVertexBuffer;
				VERIFYD3D11RESULT_EX(Direct3DDevice->CreateBuffer(&StagedReadDesc, NULL, StagingBuffer->StagedRead.GetInitReference()), Direct3DDevice);
				StagingBuffer->ShadowBufferSize = NumBytes;
				StagingBuffer->Context = Direct3DDeviceIMContext;
			}

			// Copy the contents of the vertex buffer to the staging buffer.
			D3D11_BOX SourceBox;
			SourceBox.left = Offset;
			SourceBox.right = Offset + NumBytes;
			SourceBox.top = SourceBox.front = 0;
			SourceBox.bottom = SourceBox.back = 1;
			Direct3DDeviceIMContext->CopySubresourceRegion(StagingBuffer->StagedRead, 0, 0, 0, 0, SourceBuffer->Resource, 0, &SourceBox);
		}
	}
}

void* FD3D11DynamicRHI::RHILockStagingBuffer(FRHIStagingBuffer* StagingBufferRHI, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI)
{
	check(StagingBufferRHI);
	FD3D11StagingBuffer* StagingBuffer = ResourceCast(StagingBufferRHI);
	return StagingBuffer->Lock(Offset, SizeRHI);
}

void FD3D11DynamicRHI::RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBufferRHI)
{
	FD3D11StagingBuffer* StagingBuffer = ResourceCast(StagingBufferRHI);
	StagingBuffer->Unlock();
}


TSpscQueue<FD3D11Sync> FD3D11Sync::ActiveSyncs;

void FD3D11Sync::EnqueueSync(FGraphEventRef Event)
{
	FD3D11DynamicRHI& RHI = FD3D11DynamicRHI::Get();

	// Insert an event query on the device context
	D3D11_QUERY_DESC Desc{};
	Desc.Query = D3D11_QUERY_EVENT;

	TRefCountPtr<ID3D11Query> Query;
	VERIFYD3D11RESULT(RHI.GetDevice()->CreateQuery(&Desc, Query.GetInitReference()));
	RHI.GetDeviceContext()->End(Query);

	// Store the query in the list of queries to poll
	ActiveSyncs.Enqueue(FD3D11Sync(MoveTemp(Event), MoveTemp(Query)));
}

void FD3D11Sync::PollSyncsUntil(FGraphEvent* Target)
{
	FD3D11DynamicRHI& RHI = FD3D11DynamicRHI::Get();

	// Flush the queue if we have a specific fence to wait for, otherwise the commands that signal the
	// fence may be stuck in the driver's command queue and never reach the GPU, causing an infinite loop.
	UINT const Flags = Target ? 0 : D3D11_ASYNC_GETDATA_DONOTFLUSH;

	while (FD3D11Sync* Sync = ActiveSyncs.Peek())
	{
	Retry:
		BOOL Value = 0;
		HRESULT Result = RHI.GetDeviceContext()->GetData(Sync->Query, &Value, sizeof(Value), Flags);

		if (Result == S_FALSE || (Result == S_OK && !Value))
		{
			// Fence is not done
			if (Target)
			{
				// We're waiting for a specific fence. Spin until it passes.
				goto Retry;
			}
			else
			{
				return;
			}
		}

		VERIFYD3D11RESULT(Result);

		bool bFoundTarget = Sync->Event == Target;

		// The fence has completed. Signal the graph event and remove the node.
		Sync->Event->DispatchSubsequents();
		ActiveSyncs.Dequeue();

		if (bFoundTarget)
		{
			// We found the fence we wanted. Stop polling.
			return;
		}
	}

	UE_CLOGF(Target, LogD3D11RHI, Fatal, "FD3D11Sync::PollSyncsUntil - Attempt to poll for a specific fence (0x%p), but it was not found in the queue.", Target);
}

FD3D11GPUFence::FD3D11GPUFence(FName InName)
	: FRHIGPUFence(InName)
{
}

void FD3D11GPUFence::Clear()
{
	Event = nullptr;
}

bool FD3D11GPUFence::Poll() const
{
	return Event && Event->IsComplete();
}

void FD3D11GPUFence::Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const
{
	if (Event && !Event->IsComplete())
	{
		//
		// The fence might get signalled by an earlier RHI command polling them, but we can't be sure that will happen.
		// The GPU might finish work after the RHI thread has gone idle, and then we'll never see the fence complete.
		//
		// Enqueue a command here that will block and wait for the fence if it still hasn't signalled by the time
		// the RHI thread is done with all prior commands.
		//
		RHICmdList.EnqueueLambda([Event = Event](FRHICommandListImmediate&)
		{
			if (!Event->IsComplete())
			{
				FD3D11Sync::PollSyncsUntil(Event);
			}
		});
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

		Event->Wait();
	}
}

void FD3D11GPUFence::WriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList)
{
	Event = FGraphEvent::CreateGraphEvent();

	RHICmdList.EnqueueLambda([Event = Event](FRHICommandListBase&) mutable
	{
		FD3D11Sync::EnqueueSync(Event);
	});
}

void FD3D11DynamicRHI::RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI)
{
	ResourceCast(FenceRHI)->WriteGPUFence_TopOfPipe(RHICmdList);
}

void FD3D11DynamicRHI::RHIWriteGPUFence(FRHIGPUFence* FenceRHI)
{
	checkNoEntry(); // Should never be called
}

FGPUFenceRHIRef FD3D11DynamicRHI::RHICreateGPUFence(const FName& Name)
{
	return new FD3D11GPUFence(Name);
}
