// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11ConstantBuffer.cpp: D3D Constant buffer RHI implementation.
=============================================================================*/

#include "D3D11ConstantBuffer.h"
#include "D3D11RHIPrivate.h"

// New circular buffer system for faster constant uploads.  Avoids CopyResource and speeds things up considerably
FD3D11ConstantBuffer::FD3D11ConstantBuffer(FD3D11DynamicRHI* InD3DRHI)
	: D3DRHI(InD3DRHI)
{
	InitResource(FRHICommandListImmediate::Get());
}

FD3D11ConstantBuffer::~FD3D11ConstantBuffer()
{
	ReleaseResource();
}

/**
* Creates a constant buffer on the device
*/
void FD3D11ConstantBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
// New circular buffer system for faster constant uploads.  Avoids CopyResource and speeds things up considerably
	// aligned for best performance
	ShadowData = (uint8*)FMemory::Malloc(GetMaxSize(), 16);
	FMemory::Memzero(ShadowData, GetMaxSize());
}

void FD3D11ConstantBuffer::ReleaseRHI()
{
	if(ShadowData)
	{
		FMemory::Free(ShadowData);
	}
}
