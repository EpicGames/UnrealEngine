// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Device.cpp: D3D device RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "D3D11ConstantBuffer.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <delayimp.h>
	#if WITH_AMD_AGS
	#include "amd_ags.h"
	#endif
#include "Windows/HideWindowsPlatformTypes.h"

bool D3D11RHI_ShouldAllowAsyncResourceCreation()
{
	static bool bAllowAsyncResourceCreation = !FParse::Param(FCommandLine::Get(),TEXT("nod3dasync"));
	return bAllowAsyncResourceCreation;
}

IMPLEMENT_MODULE(FD3D11DynamicRHIModule, D3D11RHI);

static TAutoConsoleVariable<int32> CVarD3D11UseD24(
	TEXT("r.D3D11.Depth24Bit"),
	0,
	TEXT("0: Use 32-bit float depth buffer\n1: Use 24-bit fixed point depth buffer(default)\n"),
	ECVF_ReadOnly
);



TAutoConsoleVariable<int32> CVarD3D11ZeroBufferSizeInMB(
	TEXT("d3d11.ZeroBufferSizeInMB"),
	4,
	TEXT("The D3D11 RHI needs a static allocation of zeroes to use when streaming textures asynchronously. It should be large enough to support the largest mipmap you need to stream. The default is 4MB."),
	ECVF_ReadOnly
	);


FD3D11DynamicRHI::FD3D11DynamicRHI(IDXGIFactory1* InDXGIFactory1, D3D_FEATURE_LEVEL InFeatureLevel, const FD3D11Adapter& InAdapter)
	: DXGIFactory1(InDXGIFactory1)
	, FeatureLevel(InFeatureLevel)
	, AmdAgsContext(NULL)
#if INTEL_EXTENSIONS
	, IntelExtensionContext(nullptr)
	, bIntelSupportsUAVOverlap(false)
#endif
	, bCurrentDepthStencilStateIsReadOnly(false)
	, CurrentDepthTexture(NULL)
	, NumSimultaneousRenderTargets(0)
	, NumUAVs(0)
	, CurrentDSVAccessType(FExclusiveDepthStencil::DepthWrite_StencilWrite)
	, bDiscardSharedConstants(false)
	, Adapter(InAdapter)
{
	// This should be called once at the start 
	check(Adapter.IsValid());
	check(IsInGameThread());
	check(!GIsThreadedRendering);

	// Allocate a buffer of zeroes. This is used when we need to pass D3D memory
	// that we don't care about and will overwrite with valid data in the future.
	ZeroBufferSize = FMath::Max(CVarD3D11ZeroBufferSizeInMB.GetValueOnAnyThread(), 0) * (1 << 20);
	ZeroBuffer = FMemory::Malloc(ZeroBufferSize);
	FMemory::Memzero(ZeroBuffer,ZeroBufferSize);

	GPoolSizeVRAMPercentage = 0;
	GTexturePoolSize = 0;
	GConfig->GetInt( TEXT( "TextureStreaming" ), TEXT( "PoolSizeVRAMPercentage" ), GPoolSizeVRAMPercentage, GEngineIni );	

	// Initialize the RHI capabilities.
	check(FeatureLevel >= D3D_FEATURE_LEVEL_11_0);

   	
	TRefCountPtr<IDXGIFactory5> Factory5;
	HRESULT HResult = DXGIFactory1->QueryInterface(IID_PPV_ARGS(Factory5.GetInitReference()));
	if (SUCCEEDED(HResult))
	{
		bDXGISupportsHDR = true;
	}
	else
	{
		bDXGISupportsHDR = false;
	}

	// Initialize the platform pixel format map.
	GPixelFormats[ PF_Unknown		].PlatformFormat	= DXGI_FORMAT_UNKNOWN;
	GPixelFormats[ PF_A32B32G32R32F	].PlatformFormat	= DXGI_FORMAT_R32G32B32A32_FLOAT;
	GPixelFormats[ PF_B8G8R8A8		].PlatformFormat	= DXGI_FORMAT_B8G8R8A8_TYPELESS;
	GPixelFormats[ PF_G8			].PlatformFormat	= DXGI_FORMAT_R8_UNORM;
	GPixelFormats[ PF_G16			].PlatformFormat	= DXGI_FORMAT_R16_UNORM;
	GPixelFormats[ PF_DXT1			].PlatformFormat	= DXGI_FORMAT_BC1_TYPELESS;
	GPixelFormats[ PF_DXT3			].PlatformFormat	= DXGI_FORMAT_BC2_TYPELESS;
	GPixelFormats[ PF_DXT5			].PlatformFormat	= DXGI_FORMAT_BC3_TYPELESS;
	GPixelFormats[ PF_BC4			].PlatformFormat	= DXGI_FORMAT_BC4_UNORM;
	GPixelFormats[ PF_UYVY			].PlatformFormat	= DXGI_FORMAT_UNKNOWN;		// TODO: Not supported in D3D11
	if (CVarD3D11UseD24.GetValueOnAnyThread())
	{
		GPixelFormats[PF_DepthStencil].PlatformFormat = DXGI_FORMAT_R24G8_TYPELESS;
		GPixelFormats[PF_DepthStencil].BlockBytes = 4;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = true;
		GPixelFormats[PF_X24_G8].PlatformFormat = DXGI_FORMAT_X24_TYPELESS_G8_UINT;
		GPixelFormats[PF_X24_G8].BlockBytes = 4;
	}
	else
	{
		GPixelFormats[PF_DepthStencil].PlatformFormat = DXGI_FORMAT_R32G8X24_TYPELESS;
		GPixelFormats[PF_DepthStencil].BlockBytes = 5;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = false;
		GPixelFormats[PF_X24_G8].PlatformFormat = DXGI_FORMAT_X32_TYPELESS_G8X24_UINT;
		GPixelFormats[PF_X24_G8].BlockBytes = 5;
	}
	GPixelFormats[ PF_DepthStencil	].Supported = true;
	GPixelFormats[ PF_X24_G8		].Supported = true;
	GPixelFormats[ PF_ShadowDepth	].PlatformFormat	= DXGI_FORMAT_R16_TYPELESS;
	GPixelFormats[ PF_ShadowDepth	].BlockBytes		= 2;
	GPixelFormats[ PF_ShadowDepth	].Supported			= true;
	GPixelFormats[ PF_R32_FLOAT		].PlatformFormat	= DXGI_FORMAT_R32_FLOAT;
	GPixelFormats[ PF_G16R16		].PlatformFormat	= DXGI_FORMAT_R16G16_UNORM;
	GPixelFormats[ PF_G16R16F		].PlatformFormat	= DXGI_FORMAT_R16G16_FLOAT;
	GPixelFormats[ PF_G16R16F_FILTER].PlatformFormat	= DXGI_FORMAT_R16G16_FLOAT;
	GPixelFormats[ PF_G32R32F		].PlatformFormat	= DXGI_FORMAT_R32G32_FLOAT;
	GPixelFormats[ PF_A2B10G10R10   ].PlatformFormat    = DXGI_FORMAT_R10G10B10A2_UNORM;
	GPixelFormats[ PF_A16B16G16R16  ].PlatformFormat    = DXGI_FORMAT_R16G16B16A16_UNORM;
	GPixelFormats[ PF_D24 ].PlatformFormat				= DXGI_FORMAT_R24G8_TYPELESS;
	GPixelFormats[ PF_R16F			].PlatformFormat	= DXGI_FORMAT_R16_FLOAT;
	GPixelFormats[ PF_R16F_FILTER	].PlatformFormat	= DXGI_FORMAT_R16_FLOAT;

	GPixelFormats[ PF_FloatRGB	].PlatformFormat		= DXGI_FORMAT_R11G11B10_FLOAT;
	GPixelFormats[ PF_FloatRGB	].BlockBytes			= 4;
	GPixelFormats[ PF_FloatRGBA	].PlatformFormat		= DXGI_FORMAT_R16G16B16A16_FLOAT;
	GPixelFormats[ PF_FloatRGBA	].BlockBytes			= 8;

	GPixelFormats[ PF_FloatR11G11B10].PlatformFormat	= DXGI_FORMAT_R11G11B10_FLOAT;
	GPixelFormats[ PF_FloatR11G11B10].BlockBytes		= 4;
	GPixelFormats[ PF_FloatR11G11B10].Supported			= true;

	GPixelFormats[ PF_V8U8			].PlatformFormat	= DXGI_FORMAT_R8G8_SNORM;
	GPixelFormats[ PF_BC5			].PlatformFormat	= DXGI_FORMAT_BC5_UNORM;
	GPixelFormats[ PF_A1			].PlatformFormat	= DXGI_FORMAT_R1_UNORM; // Not supported for rendering.
	GPixelFormats[ PF_A8			].PlatformFormat	= DXGI_FORMAT_A8_UNORM;
	GPixelFormats[ PF_R32_UINT		].PlatformFormat	= DXGI_FORMAT_R32_UINT;
	GPixelFormats[ PF_R32_SINT		].PlatformFormat	= DXGI_FORMAT_R32_SINT;

	GPixelFormats[ PF_R16_UINT         ].PlatformFormat = DXGI_FORMAT_R16_UINT;
	GPixelFormats[ PF_R16_SINT         ].PlatformFormat = DXGI_FORMAT_R16_SINT;
	GPixelFormats[ PF_R16G16B16A16_UINT].PlatformFormat = DXGI_FORMAT_R16G16B16A16_UINT;
	GPixelFormats[ PF_R16G16B16A16_SINT].PlatformFormat = DXGI_FORMAT_R16G16B16A16_SINT;

	GPixelFormats[ PF_R5G6B5_UNORM	].PlatformFormat	= DXGI_FORMAT_B5G6R5_UNORM;
	GPixelFormats[ PF_R5G6B5_UNORM  ].Supported         = true;
	GPixelFormats[ PF_B5G5R5A1_UNORM].PlatformFormat    = DXGI_FORMAT_B5G5R5A1_UNORM;
	GPixelFormats[ PF_B5G5R5A1_UNORM].Supported         = true;
	GPixelFormats[ PF_R8G8B8A8		].PlatformFormat	= DXGI_FORMAT_R8G8B8A8_TYPELESS;
	GPixelFormats[ PF_R8G8B8A8_UINT	].PlatformFormat	= DXGI_FORMAT_R8G8B8A8_UINT;
	GPixelFormats[ PF_R8G8B8A8_SNORM].PlatformFormat	= DXGI_FORMAT_R8G8B8A8_SNORM;
	GPixelFormats[ PF_R8G8			].PlatformFormat	= DXGI_FORMAT_R8G8_UNORM;
	GPixelFormats[ PF_R32G32B32A32_UINT].PlatformFormat = DXGI_FORMAT_R32G32B32A32_UINT;
	GPixelFormats[ PF_R16G16_UINT   ].PlatformFormat    = DXGI_FORMAT_R16G16_UINT;
	GPixelFormats[ PF_R16G16_SINT   ].PlatformFormat    = DXGI_FORMAT_R16G16_SINT;
	GPixelFormats[ PF_R32G32_UINT   ].PlatformFormat    = DXGI_FORMAT_R32G32_UINT;

	GPixelFormats[ PF_BC6H			].PlatformFormat	= DXGI_FORMAT_BC6H_UF16;
	GPixelFormats[ PF_BC7			].PlatformFormat	= DXGI_FORMAT_BC7_TYPELESS;
	GPixelFormats[ PF_R8_UINT		].PlatformFormat	= DXGI_FORMAT_R8_UINT;
	GPixelFormats[ PF_R8			].PlatformFormat	= DXGI_FORMAT_R8_UNORM;

	GPixelFormats[PF_R16G16B16A16_UNORM].PlatformFormat = DXGI_FORMAT_R16G16B16A16_UNORM;
	GPixelFormats[PF_R16G16B16A16_SNORM].PlatformFormat = DXGI_FORMAT_R16G16B16A16_SNORM;

	GPixelFormats[PF_NV12			].PlatformFormat = DXGI_FORMAT_NV12;
	GPixelFormats[PF_NV12			].Supported = true;

	GPixelFormats[PF_G16R16_SNORM	].PlatformFormat = DXGI_FORMAT_R16G16_SNORM;
	GPixelFormats[PF_R8G8_UINT		].PlatformFormat = DXGI_FORMAT_R8G8_UINT;
	GPixelFormats[PF_R32G32B32_UINT	].PlatformFormat = DXGI_FORMAT_R32G32B32_UINT;
	GPixelFormats[PF_R32G32B32_SINT	].PlatformFormat = DXGI_FORMAT_R32G32B32_SINT;
	GPixelFormats[PF_R32G32B32F		].PlatformFormat = DXGI_FORMAT_R32G32B32_FLOAT;
	GPixelFormats[PF_R8_SINT		].PlatformFormat = DXGI_FORMAT_R8_SINT;

	GPixelFormats[PF_P010			].PlatformFormat = DXGI_FORMAT_P010;
	GPixelFormats[PF_P010			].Supported = true;

	GSupportsSeparateRenderTargetBlendState = true;
	GMaxTextureDimensions = D3D11_REQ_TEXTURE2D_U_OR_V_DIMENSION;
	GMaxCubeTextureDimensions = D3D11_REQ_TEXTURECUBE_DIMENSION;
	GMaxTextureArrayLayers = D3D11_REQ_TEXTURE2D_ARRAY_AXIS_DIMENSION;
	GRHIMaxConstantBufferByteSize = MAX_GLOBAL_CONSTANT_BUFFER_BYTE_SIZE;
	GRHISupportsMSAADepthSampleAccess = true;
	GRHISupportsRHIThread = true;

	GMaxTextureMipCount = FMath::CeilLogTwo( GMaxTextureDimensions ) + 1;
	GMaxTextureMipCount = FMath::Min<int32>( MAX_TEXTURE_MIP_COUNT, GMaxTextureMipCount );
	GMaxShadowDepthBufferSizeX = GMaxTextureDimensions;
	GMaxShadowDepthBufferSizeY = GMaxTextureDimensions;
	GSupportsTimestampRenderQueries = true;

	GRHIMaxDispatchThreadGroupsPerDimension.X = D3D11_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION;
	GRHIMaxDispatchThreadGroupsPerDimension.Y = D3D11_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION;
	GRHIMaxDispatchThreadGroupsPerDimension.Z = D3D11_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION;

	// All D3D11.1 hardware on Windows 8+ supports binding UAVs to Vertex Shaders.
	// Enable run-time support if corresponding bit is set in DDSPI.
	GRHIGlobals.SupportsVertexShaderUAVs = true;

	GRHIGlobals.NeedsShaderUnbinds = true;

	GRHITransitionPrivateData_SizeInBytes = sizeof(FD3D11TransitionData);
	GRHITransitionPrivateData_AlignInBytes = alignof(FD3D11TransitionData);

	GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer = 1 << 27;
	GRHIGlobals.MaxViewDimensionForTypedBuffer = 1 << 27;
	
	// D3D11 always uses DXBC. This is used to differentiate standard and preview shaders in the DDC.
	GRHIGlobals.PreferredPreviewShaderCodeFormat = TEXT("DXBC");

	// Initialize the constant buffers.
	InitConstantBuffers();

	for (int32 Frequency = 0; Frequency < SF_NumStandardFrequencies; ++Frequency)
	{
		DirtyUniformBuffers[Frequency] = 0;

		for (int32 BindIndex = 0; BindIndex < MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE; ++BindIndex)
		{
			BoundUniformBuffers[Frequency][BindIndex] = nullptr;
		}
	}

	StaticUniformBuffers.AddZeroed(FUniformBufferStaticSlotRegistry::Get().GetSlotCount());

#if PLATFORM_WINDOWS
	InitializeDeviceAgnosticVendorAPI();
#endif
}

FD3D11DynamicRHI::~FD3D11DynamicRHI()
{
	// Removed until shutdown crashes in exception handler are fixed.
#if WITH_DX_PERF
	//check(Direct3DDeviceIMAnnotation == nullptr);
#endif
	//check(Direct3DDeviceIMContext == nullptr);
	//check(Direct3DDevice == nullptr);
}

void FD3D11DynamicRHI::Shutdown()
{
	UE_LOGF(LogD3D11RHI, Log, "Shutdown");
	check(IsInGameThread() && IsInRenderingThread());  // require that the render thread has been shut down

	// Cleanup the D3D device.
	CleanupD3DDevice();

	// Release the buffer of zeroes.
	FMemory::Free(ZeroBuffer);
	ZeroBuffer = NULL;
	ZeroBufferSize = 0;
}

#if WITH_RHI_BREADCRUMBS
	void FD3D11DynamicRHI::RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
	#if NV_AFTERMATH
		UE::RHICore::Nvidia::Aftermath::D3D11::BeginBreadcrumb(AftermathHandle, Breadcrumb);
	#endif

		// Only emit formatted strings to platform APIs when requested.
		if (ShouldEmitBreadcrumbs())
		{
			const TCHAR* NameStr = nullptr;
			FRHIBreadcrumb::FBuffer Buffer;
			auto GetNameStr = [&]()
			{
				if (!NameStr)
				{
					NameStr = Breadcrumb->GetTCHAR(Buffer);
				}
				return NameStr;
			};

		#if WITH_DX_PERF
			Direct3DDeviceIMAnnotation->BeginEvent(GetNameStr());
		#endif
		}

		FlushProfilerStats();

		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginBreadcrumb>(Breadcrumb);
		InsertProfilerTimestamp(&Event.GPUTimestampTOP, false);
	}

	void FD3D11DynamicRHI::RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
		FlushProfilerStats();

		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndBreadcrumb>(Breadcrumb);
		InsertProfilerTimestamp(&Event.GPUTimestampBOP, false);

		// Only emit formatted strings to platform APIs when requested.
		if (ShouldEmitBreadcrumbs())
		{
		#if WITH_DX_PERF
			Direct3DDeviceIMAnnotation->EndEvent();
		#endif
		}

	#if NV_AFTERMATH
		UE::RHICore::Nvidia::Aftermath::D3D11::EndBreadcrumb(AftermathHandle, Breadcrumb);
	#endif
	}
#endif // WITH_RHI_BREADCRUMBS


/**
 * Returns a supported screen resolution that most closely matches the input.
 * @param Width - Input: Desired resolution width in pixels. Output: A width that the platform supports.
 * @param Height - Input: Desired resolution height in pixels. Output: A height that the platform supports.
 */
void FD3D11DynamicRHI::RHIGetSupportedResolution( uint32 &Width, uint32 &Height )
{
	uint32 InitializedMode = false;
	DXGI_MODE_DESC BestMode;
	BestMode.Width = 0;
	BestMode.Height = 0;

	{
		HRESULT HResult = S_OK;
	  
		// Enumerate outputs for this adapter
		// TODO: Cap at 1 for default output
		for(uint32 o = 0;o < 1; o++)
		{
			TRefCountPtr<IDXGIOutput> Output;
			HResult = Adapter.DXGIAdapter->EnumOutputs(o, Output.GetInitReference());
			if(DXGI_ERROR_NOT_FOUND == HResult)
			{
				break;
			}
			if(FAILED(HResult))
			{
				return;
			}

			// TODO: GetDisplayModeList is a terribly SLOW call.  It can take up to a second per invocation.
			//  We might want to work around some DXGI badness here.
			DXGI_FORMAT Format = DXGI_FORMAT_R8G8B8A8_UNORM;
			uint32 NumModes = 0;
			HResult = Output->GetDisplayModeList(Format,0,&NumModes,NULL);
			if(HResult == DXGI_ERROR_NOT_FOUND)
			{
				return;
			}
			else if(HResult == DXGI_ERROR_NOT_CURRENTLY_AVAILABLE)
			{
				UE_LOGF(LogD3D11RHI, Fatal,
					"This application cannot be run over a remote desktop configuration"
					);
				return;
			}
			DXGI_MODE_DESC* ModeList = new DXGI_MODE_DESC[ NumModes ];
			VERIFYD3D11RESULT(Output->GetDisplayModeList(Format,0,&NumModes,ModeList));

			for(uint32 m = 0;m < NumModes;m++)
			{
				// Search for the best mode
				
				// Suppress static analysis warnings about a potentially out-of-bounds read access to ModeList. This is a false positive - Index is always within range.
				CA_SUPPRESS( 6385 );
				bool IsEqualOrBetterWidth = FMath::Abs((int32)ModeList[m].Width - (int32)Width) <= FMath::Abs((int32)BestMode.Width - (int32)Width);
				bool IsEqualOrBetterHeight = FMath::Abs((int32)ModeList[m].Height - (int32)Height) <= FMath::Abs((int32)BestMode.Height - (int32)Height);
				if(!InitializedMode || (IsEqualOrBetterWidth && IsEqualOrBetterHeight))
				{
					BestMode = ModeList[m];
					InitializedMode = true;
				}
			}

			delete[] ModeList;
		}
	}

	check(InitializedMode);
	Width = BestMode.Width;
	Height = BestMode.Height;
}

void FD3D11DynamicRHI::GetBestSupportedMSAASetting( DXGI_FORMAT PlatformFormat, uint32 MSAACount, uint32& OutBestMSAACount, uint32& OutMSAAQualityLevels )
{
	// start counting down from current setting (indicated the current "best" count) and move down looking for support
	for(uint32 IndexCount = MSAACount;IndexCount > 0;IndexCount--)
	{
		uint32 NumMultiSampleQualities = 0;
		if(	SUCCEEDED(Direct3DDevice->CheckMultisampleQualityLevels(PlatformFormat,IndexCount,&NumMultiSampleQualities)) && NumMultiSampleQualities > 0 )
		{
			OutBestMSAACount = IndexCount;
			OutMSAAQualityLevels = NumMultiSampleQualities;
			break;
		}
	}
}

uint32 FD3D11DynamicRHI::GetMaxMSAAQuality(uint32 SampleCount)
{
	if(SampleCount <= DX_MAX_MSAA_COUNT)
	{
		// 0 has better quality (a more even distribution)
		// higher quality levels might be useful for non box filtered AA or when using weighted samples 
		return 0;
//		return AvailableMSAAQualities[SampleCount];
	}
	// not supported
	return 0xffffffff;
}

struct FFormatSupport
{
	D3D11_FORMAT_SUPPORT FormatSupport;
	D3D11_FORMAT_SUPPORT2 FormatSupport2;
};

static FFormatSupport GetFormatSupport(FD3D11Device* InDevice, DXGI_FORMAT InFormat)
{
	FFormatSupport Result{};

	{
		D3D11_FEATURE_DATA_FORMAT_SUPPORT FormatSupport{};
		FormatSupport.InFormat = InFormat;

		HRESULT SupportHR = InDevice->CheckFeatureSupport(D3D11_FEATURE_FORMAT_SUPPORT, &FormatSupport, sizeof(FormatSupport));
		if (SUCCEEDED(SupportHR))
		{
			Result.FormatSupport = (D3D11_FORMAT_SUPPORT)FormatSupport.OutFormatSupport;
		}
	}

	{
		D3D11_FEATURE_DATA_FORMAT_SUPPORT2 FormatSupport2{};
		FormatSupport2.InFormat = InFormat;

		HRESULT Support2HR = InDevice->CheckFeatureSupport(D3D11_FEATURE_FORMAT_SUPPORT2, &FormatSupport2, sizeof(FormatSupport2));
		if (SUCCEEDED(Support2HR))
		{
			Result.FormatSupport2 = (D3D11_FORMAT_SUPPORT2)FormatSupport2.OutFormatSupport2;
		}
	}

	return Result;
}

void FD3D11DynamicRHI::SetupAfterDeviceCreation()
{
	for (uint32 FormatIndex = PF_Unknown; FormatIndex < PF_MAX; FormatIndex++)
	{
		FPixelFormatInfo& PixelFormatInfo = GPixelFormats[FormatIndex];
		const DXGI_FORMAT PlatformFormat = static_cast<DXGI_FORMAT>(PixelFormatInfo.PlatformFormat);
		const DXGI_FORMAT UAVFormat = UE::DXGIUtilities::FindUnorderedAccessFormat(PlatformFormat);

		EPixelFormatCapabilities Capabilities = EPixelFormatCapabilities::None;

		if (PlatformFormat != DXGI_FORMAT_UNKNOWN)
		{
			const FFormatSupport FormatSupport    = GetFormatSupport(Direct3DDevice, PlatformFormat);
			const FFormatSupport SRVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindShaderResourceFormat(PlatformFormat, false));
			const FFormatSupport UAVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindUnorderedAccessFormat(PlatformFormat));
			const FFormatSupport RTVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindShaderResourceFormat(PlatformFormat, false));
			const FFormatSupport DSVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindDepthStencilFormat(PlatformFormat));

			auto ConvertCap1 = [&Capabilities](const FFormatSupport& InSupport, EPixelFormatCapabilities UnrealCap, D3D11_FORMAT_SUPPORT InFlag)
			{
				if (EnumHasAllFlags(InSupport.FormatSupport, InFlag))
				{
					EnumAddFlags(Capabilities, UnrealCap);
				}
			};
			auto ConvertCap2 = [&Capabilities](const FFormatSupport& InSupport, EPixelFormatCapabilities UnrealCap, D3D11_FORMAT_SUPPORT2 InFlag)
			{
				if (EnumHasAllFlags(InSupport.FormatSupport2, InFlag))
				{
					EnumAddFlags(Capabilities, UnrealCap);
				}
			};

            ConvertCap1(FormatSupport, EPixelFormatCapabilities::Texture1D,               D3D11_FORMAT_SUPPORT_TEXTURE1D);
            ConvertCap1(FormatSupport, EPixelFormatCapabilities::Texture2D,               D3D11_FORMAT_SUPPORT_TEXTURE2D);
            ConvertCap1(FormatSupport, EPixelFormatCapabilities::Texture3D,               D3D11_FORMAT_SUPPORT_TEXTURE3D);
            ConvertCap1(FormatSupport, EPixelFormatCapabilities::TextureCube,             D3D11_FORMAT_SUPPORT_TEXTURECUBE);
            ConvertCap1(FormatSupport, EPixelFormatCapabilities::Buffer,                  D3D11_FORMAT_SUPPORT_BUFFER);
            ConvertCap1(FormatSupport, EPixelFormatCapabilities::VertexBuffer,            D3D11_FORMAT_SUPPORT_IA_VERTEX_BUFFER);
            ConvertCap1(FormatSupport, EPixelFormatCapabilities::IndexBuffer,             D3D11_FORMAT_SUPPORT_IA_INDEX_BUFFER);

            if (EnumHasAnyFlags(Capabilities, EPixelFormatCapabilities::AnyTexture))
            {
                ConvertCap1(FormatSupport, EPixelFormatCapabilities::RenderTarget,        D3D11_FORMAT_SUPPORT_RENDER_TARGET);
                ConvertCap1(FormatSupport, EPixelFormatCapabilities::DepthStencil,        D3D11_FORMAT_SUPPORT_DEPTH_STENCIL);
                ConvertCap1(FormatSupport, EPixelFormatCapabilities::TextureMipmaps,      D3D11_FORMAT_SUPPORT_MIP);
                ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::TextureLoad,      D3D11_FORMAT_SUPPORT_SHADER_LOAD);
                ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::TextureSample | EPixelFormatCapabilities::TextureFilterable,    D3D11_FORMAT_SUPPORT_SHADER_SAMPLE);
                ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::TextureGather,    D3D11_FORMAT_SUPPORT_SHADER_GATHER);
                ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TextureAtomics,   D3D11_FORMAT_SUPPORT2_UAV_ATOMIC_EXCHANGE);
                ConvertCap1(RTVFormatSupport, EPixelFormatCapabilities::TextureBlendable, D3D11_FORMAT_SUPPORT_BLENDABLE);
                ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TextureStore,     D3D11_FORMAT_SUPPORT2_UAV_TYPED_STORE);
            }

            if (EnumHasAnyFlags(Capabilities, EPixelFormatCapabilities::Buffer))
            {
                ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::BufferLoad,       D3D11_FORMAT_SUPPORT_BUFFER);
                ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::BufferStore,      D3D11_FORMAT_SUPPORT2_UAV_TYPED_STORE);
                ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::BufferAtomics,    D3D11_FORMAT_SUPPORT2_UAV_ATOMIC_EXCHANGE);
            }

			ConvertCap1(UAVFormatSupport, EPixelFormatCapabilities::UAV,                  D3D11_FORMAT_SUPPORT_TYPED_UNORDERED_ACCESS_VIEW);
			ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TypedUAVLoad,         D3D11_FORMAT_SUPPORT2_UAV_TYPED_LOAD);
			ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TypedUAVStore,        D3D11_FORMAT_SUPPORT2_UAV_TYPED_STORE);
		}

		PixelFormatInfo.Capabilities = Capabilities;
	}

	// without that the first RHIClear would get a scissor rect of (0,0)-(0,0) which means we get a draw call clear 
	RHISetScissorRect(false, 0, 0, 0, 0);

	UpdateMSAASettings();

	if (GRHISupportsAsyncTextureCreation)
	{
		UE_LOGF(LogD3D11RHI, Log, "Async texture creation enabled");
	}
	else
	{
		UE_LOGF(LogD3D11RHI, Log, "Async texture creation disabled: %ls",
			D3D11RHI_ShouldAllowAsyncResourceCreation() ? TEXT("no driver support") : TEXT("disabled by user"));
	}
	{
		D3D11_FEATURE_DATA_D3D11_OPTIONS Data;
		if (const HRESULT Result = Direct3DDevice->CheckFeatureSupport(D3D11_FEATURE_D3D11_OPTIONS, &Data, sizeof(Data)); SUCCEEDED(Result))
		{
			GRHISupportsMapWriteNoOverwrite = Data.MapNoOverwriteOnDynamicBufferSRV;
			if (GRHISupportsMapWriteNoOverwrite)
			{
				UE_LOGF(LogD3D11RHI, Log, "D3D11_MAP_WRITE_NO_OVERWRITE for dynamic buffer SRVs is supported");
			}
		}
	}

	{
#if 0
		D3D11_FEATURE_DATA_D3D11_OPTIONS2 Data;
		HRESULT Result = Direct3DDevice->CheckFeatureSupport(D3D11_FEATURE_D3D11_OPTIONS2, &Data, sizeof(Data));
		GRHISupportsStencilRefFromPixelShader = SUCCEEDED(Result) && Data.PSSpecifiedStencilRefSupported;
		if (GRHISupportsStencilRefFromPixelShader)
		{
			UE_LOGF(LogD3D11RHI, Log, "Stencil ref from pixel shader is supported");
		}
#else
		GRHISupportsStencilRefFromPixelShader = false; // this boolean is later used to choose a code path that requires DXIL shaders. Cannot set to true without fixing that first.
#endif
	}

	{
		D3D11_FEATURE_DATA_D3D11_OPTIONS3 Data;
		HRESULT Result = Direct3DDevice->CheckFeatureSupport(D3D11_FEATURE_D3D11_OPTIONS3, &Data, sizeof(Data));
		GRHISupportsArrayIndexFromAnyShader = SUCCEEDED(Result) && Data.VPAndRTArrayIndexFromAnyShaderFeedingRasterizer;
		if (GRHISupportsArrayIndexFromAnyShader)
		{
			UE_LOGF(LogD3D11RHI, Log, "Array index from any shader is supported");
		}
	}

	TimestampCalibration = CalibrateTimers();

	// Register the single graphics GPU queue we have access to in D3D11.
	UE::RHI::GPUProfiler::FQueue Queue(UE::RHI::GPUProfiler::FQueue::EType::Graphics, 0, 0);
	UE::RHI::GPUProfiler::InitializeQueues(MakeConstArrayView(&Queue, 1));

	//
	// Since we can't tell when the GPU is actually executing engine work in D3D11,
	// just mark the GPU as always busy in the frame. We also push begin/end work
	// markers in RHIEndFrame either side of the frame boundary.
	//
	{
		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(FPlatformTime::Cycles64());
		InsertProfilerTimestamp(&Event.GPUTimestampTOP, false);
	}
}

void FD3D11DynamicRHI::UpdateMSAASettings()
{	
	check(DX_MAX_MSAA_COUNT == 8);

	// quality levels are only needed for CSAA which we cannot use with custom resolves

	// 0xffffffff means not available
	AvailableMSAAQualities[0] = 0xffffffff;
	AvailableMSAAQualities[1] = 0xffffffff;
	AvailableMSAAQualities[2] = 0;
	AvailableMSAAQualities[3] = 0xffffffff;
	AvailableMSAAQualities[4] = 0;
	AvailableMSAAQualities[5] = 0xffffffff;
	AvailableMSAAQualities[6] = 0xffffffff;
	AvailableMSAAQualities[7] = 0xffffffff;
	AvailableMSAAQualities[8] = 0;
}

#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
static int32 ReportDiedDuringDeviceShutdown(LPEXCEPTION_POINTERS ExceptionInfo)
{
	UE_LOGF(LogD3D11RHI, Error, "Crashed freeing up the D3D11 device.");
	if (GDynamicRHI)
	{
		GDynamicRHI->FlushPendingLogs();
	}

	return EXCEPTION_EXECUTE_HANDLER;
}
#endif

void FD3D11DynamicRHI::CleanupD3DDevice()
{
	UE_LOGF(LogD3D11RHI, Log, "CleanupD3DDevice");

	if(GIsRHIInitialized)
	{
		check(Direct3DDevice);
		check(Direct3DDeviceIMContext);
	#if WITH_DX_PERF
		check(Direct3DDeviceIMAnnotation);
	#endif

		// Reset the RHI initialized flag.
		GIsRHIInitialized = false;

		check(!GIsCriticalError);

		CurrentComputeShader = nullptr;

		// Ask all initialized FRenderResources to release their RHI resources.
		FRenderResource::ReleaseRHIForAllResources();

		extern void EmptyD3DSamplerStateCache();
		EmptyD3DSamplerStateCache();

		// Release references to bound uniform buffers.
		for (int32 Frequency = 0; Frequency < SF_NumStandardFrequencies; ++Frequency)
		{
			for (int32 BindIndex = 0; BindIndex < MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE; ++BindIndex)
			{
				BoundUniformBuffers[Frequency][BindIndex] = nullptr;
			}
		}

		// Release the device and its IC
		StateCache.SetContext(nullptr);

		// Flush all pending deletes before destroying the device.
		FRHICommandListExecutor::GetImmediateCommandList().ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		ReleasePooledUniformBuffers();

#if WITH_AMD_AGS
		// Clean up the AMD extensions and shut down the AMD AGS utility library
		if (AmdAgsContext != NULL)
		{
			check(UE::RHICore::AllowVendorDevice());

			// AGS is holding an extra reference to the immediate context. Release it before calling DestroyDevice.
			Direct3DDeviceIMContext->Release();
			agsDriverExtensionsDX11_DestroyDevice(AmdAgsContext, Direct3DDevice, NULL, Direct3DDeviceIMContext, NULL);
			agsDeInitialize(AmdAgsContext);
			GRHIDeviceIsAMDPreGCNArchitecture = false;
			AmdAgsContext = NULL;
		}
#endif // WITH_AMD_AGS

#if INTEL_EXTENSIONS
		if (IsRHIDeviceIntel() && UE::RHICore::AllowVendorDevice())
		{
			StopIntelExtensions();
		}
#endif // INTEL_EXTENSIONS

		// When running with D3D debug, clear state and flush the device to get rid of spurious live objects in D3D11's report.
		if (GRHIGlobals.IsDebugLayerEnabled)
		{
			Direct3DDeviceIMContext->ClearState();
			Direct3DDeviceIMContext->Flush();

			// Perform a detailed live object report (with resource types)
			ID3D11Debug* D3D11Debug;
			Direct3DDevice->QueryInterface(__uuidof(ID3D11Debug), (void**)(&D3D11Debug));
			if (D3D11Debug)
			{
				D3D11Debug->ReportLiveDeviceObjects(D3D11_RLDO_DETAIL);
			}
		}

		if (ExceptionHandlerHandle != INVALID_HANDLE_VALUE)
		{
			RemoveVectoredExceptionHandler(ExceptionHandlerHandle);
		}

		// ORION - avoid shutdown crash that is currently present in UE
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		{
		#if WITH_DX_PERF
			//UE-18906: Workaround to trap crash in NV driver
			__try
			{
				// Perform a detailed live object report (with resource types)
				Direct3DDeviceIMAnnotation = nullptr;
			}
			__except (ReportDiedDuringDeviceShutdown(GetExceptionInformation()))
			{
				FPlatformMisc::MemoryBarrier();
			}
		#endif

			//UE-18906: Workaround to trap crash in NV driver
			__try
			{
				// Perform a detailed live object report (with resource types)
				Direct3DDeviceIMContext = nullptr;
			}
			__except (ReportDiedDuringDeviceShutdown(GetExceptionInformation()))
			{
				FPlatformMisc::MemoryBarrier();
			}

			//UE-18906: Workaround to trap crash in NV driver
			__try
			{
				// Perform a detailed live object report (with resource types)
				Direct3DDevice = nullptr;
			}
			__except (ReportDiedDuringDeviceShutdown(GetExceptionInformation()))
			{
				FPlatformMisc::MemoryBarrier();
			}
		}
#else
		{
		#if WITH_DX_PERF
			Direct3DDeviceIMAnnotation = nullptr;
		#endif
			Direct3DDeviceIMContext = nullptr;
			Direct3DDevice = nullptr;
		}
#endif
	}
}

void* FD3D11DynamicRHI::RHIGetNativeDevice()
{
	return (void*)Direct3DDevice.GetReference();
}

void* FD3D11DynamicRHI::RHIGetNativeCommandBuffer()
{
	return (void*)Direct3DDeviceIMContext.GetReference();
}

static bool CanFormatBeDisplayed(const FD3D11DynamicRHI* InD3DRHI, EPixelFormat InPixelFormat)
{
	const DXGI_FORMAT DxgiFormat = FD3D11Viewport::GetRenderTargetFormat(InPixelFormat);

	UINT FormatSupport = 0;
	const HRESULT FormatSupportResult = InD3DRHI->GetDevice()->CheckFormatSupport(DxgiFormat, &FormatSupport);
	if (FAILED(FormatSupportResult))
	{
		const TCHAR* D3DFormatString = UE::DXGIUtilities::GetFormatString(DxgiFormat);
		UE_LOGF(LogD3D11RHI, Warning, "CheckFormatSupport(%ls) failed: 0x%08x", D3DFormatString, FormatSupportResult);
		return false;
	}

	const UINT RequiredFlags = D3D11_FORMAT_SUPPORT_DISPLAY | D3D11_FORMAT_SUPPORT_RENDER_TARGET | D3D11_FORMAT_SUPPORT_SHADER_SAMPLE;
	return (FormatSupport & RequiredFlags) == RequiredFlags;
}

EPixelFormat FD3D11DynamicRHI::GetDisplayFormat(EPixelFormat InPixelFormat) const
{
	EPixelFormat CandidateFormat = InPixelFormat;

	// Small list of supported formats and what they should fall back to. This could be expanded more in the future.
	struct SDisplayFormat { EPixelFormat PixelFormat; EPixelFormat FallbackPixelFormat; }
	static const DisplayFormats[]
	{
		{ PF_FloatRGBA, PF_A2B10G10R10 },
		{ PF_A2B10G10R10, PF_B8G8R8A8 },
	};

	for (const SDisplayFormat& DisplayFormat : DisplayFormats)
	{
		if (CandidateFormat == DisplayFormat.PixelFormat && !CanFormatBeDisplayed(this, CandidateFormat))
		{
			CandidateFormat = DisplayFormat.FallbackPixelFormat;
		}
	}

	check(CanFormatBeDisplayed(this, CandidateFormat));
	return CandidateFormat;
}

