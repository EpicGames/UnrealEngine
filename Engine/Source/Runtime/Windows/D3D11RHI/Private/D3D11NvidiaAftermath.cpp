// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D11NvidiaAftermath.h"

#if NV_AFTERMATH

	#include "Windows/AllowWindowsPlatformTypes.h"
	THIRD_PARTY_INCLUDES_START

		#include "GFSDK_Aftermath.h"
		#include "GFSDK_Aftermath_GpuCrashdump.h"
		#include "GFSDK_Aftermath_GpuCrashDumpDecoding.h"

	THIRD_PARTY_INCLUDES_END
	#include "Windows/HideWindowsPlatformTypes.h"

namespace UE::RHICore::Nvidia::Aftermath::D3D11
{
	FCommandList InitializeDevice(ID3D11Device* Device, ID3D11DeviceContext* DeviceContext)
	{
		bool bInitialized = UE::RHICore::Nvidia::Aftermath::InitializeDevice([&](uint32 Flags)
		{
			return GFSDK_Aftermath_DX11_Initialize(GFSDK_Aftermath_Version_API, Flags, Device);
		});

		FCommandList Handle{};
		if (bInitialized)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_DX11_CreateContextHandle(DeviceContext, &Handle);
			if (!ensureMsgf(Result == GFSDK_Aftermath_Result_Success, TEXT("GFSDK_Aftermath_DX11_CreateContextHandle failed: 0x%08x"), Result))
			{
				Handle = {};
			}
		}

		return Handle;
	}

	void UnregisterCommandList(FCommandList CommandList)
	{
		if (IsEnabled() && CommandList)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_ReleaseContextHandle(CommandList);
			ensureMsgf(Result == GFSDK_Aftermath_Result_Success, TEXT("GFSDK_Aftermath_ReleaseContextHandle failed: 0x%08x"), Result);
		}
	}

#if WITH_RHI_BREADCRUMBS
	void BeginBreadcrumb(FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb)
	{
		FMarker Marker(Breadcrumb);
		if (Marker)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_SetEventMarker(CommandList, Marker.GetPtr(), Marker.GetSize());
			ensureMsgf(Result == GFSDK_Aftermath_Result_Success, TEXT("GFSDK_Aftermath_SetEventMarker failed in BeginBreadcrumb: 0x%08x"), Result);
		}
	}

	void EndBreadcrumb(FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb)
	{
		FMarker Marker(Breadcrumb->GetParent());
		if (Marker)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_SetEventMarker(CommandList, Marker.GetPtr(), Marker.GetSize());
			ensureMsgf(Result == GFSDK_Aftermath_Result_Success, TEXT("GFSDK_Aftermath_SetEventMarker failed in EndBreadcrumb: 0x%08x"), Result);
		}
	}
#endif
}

#endif // NV_AFTERMATH
