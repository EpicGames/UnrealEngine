// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHICoreNvidiaAftermath.h"

#if NV_AFTERMATH

	#include "Windows/D3D11ThirdParty.h"

	struct GFSDK_Aftermath_ContextHandle__;
	struct GFSDK_Aftermath_ResourceHandle__;

	namespace UE::RHICore::Nvidia::Aftermath::D3D11
	{
		typedef struct GFSDK_Aftermath_ContextHandle__* FCommandList;
		typedef struct GFSDK_Aftermath_ResourceHandle__* FResource;

		FCommandList InitializeDevice(ID3D11Device* Device, ID3D11DeviceContext* DeviceContext);
		void UnregisterCommandList(FCommandList CommandList);

	#if WITH_RHI_BREADCRUMBS
		void BeginBreadcrumb(FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb);
		void EndBreadcrumb  (FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb);
	#endif
	}

#endif // NV_AFTERMATH
