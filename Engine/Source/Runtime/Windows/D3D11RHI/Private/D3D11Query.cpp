// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Query.cpp: D3D query RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "RenderCore.h"

float GD3D11AbsoluteTimeQueryTimeoutValue = 30.0f;
static FAutoConsoleVariableRef CVarD3D11AbsoluteTimeQueryTimeoutValue(
	TEXT("r.D3D11.AbsoluteTimeQueryTimeoutValue"),
	GD3D11AbsoluteTimeQueryTimeoutValue,
	TEXT("Set the timeout value, in seconds, to wait for a D3D11 absolute time query."),
	ECVF_Default
);

float GD3D11QueryTimeoutValue = 5.0f;
static FAutoConsoleVariableRef CVarD3D11QueryTimeoutValue(
	TEXT("r.D3D11.QueryTimeoutValue"),
	GD3D11QueryTimeoutValue,
	TEXT("Set the timeout value, in seconds, to wait for a D3D11 query. This value does not apply to absolute time queries (which are controlled by r.D3D11.AbsoluteTimeQueryTimeoutValue)."),
	ECVF_Default
);

FD3D11RenderQuery::FD3D11RenderQuery(EType Type)
	: Type(Type)
{
	D3D11_QUERY_DESC Desc {};

	switch (Type)
	{
	case EType::Occlusion: Desc.Query = D3D11_QUERY_OCCLUSION; break;
	case EType::Timestamp: Desc.Query = D3D11_QUERY_TIMESTAMP; break;

	case EType::Profiler:
	case EType::ProfilerEndFrame:
		Desc.Query = D3D11_QUERY_TIMESTAMP;
		break;

	default:
		checkNoEntry();
		return;
	}

	ID3D11Device* Device = FD3D11DynamicRHI::Get().GetDevice();
	VERIFYD3D11RESULT_EX(Device->CreateQuery(&Desc, Resource.GetInitReference()), Device);
}

FD3D11RenderQuery::~FD3D11RenderQuery()
{
	Unlink();
}

void FD3D11RenderQuery::Begin(ID3D11DeviceContext* Context)
{
	check(Type == EType::Occlusion);
	Context->Begin(Resource);
}

void FD3D11RenderQuery::End(ID3D11DeviceContext* Context, uint64* NewTarget)
{
	BOPCounter++;

	Context->End(Resource);
	Target = NewTarget;

	Link();
}

FRenderQueryRHIRef FD3D11DynamicRHI::RHICreateRenderQuery(ERenderQueryType QueryType)
{
	switch (QueryType)
	{
	case RQT_AbsoluteTime: return new FD3D11RenderQuery_RHI(FD3D11RenderQuery::EType::Timestamp);
	case RQT_Occlusion   : return new FD3D11RenderQuery_RHI(FD3D11RenderQuery::EType::Occlusion);

	default:
		checkNoEntry();
		return nullptr;
	}
}

void FD3D11DynamicRHI::RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery)
{
	FD3D11RenderQuery* Query = ResourceCast(RenderQuery);
	Query->TOPCounter++;

	FDynamicRHI::RHIEndRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
}

void FD3D11DynamicRHI::RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery)
{
	ResourceCast(RenderQuery)->Begin(Direct3DDeviceIMContext);
}

void FD3D11DynamicRHI::RHIEndRenderQuery(FRHIRenderQuery* RenderQuery)
{
	FD3D11RenderQuery_RHI* Query = ResourceCast(RenderQuery);
	Query->End(Direct3DDeviceIMContext, &Query->Result);
}

bool FD3D11DynamicRHI::RHIGetRenderQueryResult(FRHIRenderQuery* QueryRHI, uint64& OutResult, bool bWait, uint32 GPUIndex)
{
	check(IsInRenderingThread());
	FD3D11RenderQuery_RHI* Query = ResourceCast(QueryRHI);

	bool bRHIThreadFlushed = false;

Retry:
	if (Query->TOPCounter == Query->LastCachedBOPCounter.load(std::memory_order_acquire))
	{
		// Early return for queries we already have the result for.
		check(!Query->IsLinked());
		OutResult = Query->Result;
		return true;
	}

	if (FRHICommandListExecutor::AreRHITasksActive())
	{
		if (!bWait)
		{
			//
			// The RHI thread is still processing work, the query has not yet completed, and we don't want to wait for the query result.
			// Return. The RHI thread will poll for results later.
			//
			OutResult = 0;
			return false;
		}
		else
		{
			//
			// The RHI thread is active, the query has not yet completed, and we want to wait for results.
			// 
			// Flushing the RHI thread will ensure a query poll operation has happened before the render thread resumes, which might successfully cache the results.
			// It will also make it safe for us to use the immediate device context in case the query still wasn't done when the RHI thread last polled for results.
			//
			FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			bRHIThreadFlushed = true;

			// Now the RHI thread is idle, retry grabbing the query results.
			goto Retry;
		}
	}

	//
	// From this point, the RHI thread is idle (although possibly not flushed). It is safe to use the immediate device context.
	// The query is unresolved. Either the GPU isn't done, or the commands to signal the query were never submitted (still recorded in the immediate command list).
	//

	if (Query->TOPCounter != Query->BOPCounter && !bRHIThreadFlushed)
	{
		// When TOPCounter != BOPCounter, there's an End() operation that was recorded at the TOP, but has not yet been submitted for translation by the RHI thread.
		// Flush the immediate command list to push this command into the RHI pipeline.
		FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		bRHIThreadFlushed = true;

		// Now the RHI thread is flushed, retry grabbing the query results.
		goto Retry;
	}

	checkf(Query->TOPCounter == Query->BOPCounter, TEXT("Attempting to get data from an RHI render query which was never issued."));
	if (!Query->CacheResult(*this, bWait))
	{
		OutResult = 0;
		return false;
	}

	check(!Query->IsLinked());
	OutResult = Query->Result;

	return true;
}

bool FD3D11RenderQuery::CacheResult(FD3D11DynamicRHI& RHI, bool bWait)
{
	if (BOPCounter == LastCachedBOPCounter.load(std::memory_order_relaxed))
	{
		// Value has been cached and no newer query operation has started.
		check(!IsLinked());
		return true;
	}

	check(Target);

	// Attempt to read the result from the GPU.
	uint64 Temp;
	if (!RHI.GetQueryData(Resource, &Temp, sizeof(Temp), Type == EType::Timestamp, /*bWait = */ bWait))
	{
		return false;
	}

	// Data retrieved.
	// Adjust timer queries to engine-clock ticks.
	switch (Type)
	{
	case EType::Profiler:
	case EType::ProfilerEndFrame:
		{
			// Convert from GPU timestamp to CPU timestamp (relative to FPlatformTime::Cycles64())
			uint64 GPUDelta = Temp - RHI.TimestampCalibration->GPUTimestamp;
			uint64 CPUDelta = (GPUDelta * RHI.TimestampCalibration->CPUFrequency) / RHI.TimestampCalibration->GPUFrequency;

			Temp = CPUDelta + RHI.TimestampCalibration->CPUTimestamp;
		}
		break;

	case EType::Timestamp:
		{
			uint64 Div = FMath::Max(1llu, RHI.TimestampCalibration->GPUFrequency / (1000 * 1000));

			// convert from GPU specific timestamp to micro sec (1 / 1 000 000 s) which seems a reasonable resolution
			Temp = Temp / Div;
		}
		break;
	}

	*Target = Temp;
	Target = nullptr;

	Unlink();

	LastCachedBOPCounter.store(BOPCounter, std::memory_order_release);

	// Don't return ProfilerEndFrame queries to the pool yet.
	// References to these are held by historic GPU profiler frames, and are used to determine when it is safe to process the timestamp data.
	// They are returned to the pool manually in PollQueryResultsForEndFrame(...).
	if (Type == EType::Profiler)
	{
		// Return the query to the pool
		RHI.Profiler.TimestampPool.Push(this);
	}

	return true;
}

void FD3D11RenderQuery::Link()
{
	// The renderer might re-use a query without reading its results back first.
	// Ensure this query is unlinked, so it can be re-linked at the end of the list.
	Unlink();

	auto& List = FD3D11DynamicRHI::Get().ActiveQueries;

	if (!List.First)
	{
		check(!List.Last);
		check(Next == nullptr);

		List.First = this;
		Prev = &List.First;
	}
	else
	{
		check(List.Last);
		check(List.Last->Next == nullptr);

		List.Last->Next = this;
		Prev = &List.Last->Next;
	}

	List.Last = this;
}

void FD3D11RenderQuery::Unlink()
{
	if (!IsLinked())
		return;

	auto& List = FD3D11DynamicRHI::Get().ActiveQueries;

	if (List.Last == this)
	{
		// This is the last node in the list, so the "List.Last" pointer needs fixing up.
		if (Prev == &List.First)
		{
			// This is also the first node in the list, meaning there's only 1 node total.
			// Just clear the "List.Last" pointer.
			List.Last = nullptr;
		}
		else
		{
			//
			// There's at least one real node before us.
			// 
			// "Prev" points to the "Next" member field of the previous node.
			// Subtract the "Next" field offset to get the actual previous node address.
			//
			List.Last = reinterpret_cast<FD3D11RenderQuery*>(reinterpret_cast<uintptr_t>(Prev) - UFIELD_OFFSET(FD3D11RenderQuery, Next));
		}
	}

	if (Next) { Next->Prev = Prev; }
	if (Prev) { *Prev = Next; }

	Next = nullptr;
	Prev = nullptr;
}

void FD3D11DynamicRHI::PollQueryResults()
{
	while (ActiveQueries.First)
	{
		if (!ActiveQueries.First->CacheResult(*this, /*bWait = */ false))
			break;
	}
}

bool FD3D11DynamicRHI::PollQueryResultsForEndFrame(FD3D11RenderQuery* Query)
{
	check(Query->Type == FD3D11RenderQuery::EType::ProfilerEndFrame);

	// Poll as many query results as possible.
	PollQueryResults();

	if (Query->IsLinked())
	{
		// The end-frame query is still linked. The results have not arrived yet.
		return false;
	}
	else
	{
		// An unlinked end-frame query indicates the results have arrived.
		// Return the query to the pool for reuse.
		Profiler.TimestampPoolEndFrame.Push(Query);

		return true;
	}
}

bool FD3D11DynamicRHI::GetQueryData(ID3D11Query* Query, void* Data, SIZE_T DataSize, bool bTimestamp, bool bWait)
{
	// Request the data from the query.
	HRESULT Result = Direct3DDeviceIMContext->GetData(Query, Data, DataSize, 0);

	// Isn't the query finished yet, and can we wait for it?
	if (Result == S_FALSE && bWait)
	{
		SCOPE_CYCLE_COUNTER( STAT_RenderQueryResultTime );
		FRenderThreadIdleScope IdleScope(ERenderThreadIdleTypes::WaitingForGPUQuery);

		double StartTime = FPlatformTime::Seconds();
		double TimeoutWarningLimit = 5.0;
		// timer queries are used for Benchmarks which can stall a bit more
		double TimeoutValue = bTimestamp ? GD3D11AbsoluteTimeQueryTimeoutValue : GD3D11QueryTimeoutValue;

		do
		{
			Result = Direct3DDeviceIMContext->GetData(Query, Data, DataSize, 0);

			if (Result == S_OK)
			{
				return true;
			}

			float DeltaTime = FPlatformTime::Seconds() - StartTime;
			if (DeltaTime > TimeoutWarningLimit)
			{
				HRESULT DeviceRemovedReason = Direct3DDevice->GetDeviceRemovedReason();
				TimeoutWarningLimit += 5.0;
				UE_LOGF(LogD3D11RHI, Log, "GetQueryData is taking a very long time (%.1f s) (%08x)", DeltaTime, (uint32)DeviceRemovedReason);
			}

			if (DeltaTime > TimeoutValue)
			{
				UE_LOGF(LogD3D11RHI, Log, "Timed out while waiting for GPU query. (Timeout %.1f s) (ErrorCode %08x)", TimeoutValue, (uint32)Result);
				VERIFYD3D11RESULT_EX(Result, Direct3DDevice);
				return false;
			}
		} while (Result == S_FALSE);
	}

	if (Result == S_OK)
	{
		return true;
	}
	else if (Result == S_FALSE && !bWait)
	{
		// Return failure if the query isn't complete, and waiting wasn't requested.
		return false;
	}
	else
	{
		VERIFYD3D11RESULT_EX(Result, Direct3DDevice);
		return false;
	}
}

TOptional<FD3D11DynamicRHI::FTimestampCalibration> FD3D11DynamicRHI::CalibrateTimers()
{
	// Attempt to generate a timestamp on GPU and CPU as closely to each other as possible.
	// This works by first flushing any pending GPU work, then writing a GPU timestamp and waiting for GPU to finish.
	// CPU timestamp is continuously captured while we are waiting on GPU.

	HRESULT D3DResult = E_FAIL;

	TRefCountPtr<ID3D11Query> DisjointQuery;
	{
		D3D11_QUERY_DESC QueryDesc;
		QueryDesc.Query = D3D11_QUERY_TIMESTAMP_DISJOINT;
		QueryDesc.MiscFlags = 0;
		D3DResult = Direct3DDevice->CreateQuery(&QueryDesc, DisjointQuery.GetInitReference());

		if (D3DResult != S_OK)
			return {};
	}

	TRefCountPtr<ID3D11Query> TimestampQuery;
	{
		D3D11_QUERY_DESC QueryDesc;
		QueryDesc.Query = D3D11_QUERY_TIMESTAMP;
		QueryDesc.MiscFlags = 0;
		D3DResult = Direct3DDevice->CreateQuery(&QueryDesc, TimestampQuery.GetInitReference());

		if (D3DResult != S_OK)
			return {};
	}

	TRefCountPtr<ID3D11Query> PendingWorkDoneQuery;
	TRefCountPtr<ID3D11Query> TimestampDoneQuery;
	{
		D3D11_QUERY_DESC QueryDesc;
		QueryDesc.Query = D3D11_QUERY_EVENT;
		QueryDesc.MiscFlags = 0;

		D3DResult = Direct3DDevice->CreateQuery(&QueryDesc, PendingWorkDoneQuery.GetInitReference());
		if (D3DResult != S_OK)
			return {};

		D3DResult = Direct3DDevice->CreateQuery(&QueryDesc, TimestampDoneQuery.GetInitReference());
		if (D3DResult != S_OK)
			return {};
	}

	// Flush any currently pending GPU work and wait for it to finish
	Direct3DDeviceIMContext->End(PendingWorkDoneQuery);
	Direct3DDeviceIMContext->Flush();

	for (;;)
	{
		BOOL EventComplete = false;
		Direct3DDeviceIMContext->GetData(PendingWorkDoneQuery, &EventComplete, sizeof(EventComplete), 0);

		if (EventComplete)
			break;

		FPlatformProcess::Sleep(0.001f);
	}

	const uint32 MaxCalibrationAttempts = 10;
	for (uint32 CalibrationAttempt = 0; CalibrationAttempt < MaxCalibrationAttempts; ++CalibrationAttempt)
	{
		Direct3DDeviceIMContext->Begin(DisjointQuery);
		Direct3DDeviceIMContext->End(TimestampQuery);
		Direct3DDeviceIMContext->End(DisjointQuery);
		Direct3DDeviceIMContext->End(TimestampDoneQuery);

		Direct3DDeviceIMContext->Flush();

		uint64 CPUTimestamp = 0;
		uint64 GPUTimestamp = 0;

		// Busy-wait for GPU to finish and capture CPU timestamp approximately when GPU work is done
		for (;;)
		{
			BOOL EventComplete = false;

			CPUTimestamp = FPlatformTime::Cycles64();
			Direct3DDeviceIMContext->GetData(TimestampDoneQuery, &EventComplete, sizeof(EventComplete), 0);

			if (EventComplete)
				break;
		}

		D3D11_QUERY_DATA_TIMESTAMP_DISJOINT DisjointQueryData = {};
		D3DResult = Direct3DDeviceIMContext->GetData(DisjointQuery, &DisjointQueryData, sizeof(DisjointQueryData), 0);

		// If timestamp was unreliable, try again
		if (D3DResult != S_OK || DisjointQueryData.Disjoint)
		{
			continue;
		}

		D3DResult = Direct3DDeviceIMContext->GetData(TimestampQuery, &GPUTimestamp, sizeof(GPUTimestamp), 0);

		// If we managed to get valid timestamps, save both of them (CPU & GPU) and return
		if (D3DResult == S_OK && GPUTimestamp)
		{
			return FD3D11DynamicRHI::FTimestampCalibration
			{
				.CPUTimestamp = CPUTimestamp,
				.CPUFrequency = uint64(1.0 / FPlatformTime::GetSecondsPerCycle64()),

				.GPUTimestamp = GPUTimestamp,
				.GPUFrequency = DisjointQueryData.Frequency
			};
		}
	}

	return {};
}
