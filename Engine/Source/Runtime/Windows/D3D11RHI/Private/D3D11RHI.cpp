// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11RHI.cpp: Unreal D3D RHI library implementation.
=============================================================================*/

#include "D3D11RHI.h"
#include "D3D11RHIPrivate.h"
#include "RHIStaticStates.h"
#include "StaticBoundShaderState.h"
#include "Engine/GameViewportClient.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "RHICoreStats.h"

#include "OneColorShader.h"

DEFINE_LOG_CATEGORY(LogD3D11RHI);

extern void UniformBufferBeginFrame();

// http://developer.download.nvidia.com/devzone/devcenter/gamegraphics/files/OptimusRenderingPolicies.pdf
// The following line is to favor the high performance NVIDIA GPU if there are multiple GPUs
// Has to be .exe module to be correctly detected.
// extern "C" { _declspec(dllexport) uint32 NvOptimusEnablement = 0x00000001; }

void FD3D11DynamicRHI::RHIEndFrame(const FRHIEndFrameArgs& Args)
{
	// End Frame
	{
		// End GPU work.
		// The EndWork query is always the last timestamp query of the frame. Keep a reference to it, so we can poll for frame completion below.
		auto& EndWork = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>();
		Profiler.Current.EndFrameQuery.Ptr = InsertProfilerTimestamp(&EndWork.GPUTimestampBOP, true);

		uint64 Timestamp = FPlatformTime::Cycles64();

		// Insert frame boundary
		EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FFrameBoundary>(ERHIPipeline::Graphics, Args, Timestamp);

		Profiler.Pending.Enqueue(MakeUnique<FProfiler::FFrame>(MoveTemp(Profiler.Current)));

		// Attempt to process historic results
		while (TUniquePtr<FProfiler::FFrame>* PreviousFramePtr = Profiler.Pending.Peek())
		{
			TUniquePtr<FProfiler::FFrame>& PreviousFrame = *PreviousFramePtr;
			if (!PollQueryResultsForEndFrame(PreviousFrame->EndFrameQuery.Ptr))
			{
				// Frame not yet finished on the GPU
				break;
			}

			// Previous frame has completed and the data is available. Publish the profiler events.
			UE::RHI::GPUProfiler::ProcessEvents(MakeArrayView(&PreviousFrame->EventStream, 1));

			Profiler.Pending.Pop();
		}

		// Start the next frame's GPU work
		auto& BeginWork = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(Timestamp);
		InsertProfilerTimestamp(&BeginWork.GPUTimestampTOP, false);
	}

	UpdateMemoryStats();
	CurrentComputeShader = nullptr;

	// Begin Frame
	UniformBufferBeginFrame();
}

template <int32 Frequency>
void ClearShaderResource(ID3D11DeviceContext* Direct3DDeviceIMContext, uint32 ResourceIndex)
{
	ID3D11ShaderResourceView* NullView = NULL;
	switch(Frequency)
	{
	case SF_Pixel:   Direct3DDeviceIMContext->PSSetShaderResources(ResourceIndex,1,&NullView); break;
	case SF_Compute: Direct3DDeviceIMContext->CSSetShaderResources(ResourceIndex,1,&NullView); break;
	case SF_Geometry:Direct3DDeviceIMContext->GSSetShaderResources(ResourceIndex,1,&NullView); break;
	case SF_Vertex:  Direct3DDeviceIMContext->VSSetShaderResources(ResourceIndex,1,&NullView); break;
	};
}

void FD3D11DynamicRHI::ClearState()
{
	StateCache.ClearState();

	FMemory::Memzero(CurrentResourcesBoundAsSRVs, sizeof(CurrentResourcesBoundAsSRVs));
	FMemory::Memzero(CurrentResourcesBoundAsVBs, sizeof(CurrentResourcesBoundAsVBs));
	CurrentResourceBoundAsIB = nullptr;
	for (int32 Frequency = 0; Frequency < SF_NumStandardFrequencies; Frequency++)
	{
		MaxBoundShaderResourcesIndex[Frequency] = INDEX_NONE;
	}
	MaxBoundVertexBufferIndex = INDEX_NONE;
}

void GetMipAndSliceInfoFromSRV(ID3D11ShaderResourceView* SRV, int32& MipLevel, int32& NumMips, int32& ArraySlice, int32& NumSlices)
{
	MipLevel = -1;
	NumMips = -1;
	ArraySlice = -1;
	NumSlices = -1;

	if (SRV)
	{
		D3D11_SHADER_RESOURCE_VIEW_DESC SRVDesc;
		SRV->GetDesc(&SRVDesc);
		switch (SRVDesc.ViewDimension)
		{			
			case D3D11_SRV_DIMENSION_TEXTURE1D:
				MipLevel	= SRVDesc.Texture1D.MostDetailedMip;
				NumMips		= SRVDesc.Texture1D.MipLevels;
				break;
			case D3D11_SRV_DIMENSION_TEXTURE1DARRAY:
				MipLevel	= SRVDesc.Texture1DArray.MostDetailedMip;
				NumMips		= SRVDesc.Texture1DArray.MipLevels;
				ArraySlice	= SRVDesc.Texture1DArray.FirstArraySlice;
				NumSlices	= SRVDesc.Texture1DArray.ArraySize;
				break;
			case D3D11_SRV_DIMENSION_TEXTURE2D:
				MipLevel	= SRVDesc.Texture2D.MostDetailedMip;
				NumMips		= SRVDesc.Texture2D.MipLevels;
				break;
			case D3D11_SRV_DIMENSION_TEXTURE2DARRAY:
				MipLevel	= SRVDesc.Texture2DArray.MostDetailedMip;
				NumMips		= SRVDesc.Texture2DArray.MipLevels;
				ArraySlice	= SRVDesc.Texture2DArray.FirstArraySlice;
				NumSlices	= SRVDesc.Texture2DArray.ArraySize;
				break;
			case D3D11_SRV_DIMENSION_TEXTURE2DMS:
				MipLevel	= 0;
				NumMips		= 1;
				break;
			case D3D11_SRV_DIMENSION_TEXTURE2DMSARRAY:
				MipLevel	= 0;
				NumMips		= 1;
				ArraySlice	= SRVDesc.Texture2DMSArray.FirstArraySlice;
				NumSlices	= SRVDesc.Texture2DMSArray.ArraySize;
				break;
			case D3D11_SRV_DIMENSION_TEXTURE3D:
				MipLevel	= SRVDesc.Texture3D.MostDetailedMip;
				NumMips		= SRVDesc.Texture3D.MipLevels;
				break;
			case D3D11_SRV_DIMENSION_TEXTURECUBE:
				MipLevel = SRVDesc.TextureCube.MostDetailedMip;
				NumMips		= SRVDesc.TextureCube.MipLevels;
				break;
			case D3D11_SRV_DIMENSION_TEXTURECUBEARRAY:
				MipLevel	= SRVDesc.TextureCubeArray.MostDetailedMip;
				NumMips		= SRVDesc.TextureCubeArray.MipLevels;
				ArraySlice	= SRVDesc.TextureCubeArray.First2DArrayFace;
				NumSlices	= SRVDesc.TextureCubeArray.NumCubes;
				break;
			case D3D11_SRV_DIMENSION_BUFFER:
			case D3D11_SRV_DIMENSION_BUFFEREX:
			default:
				break;
		}
	}
}

template <EShaderFrequency ShaderFrequency>
void FD3D11DynamicRHI::InternalSetShaderResourceView(FD3D11ViewableResource* Resource, ID3D11ShaderResourceView* SRV, int32 ResourceIndex)
{
	// Check either both are set, or both are null.
	check((Resource && SRV) || (!Resource && !SRV));

	//avoid state cache crash
	if (!((Resource && SRV) || (!Resource && !SRV)))
	{
		//UE_LOGF(LogRHI, Warning, "Bailing on InternalSetShaderResourceView on resource: %i, %ls", ResourceIndex, *SRVName.ToString());
		return;
	}

	FD3D11ViewableResource*& ResourceSlot = CurrentResourcesBoundAsSRVs[ShaderFrequency][ResourceIndex];
	int32& MaxResourceIndex = MaxBoundShaderResourcesIndex[ShaderFrequency];

	if (Resource)
	{
		// We are binding a new SRV.
		// Update the max resource index to the highest bound resource index.
		MaxResourceIndex = FMath::Max(MaxResourceIndex, ResourceIndex);
		ResourceSlot = Resource;
	}
	else if (ResourceSlot != nullptr)
	{
		// Unbind the resource from the slot.
		ResourceSlot = nullptr;

		// If this was the highest bound resource...
		if (MaxResourceIndex == ResourceIndex)
		{
			// Adjust the max resource index downwards until we
			// hit the next non-null slot, or we've run out of slots.
			do
			{
				MaxResourceIndex--;
			}
			while (MaxResourceIndex >= 0 && CurrentResourcesBoundAsSRVs[ShaderFrequency][MaxResourceIndex] == nullptr);
		} 
	}

	// Set the SRV we have been given (or null).
	StateCache.SetShaderResourceView<ShaderFrequency>(SRV, ResourceIndex);
}

template void FD3D11DynamicRHI::InternalSetShaderResourceView<SF_Vertex>  (FD3D11ViewableResource* Resource, ID3D11ShaderResourceView* SRV, int32 ResourceIndex);
template void FD3D11DynamicRHI::InternalSetShaderResourceView<SF_Pixel>   (FD3D11ViewableResource* Resource, ID3D11ShaderResourceView* SRV, int32 ResourceIndex);
template void FD3D11DynamicRHI::InternalSetShaderResourceView<SF_Geometry>(FD3D11ViewableResource* Resource, ID3D11ShaderResourceView* SRV, int32 ResourceIndex);
template void FD3D11DynamicRHI::InternalSetShaderResourceView<SF_Compute> (FD3D11ViewableResource* Resource, ID3D11ShaderResourceView* SRV, int32 ResourceIndex);

void FD3D11DynamicRHI::TrackResourceBoundAsVB(FD3D11ViewableResource* Resource, int32 StreamIndex)
{
	check(StreamIndex >= 0 && StreamIndex < D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT);
	if (Resource)
	{
		// We are binding a new VB.
		// Update the max resource index to the highest bound resource index.
		MaxBoundVertexBufferIndex = FMath::Max(MaxBoundVertexBufferIndex, StreamIndex);
		CurrentResourcesBoundAsVBs[StreamIndex] = Resource;
	}
	else if (CurrentResourcesBoundAsVBs[StreamIndex] != nullptr)
	{
		// Unbind the resource from the slot.
		CurrentResourcesBoundAsVBs[StreamIndex] = nullptr;

		// If this was the highest bound resource...
		if (MaxBoundVertexBufferIndex == StreamIndex)
		{
			// Adjust the max resource index downwards until we
			// hit the next non-null slot, or we've run out of slots.
			do
			{
				MaxBoundVertexBufferIndex--;
			} while (MaxBoundVertexBufferIndex >= 0 && CurrentResourcesBoundAsVBs[MaxBoundVertexBufferIndex] == nullptr);
		}
	}
}

void FD3D11DynamicRHI::TrackResourceBoundAsIB(FD3D11ViewableResource* Resource)
{
	CurrentResourceBoundAsIB = Resource;
}

template <EShaderFrequency ShaderFrequency>
void FD3D11DynamicRHI::ClearShaderResourceViews(FD3D11ViewableResource* Resource)
{
	int32 MaxIndex = MaxBoundShaderResourcesIndex[ShaderFrequency];
	for (int32 ResourceIndex = MaxIndex; ResourceIndex >= 0; --ResourceIndex)
	{
		if (CurrentResourcesBoundAsSRVs[ShaderFrequency][ResourceIndex] == Resource)
		{
			// Unset the SRV from the device context
			SetShaderResourceView<ShaderFrequency>(nullptr, nullptr, ResourceIndex);
		}
	}
}

void FD3D11DynamicRHI::ConditionalClearShaderResource(FD3D11ViewableResource* Resource, bool bCheckBoundInputAssembler)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D11ClearShaderResourceTime);
	check(Resource);
	ClearShaderResourceViews<SF_Vertex>(Resource);
	ClearShaderResourceViews<SF_Pixel>(Resource);
	ClearShaderResourceViews<SF_Geometry>(Resource);
	ClearShaderResourceViews<SF_Compute>(Resource);

	if (bCheckBoundInputAssembler)
	{
		for (int32 ResourceIndex = MaxBoundVertexBufferIndex; ResourceIndex >= 0; --ResourceIndex)
		{
			if (CurrentResourcesBoundAsVBs[ResourceIndex] == Resource)
			{
				// Unset the vertex buffer from the device context
				TrackResourceBoundAsVB(nullptr, ResourceIndex);
				StateCache.SetStreamSource(nullptr, ResourceIndex, 0);
			}
		}

		if (Resource == CurrentResourceBoundAsIB)
		{
			TrackResourceBoundAsIB(nullptr);
			StateCache.SetIndexBuffer(nullptr, DXGI_FORMAT_R16_UINT, 0);
		}
	}
}

template <EShaderFrequency ShaderFrequency>
void FD3D11DynamicRHI::ClearAllShaderResourcesForFrequency()
{
	int32 MaxIndex = MaxBoundShaderResourcesIndex[ShaderFrequency];
	for (int32 ResourceIndex = MaxIndex; ResourceIndex >= 0; --ResourceIndex)
	{
		if (CurrentResourcesBoundAsSRVs[ShaderFrequency][ResourceIndex] != nullptr)
		{
			// Unset the SRV from the device context
			SetShaderResourceView<ShaderFrequency>(nullptr, nullptr, ResourceIndex);
		}
	}
	StateCache.ClearConstantBuffers<ShaderFrequency>();
}

// For D3D11Commands.cpp
template void FD3D11DynamicRHI::ClearAllShaderResourcesForFrequency<SF_Compute>();

void FD3D11DynamicRHI::ClearAllShaderResources()
{
	ClearAllShaderResourcesForFrequency<SF_Vertex>();
	ClearAllShaderResourcesForFrequency<SF_Geometry>();
	ClearAllShaderResourcesForFrequency<SF_Pixel>();
	ClearAllShaderResourcesForFrequency<SF_Compute>();
}

static void D3D11UpdateBufferStatsCommon(ID3D11Buffer* Buffer, int64 BufferSize, bool bAllocating)
{
	// this is a work-around on Windows. Due to the fact that there is no way
	// to hook the actual d3d allocations we can't track the memory in the normal way.
	// Instead we simply tell LLM the size of these resources.

	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT(ELLMTag::GraphicsPlatform, bAllocating ? BufferSize : -BufferSize, ELLMTracker::Platform, ELLMAllocType::None);

#if UE_MEMORY_TRACE_ENABLED
	if (bAllocating)
	{
		MemoryTrace_Alloc((uint64)Buffer, BufferSize, 0, EMemoryTraceRootHeap::VideoMemory);
	}
	else
	{
		MemoryTrace_Free((uint64)Buffer, EMemoryTraceRootHeap::VideoMemory);
	}
#endif
}

static void D3D11UpdateAllocationTags(ID3D11Buffer* Buffer, int64 BufferSize)
{
	// We do not track d3d11 allocations with LLM, only insights
#if UE_MEMORY_TRACE_ENABLED
	MemoryTrace_UpdateAlloc((uint64)Buffer, EMemoryTraceRootHeap::VideoMemory);
#endif
}

void D3D11BufferStats::UpdateUniformBufferStats(ID3D11Buffer* Buffer, int64 BufferSize, bool bAllocating)
{
	UE::RHICore::UpdateGlobalUniformBufferStats(BufferSize, bAllocating);
	D3D11UpdateBufferStatsCommon(Buffer, BufferSize, bAllocating);
}

void D3D11BufferStats::UpdateBufferStats(FD3D11Buffer& Buffer, bool bAllocating)
{
	if (ID3D11Buffer* Resource = Buffer.Resource)
	{
		const FRHIBufferDesc& BufferDesc = Buffer.GetDesc();

		UE::RHICore::UpdateGlobalBufferStats(BufferDesc, BufferDesc.Size, bAllocating);
		D3D11UpdateBufferStatsCommon(Resource, BufferDesc.Size, bAllocating);
	}
}

void FD3D11DynamicRHI::UpdateMemoryStats()
{
#if PLATFORM_WINDOWS && (STATS || CSV_PROFILER_STATS)
	// Some older drivers don't support querying memory stats, so don't do anything if this fails.
	if (SUCCEEDED(UE::DXGIUtilities::GetD3DMemoryStats(GetAdapter().DXGIAdapter, MemoryStats)))
	{
		UpdateD3DMemoryStatsAndCSV(MemoryStats, true);
	}
#endif // PLATFORM_WINDOWS && (STATS || CSV_PROFILER_STATS)
}

void FD3D11DynamicRHI::RHIGetMemoryStats(FRHIMemoryStats& OutStats)
{
	OutStats = MemoryStats;
}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
void FD3D11DynamicRHI::RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	check(RHICmdList.IsBottomOfPipe());
	FD3D11Buffer* Buffer = ResourceCast(BufferRHI);

	if (ID3D11Buffer* Resource = Buffer->Resource)
	{
		const FRHIBufferDesc& BufferDesc = Buffer->GetDesc();

		D3D11UpdateAllocationTags(Resource, BufferDesc.Size);
	}
}
#endif // #if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED

ID3D11Device* FD3D11DynamicRHI::RHIGetDevice() const
{
	return GetDevice();
}

ID3D11DeviceContext* FD3D11DynamicRHI::RHIGetDeviceContext() const
{
	return GetDeviceContext();
}

IDXGIAdapter* FD3D11DynamicRHI::RHIGetAdapter() const
{
	return GetAdapter().DXGIAdapter;
}

FRHIViewport* FD3D11DynamicRHI::RHIGetSingletonViewport() const
{
	return Viewports.Num() == 1
		? Viewports[0]
		: nullptr;
}

IDXGISwapChain* FD3D11DynamicRHI::RHIGetSwapChain(FRHIViewport* InViewport) const
{
	FD3D11Viewport* Viewport = static_cast<FD3D11Viewport*>(InViewport);
	return Viewport->GetSwapChain();
}

DXGI_FORMAT FD3D11DynamicRHI::RHIGetSwapChainFormat(EPixelFormat InFormat) const
{
	const DXGI_FORMAT PlatformFormat = UE::DXGIUtilities::FindDepthStencilFormat(static_cast<DXGI_FORMAT>(GPixelFormats[InFormat].PlatformFormat));
	return UE::DXGIUtilities::FindShaderResourceFormat(PlatformFormat, true);
}

ID3D11Buffer* FD3D11DynamicRHI::RHIGetResource(FRHIBuffer* InBuffer) const
{
	FD3D11Buffer* Buffer = ResourceCast(InBuffer);
	return Buffer->Resource;
}

ID3D11Resource* FD3D11DynamicRHI::RHIGetResource(FRHITexture* InTexture) const
{
	FD3D11Texture* D3D11Texture = ResourceCast(InTexture);
	return D3D11Texture->GetResource();
}

int64 FD3D11DynamicRHI::RHIGetResourceMemorySize(FRHITexture* InTexture) const
{
	FD3D11Texture* D3D11Texture = ResourceCast(InTexture);
	return D3D11Texture->GetMemorySize();
}

ID3D11RenderTargetView* FD3D11DynamicRHI::RHIGetRenderTargetView(FRHITexture* InTexture, int32 InMipIndex, int32 InArraySliceIndex) const
{
	FD3D11Texture* D3D11Texture = ResourceCast(InTexture);
	return D3D11Texture->GetRenderTargetView(InMipIndex, InArraySliceIndex);
}

ID3D11ShaderResourceView* FD3D11DynamicRHI::RHIGetShaderResourceView(FRHITexture* InTexture) const
{
	FD3D11Texture* D3D11Texture = ResourceCast(InTexture);
	return D3D11Texture->GetShaderResourceView();
}

void FD3D11DynamicRHI::RHIVerifyResult(ID3D11Device* Device, HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line) const
{
	VerifyD3D11Result(Result, Code, Filename, Line, Device);
}

FD3D11RenderQuery* FD3D11DynamicRHI::InsertProfilerTimestamp(uint64* Target, bool bEndFrame)
{
	TArray<FD3D11RenderQuery*>& Pool = bEndFrame
		? Profiler.TimestampPoolEndFrame
		: Profiler.TimestampPool;

	FD3D11RenderQuery* Query;
	if (Pool.IsEmpty())
	{
		Query = new FD3D11RenderQuery(bEndFrame
			? FD3D11RenderQuery::EType::ProfilerEndFrame
			: FD3D11RenderQuery::EType::Profiler
		);
	}
	else
	{
		Query = Pool.Pop();
	}

	Query->End(Direct3DDeviceIMContext, Target);
	return Query;
}
