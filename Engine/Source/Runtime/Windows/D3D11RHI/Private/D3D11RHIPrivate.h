// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11RHIPrivate.h: Private D3D RHI definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ID3D11DynamicRHI.h"
#include "D3D11RHI.h"
// Dependencies.
#include "RHI.h"
#include "GPUProfiler.h"
#include "ShaderCore.h"
#include "Containers/ResourceArray.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "HDRHelper.h"
#include "BoundShaderStateHistory.h"
#include "DXGIUtilities.h"

DECLARE_LOG_CATEGORY_EXTERN(LogD3D11RHI, Log, All);

#include "Containers/StaticArray.h"

// D3D RHI public headers.
#include "D3D11Util.h"
#include "D3D11State.h"
#include "D3D11Resources.h"
#include "D3D11Viewport.h"
#include "D3D11ConstantBuffer.h"
#include "D3D11StateCache.h"
#include "D3D11NvidiaAftermath.h"
#include "RHIValidationCommon.h"
#include "RHICoreShader.h"

#ifndef WITH_DX_PERF
#define WITH_DX_PERF	1
#endif

#if INTEL_EXTENSIONS
THIRD_PARTY_INCLUDES_START
	#define INTC_IGDEXT_D3D11 1
	#include "igdext.h"
THIRD_PARTY_INCLUDES_END
#endif // INTEL_EXTENSIONS

// DX11 doesn't support higher MSAA count
#define DX_MAX_MSAA_COUNT 8

#ifndef WITH_NV_API
#define WITH_NV_API 0
#endif

#ifndef WITH_AMD_AGS
#define WITH_AMD_AGS 0
#endif

/**
 * The D3D RHI stats.
 */
DECLARE_CYCLE_STAT_EXTERN(TEXT("Present time"),STAT_D3D11PresentTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CustomPresent time"), STAT_D3D11CustomPresentTime, STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CreateTexture time"),STAT_D3D11CreateTextureTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("LockTexture time"),STAT_D3D11LockTextureTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UnlockTexture time"),STAT_D3D11UnlockTextureTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CopyTexture time"),STAT_D3D11CopyTextureTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CreateBoundShaderState time"),STAT_D3D11CreateBoundShaderStateTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("New bound shader state time"),STAT_D3D11NewBoundShaderStateTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Clean uniform buffer pool"),STAT_D3D11CleanUniformBufferTime,STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Clear shader resources"),STAT_D3D11ClearShaderResourceTime,STATGROUP_D3D11RHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Uniform buffer pool num free"),STAT_D3D11NumFreeUniformBuffers,STATGROUP_D3D11RHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Immutable Uniform buffers"), STAT_D3D11NumImmutableUniformBuffers, STATGROUP_D3D11RHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Bound Shader State"), STAT_D3D11NumBoundShaderState, STATGROUP_D3D11RHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Uniform buffer pool memory"), STAT_D3D11FreeUniformBufferMemory, STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update uniform buffer"),STAT_D3D11UpdateUniformBufferTime,STATGROUP_D3D11RHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Textures Allocated"),STAT_D3D11TexturesAllocated,STATGROUP_D3D11RHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Textures Released"),STAT_D3D11TexturesReleased,STATGROUP_D3D11RHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture object pool memory"),STAT_D3D11TexturePoolMemory,STATGROUP_D3D11RHI, );


DECLARE_CYCLE_STAT_EXTERN(TEXT("RenderTargetCommit"), STAT_D3D11RenderTargetCommits, STATGROUP_D3D11RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RenderTargetCommitUAV"), STAT_D3D11RenderTargetCommitsUAV, STATGROUP_D3D11RHI, );

extern TAutoConsoleVariable<int32> GCVarUseSharedKeyedMutex;

extern TAutoConsoleVariable<int32> GD3D11DebugCvar;

struct FD3D11GlobalStats
{
	// in bytes, never change after RHI, needed to scale game features
	static int64 GDedicatedVideoMemory;
	
	// in bytes, never change after RHI, needed to scale game features
	static int64 GDedicatedSystemMemory;
	
	// in bytes, never change after RHI, needed to scale game features
	static int64 GSharedSystemMemory;
	
	// In bytes. Never changed after RHI init. Our estimate of the amount of memory that we can use for graphics resources in total.
	static int64 GTotalGraphicsMemory;
};

struct FD3D11TransitionData
{
	bool bUAVBarrier;
};

/** Forward declare the context for the AMD AGS utility library. */
struct AGSContext;


struct FD3D11DeviceBasicInfo
{
	D3D_FEATURE_LEVEL           MaxFeatureLevel;
	bool						bUMA;
};

struct FD3D11Adapter
{
	/** Null if not supported or FindAdapter() wasn't called. */
	TRefCountPtr<IDXGIAdapter> DXGIAdapter;

	DXGI_ADAPTER_DESC DXGIAdapterDesc;

	/** The maximum D3D11 feature level supported. 0 if not supported or FindAdapter() wasn't called */
	D3D_FEATURE_LEVEL MaxSupportedFeatureLevel;

	/** Whether this is a software adapter */
	bool bSoftwareAdapter;

	/** Whether the GPU has unified memory. This can be used to identify integrated GPU as well. */
	bool bUMA;

	// constructors
	FD3D11Adapter() 
	{
	}

	FD3D11Adapter(TRefCountPtr<IDXGIAdapter> InDXGIAdapter, bool bInSoftwareAdatper, const FD3D11DeviceBasicInfo& DeviceInfo)
		: DXGIAdapter(InDXGIAdapter)
		, MaxSupportedFeatureLevel(DeviceInfo.MaxFeatureLevel)
		, bSoftwareAdapter(bInSoftwareAdatper)
		, bUMA(DeviceInfo.bUMA)
	{
		if (DXGIAdapter.IsValid())
		{
			VERIFYD3D11RESULT(DXGIAdapter->GetDesc(&DXGIAdapterDesc));
		}
	}

	bool IsValid() const
	{
		return DXGIAdapter.IsValid();
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/** The interface which is implemented by the dynamically bound RHI. */
class FD3D11DynamicRHI : public ID3D11DynamicRHI, public IRHICommandContextPSOFallback
{
public:
	typedef TMap<FD3D11LockedKey, FD3D11LockedData> FD3D11LockTracker;
	friend class FD3D11Viewport;

	/** Initialization constructor. */
	D3D11RHI_API FD3D11DynamicRHI(IDXGIFactory1* InDXGIFactory1, D3D_FEATURE_LEVEL InFeatureLevel, const FD3D11Adapter& InAdapter);

	/** Destructor */
	D3D11RHI_API virtual ~FD3D11DynamicRHI();

	/** If it hasn't been initialized yet, initializes the D3D device. */
	D3D11RHI_API virtual void InitD3DDevice();

	// FDynamicRHI interface.
	D3D11RHI_API virtual void Init() override;
	D3D11RHI_API virtual void Shutdown() override;
	virtual const TCHAR* GetName() override { return TEXT("D3D11"); }

	D3D11RHI_API virtual void FlushPendingLogs() override;

	static FD3D11DynamicRHI& Get() { return *GetDynamicRHI<FD3D11DynamicRHI>(); }

	template<typename TRHIType>
	static FORCEINLINE typename TD3D11ResourceTraits<TRHIType>::TConcreteType* ResourceCast(TRHIType* Resource)
	{
		return static_cast<typename TD3D11ResourceTraits<TRHIType>::TConcreteType*>(Resource);
	}

	static inline FD3D11Texture* ResourceCast(FRHITexture* Texture)
	{
		if (!Texture)
		{
			return nullptr;
		}

		FD3D11Texture* Result = static_cast<FD3D11Texture*>(Texture->GetTextureBaseRHI());
		check(Result);

		return Result;
	}

	template<EShaderFrequency ShaderFrequency>
	void BindUniformBuffer(uint32 BufferIndex, FRHIUniformBuffer* BufferRHI);

	template <typename TRHIShader>
	void ApplyStaticUniformBuffers(TRHIShader* Shader);

	template<EShaderFrequency ShaderFrequency>
	void SetShaderParametersCommon(FD3D11ConstantBuffer* StageConstantBuffer, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters);

	template<EShaderFrequency ShaderFrequency>
	void SetShaderUnbindsCommon(TConstArrayView<FRHIShaderParameterUnbind> InUnbinds);

	/**
	 * Reads a D3D query's data into the provided buffer.
	 * @param Query - The D3D query to read data from.
	 * @param Data - The buffer to read the data into.
	 * @param DataSize - The size of the buffer.
	 * @param QueryType e.g. RQT_Occlusion or RQT_AbsoluteTime
	 * @param bWait - If true, it will wait for the query to finish.
	 * @return true if the query finished.
	 */
	D3D11RHI_API bool GetQueryData(ID3D11Query* Query, void* Data, SIZE_T DataSize, bool bTimestamp, bool bWait);

	D3D11RHI_API virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) final override;
	D3D11RHI_API virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) final override;
	D3D11RHI_API virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) final override;
	D3D11RHI_API virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) final override;
	D3D11RHI_API virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) final override;
	D3D11RHI_API virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	D3D11RHI_API virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	D3D11RHI_API virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	D3D11RHI_API virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	D3D11RHI_API virtual FStagingBufferRHIRef RHICreateStagingBuffer() final override;
	D3D11RHI_API virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBuffer, FRHIStagingBuffer* DestinationStagingBuffer, uint32 Offset, uint32 NumBytes) final override;
	D3D11RHI_API virtual FGPUFenceRHIRef RHICreateGPUFence(const FName& Name) final override;
	D3D11RHI_API virtual void RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI) final override;
	D3D11RHI_API virtual void RHIWriteGPUFence(FRHIGPUFence* Fence) final override;
	D3D11RHI_API virtual void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) final override;
    D3D11RHI_API virtual void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer) final override;
	D3D11RHI_API virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader) final override;
	D3D11RHI_API virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) final override;
	D3D11RHI_API virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) final override;
	
#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	D3D11RHI_API virtual void RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
#endif

	[[nodiscard]] D3D11RHI_API FD3D11Buffer* BeginCreateBufferInternal(const FRHIBufferCreateDesc& CreateDesc);
	D3D11RHI_API void FinalizeCreateBufferInternal(FD3D11Buffer* Buffer, TConstArrayView<uint8> InitialData);
	[[nodiscard]] D3D11RHI_API FD3D11Buffer* CreateBufferInternal(const FRHIBufferCreateDesc& CreateDesc, TConstArrayView<uint8> InitialData);

	[[nodiscard]] D3D11RHI_API virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) final override;

	D3D11RHI_API virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) final override;
	D3D11RHI_API virtual void* LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 Size, EResourceLockMode LockMode) final override;
	D3D11RHI_API virtual void UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
	D3D11RHI_API virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex) final override;
	D3D11RHI_API virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) final override;
	D3D11RHI_API virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData,int32 SizeX,int32 SizeY,int32 Pitch,int32 PixelSize) final override;

	[[nodiscard]] D3D11RHI_API FD3D11Texture* BeginCreateTextureInternal(const FRHITextureCreateDesc& CreateDesc);
	D3D11RHI_API void  FinalizeCreateTexture2DInternal(FD3D11Texture* Texture, TConstArrayView<D3D11_SUBRESOURCE_DATA> InitialData);
	D3D11RHI_API void  FinalizeCreateTexture3DInternal(FD3D11Texture* Texture, TConstArrayView<D3D11_SUBRESOURCE_DATA> InitialData);
	D3D11RHI_API FD3D11Texture* FinalizeCreateTextureInternal(FD3D11Texture* Texture, const FRHITextureDesc& InDesc, TConstArrayView<uint8> InitialData);
	[[nodiscard]] D3D11RHI_API FD3D11Texture* CreateTextureInternal(const FRHITextureCreateDesc& CreateDesc, TConstArrayView<uint8> InitialData);
	[[nodiscard]] D3D11RHI_API virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc) final override;
	D3D11RHI_API virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) final override;
	D3D11RHI_API virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) final override;
	D3D11RHI_API virtual void RHIGetMemoryStats(FRHIMemoryStats& OutStats) final override;

	D3D11RHI_API virtual FRHILockTextureResult RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;
	D3D11RHI_API virtual void                  RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;

	D3D11RHI_API virtual void RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) final override;
	D3D11RHI_API virtual void RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) final override;
	D3D11RHI_API virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name) final override;
	D3D11RHI_API virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const TCHAR* Name) final override;
	D3D11RHI_API virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name) final override;
	D3D11RHI_API virtual void RHIReadSurfaceData(FRHITexture* Texture,FIntRect Rect,TArray<FColor>& OutData,FReadSurfaceDataFlags InFlags) final override;
	D3D11RHI_API virtual void RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags) final override;
	D3D11RHI_API virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = 0) final override;
	D3D11RHI_API virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = 0) final override;
	D3D11RHI_API virtual void RHIReadSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,TArray<FFloat16Color>& OutData,ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex) final override;
	D3D11RHI_API virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData) final override;
	D3D11RHI_API virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) final override;
	D3D11RHI_API virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) final override;
	D3D11RHI_API virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) final override;
	D3D11RHI_API virtual void RHIAliasTextureResources(FTextureRHIRef& DestTexture, FTextureRHIRef& SrcTexture) final override;
	D3D11RHI_API virtual FTextureRHIRef RHICreateAliasedTexture(FTextureRHIRef& SourceTexture) final override;
	D3D11RHI_API virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) override;
	D3D11RHI_API virtual void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	D3D11RHI_API virtual void RHIHandleDisplayChange() final override;
	D3D11RHI_API virtual void RHITick(float DeltaTime) final override;
	D3D11RHI_API virtual void RHIBlockUntilGPUIdle() final override;
	D3D11RHI_API virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) final override;
	D3D11RHI_API virtual bool RHIGetAvailableResolutionsForDisplay(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate, void* NativeMonitorHandle) final override;
	D3D11RHI_API virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) final override;
	D3D11RHI_API virtual void* RHIGetNativeDevice() final override;
	D3D11RHI_API virtual void* RHIGetNativeCommandBuffer() final override;
	D3D11RHI_API virtual class IRHICommandContext* RHIGetDefaultContext() final override;
	D3D11RHI_API virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) final override;
	D3D11RHI_API virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) final override;
	D3D11RHI_API virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) final override;

	// SRV / UAV creation functions
	D3D11RHI_API virtual FShaderResourceViewRHIRef  RHICreateShaderResourceView (class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;
	D3D11RHI_API virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;

	D3D11RHI_API virtual void RHISetComputeShader(FRHIComputeShader* ComputeShader) final override;
	D3D11RHI_API virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
	D3D11RHI_API virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	D3D11RHI_API virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) final override;
	D3D11RHI_API virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override;
	D3D11RHI_API virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override;
	D3D11RHI_API virtual void RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo) final override;
	D3D11RHI_API virtual void RHICopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes) final override;
	D3D11RHI_API virtual void RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo) final override;
	D3D11RHI_API virtual void RHIReleaseTransition(FRHITransition* Transition) final override;
	D3D11RHI_API virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) override final;
	D3D11RHI_API virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) override final;

	D3D11RHI_API virtual void RHIEndRenderQuery_TopOfPipe  (FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) override final;
	D3D11RHI_API virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) final override;
	D3D11RHI_API virtual void RHIEndRenderQuery  (FRHIRenderQuery* RenderQuery) final override;

	D3D11RHI_API virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, const FRHIPresentArgs& PresentArgs) final override;
	D3D11RHI_API virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) final override;
	D3D11RHI_API virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) final override;
	D3D11RHI_API virtual void RHISetRasterizerState(FRHIRasterizerState* NewState) final override;
	D3D11RHI_API virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) final override;
	D3D11RHI_API virtual void RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ) override;
	D3D11RHI_API virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) final override;
	D3D11RHI_API virtual void RHISetBoundShaderState(FRHIBoundShaderState* BoundShaderState) final override;
	D3D11RHI_API virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState) final override;
	D3D11RHI_API virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) final override;
	D3D11RHI_API virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* Buffer) final override;
	D3D11RHI_API virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	D3D11RHI_API virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	D3D11RHI_API virtual void RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override;
	D3D11RHI_API virtual void RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override;
	D3D11RHI_API virtual void RHISetDepthStencilState(FRHIDepthStencilState* NewState, uint32 StencilRef) final override;
	D3D11RHI_API virtual void RHISetStencilRef(uint32 StencilRef) final override;
	D3D11RHI_API virtual void RHISetBlendState(FRHIBlendState* NewState, const FLinearColor& BlendFactor) final override;
	D3D11RHI_API virtual void RHISetBlendFactor(const FLinearColor& BlendFactor) final override;
	D3D11RHI_API void SetRenderTargets(uint32 NumSimultaneousRenderTargets, const FRHIRenderTargetView* NewRenderTargets, const FRHIDepthRenderTargetView* NewDepthStencilTarget);
	D3D11RHI_API void InternalSetUAVCS(uint32 BindIndex, FD3D11UnorderedAccessView* UnorderedAccessViewRHI);
	D3D11RHI_API void InternalSetUAVVSPS(uint32 BindIndex, FD3D11UnorderedAccessView* UnorderedAccessViewRHI);
	D3D11RHI_API void SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo);
	D3D11RHI_API virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	D3D11RHI_API virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	D3D11RHI_API virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	D3D11RHI_API virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	virtual void RHIEnableDepthBoundsTest(bool bEnable) final override
	{
		if (GSupportsDepthBoundsTest && StateCache.bDepthBoundsEnabled != bEnable)
		{
			EnableDepthBoundsTest(bEnable, 0.0f, 1.0f);
		}
	}
	virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) final override
	{
		if (GSupportsDepthBoundsTest && (StateCache.DepthBoundsMin != MinDepth || StateCache.DepthBoundsMax != MaxDepth))
		{
			EnableDepthBoundsTest(true, MinDepth, MaxDepth);
		}
	}
#if WITH_RHI_BREADCRUMBS
	D3D11RHI_API virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override;
	D3D11RHI_API virtual void RHIEndBreadcrumbGPU  (FRHIBreadcrumbNode* Breadcrumb) final override;
#endif

	D3D11RHI_API void PollQueryResults();
	D3D11RHI_API bool PollQueryResultsForEndFrame(FD3D11RenderQuery* Query);

	D3D11RHI_API virtual void RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData) final override;

	D3D11RHI_API virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName) final override;
	D3D11RHI_API virtual void RHIEndRenderPass() final override;

	D3D11RHI_API void ResolveTexture(UE::RHICore::FResolveTextureInfo Info);

	// ID3D11DynamicRHI interface
	D3D11RHI_API virtual ID3D11Device*         RHIGetDevice() const final override;
	D3D11RHI_API virtual ID3D11DeviceContext*  RHIGetDeviceContext() const final override;
	D3D11RHI_API virtual IDXGIAdapter*         RHIGetAdapter() const final override;
	D3D11RHI_API virtual IDXGISwapChain*       RHIGetSwapChain(FRHIViewport* InViewport) const final override;
	D3D11RHI_API virtual FRHIViewport*         RHIGetSingletonViewport() const final override;
	D3D11RHI_API virtual DXGI_FORMAT           RHIGetSwapChainFormat(EPixelFormat InFormat) const final override;
	D3D11RHI_API virtual FTextureRHIRef        RHICreateTexture2DFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* Resource) final override;
	D3D11RHI_API virtual FTextureRHIRef        RHICreateTexture2DArrayFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding&, ID3D11Texture2D* Resource) final override;
	D3D11RHI_API virtual FTextureRHIRef        RHICreateTextureCubeFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* Resource) final override;
	D3D11RHI_API virtual ID3D11Buffer*         RHIGetResource(FRHIBuffer* InBuffer) const final override;
	D3D11RHI_API virtual ID3D11Resource*       RHIGetResource(FRHITexture* InTexture) const final override;
	D3D11RHI_API virtual int64                 RHIGetResourceMemorySize(FRHITexture* InTexture) const final override;
	D3D11RHI_API virtual ID3D11RenderTargetView* RHIGetRenderTargetView(FRHITexture* InTexture, int32 InMipIndex = 0, int32 InArraySliceIndex = -1) const final override;
	D3D11RHI_API virtual ID3D11ShaderResourceView* RHIGetShaderResourceView(FRHITexture* InTexture) const final override;
	D3D11RHI_API virtual void                  RHIVerifyResult(ID3D11Device* Device, HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line) const final override;

	D3D11RHI_API virtual void				  RHIGetDisplaysInformation(FDisplayInformationArray& OutDisplayInformation) final override;

	// Accessors.
	ID3D11Device* GetDevice() const
	{
		return Direct3DDevice;
	}
	FD3D11DeviceContext* GetDeviceContext() const
	{
		return Direct3DDeviceIMContext;
	}

	IDXGIFactory1* GetFactory() const
	{
		return DXGIFactory1;
	}

	void AddLockedData(const FD3D11LockedKey& Key, const FD3D11LockedData& LockedData)
	{
		FScopeLock Lock(&LockTrackerCS);
		LockTracker.Add(Key, LockedData);
	}

	bool RemoveLockedData(const FD3D11LockedKey& Key, FD3D11LockedData& OutLockedData)
	{
		FScopeLock Lock(&LockTrackerCS);
		return LockTracker.RemoveAndCopyValue(Key, OutLockedData);
	}

	D3D11RHI_API bool IsQuadBufferStereoEnabled();
	D3D11RHI_API void DisableQuadBufferStereo();

	void BeginRecursiveCommand()
	{
		// Nothing to do
	}

private:
#if PLATFORM_WINDOWS
	void InitializeDeviceAgnosticVendorAPI();
#endif

	D3D11RHI_API void EnableDepthBoundsTest(bool bEnable, float MinDepth, float MaxDepth);

	static D3D11RHI_API void ClearUAV(TRHICommandList_RecursiveHazardous<FD3D11DynamicRHI>& RHICmdList, FD3D11UnorderedAccessView* UAV, const void* ClearValues, bool bFloat);

	enum class EForceFullScreenClear
	{
		EDoNotForce,
		EForce
	};

	D3D11RHI_API virtual void RHIClearMRTImpl(const bool* bClearColorArray, int32 NumClearColors, const FLinearColor* ColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil);

	template <EShaderFrequency ShaderFrequency>
	void ClearShaderResourceViews(FD3D11ViewableResource* Resource);

	template <EShaderFrequency ShaderFrequency>
	void ClearAllShaderResourcesForFrequency();

	template <EShaderFrequency ShaderFrequency>
	void InternalSetShaderResourceView(FD3D11ViewableResource* Resource, ID3D11ShaderResourceView* SRV, int32 ResourceIndex);

	D3D11RHI_API void TrackResourceBoundAsVB(FD3D11ViewableResource* Resource, int32 StreamIndex);
	D3D11RHI_API void TrackResourceBoundAsIB(FD3D11ViewableResource* Resource);

	void SetCurrentComputeShader(FRHIComputeShader* ComputeShader)
	{
		CurrentComputeShader = ComputeShader;
	}
	
	const FComputeShaderRHIRef& GetCurrentComputeShader() const
	{
		return CurrentComputeShader;
	}

public:

	template <EShaderFrequency ShaderFrequency>
	void SetShaderResourceView(FD3D11ViewableResource* Resource, ID3D11ShaderResourceView* SRV, int32 ResourceIndex)
	{
		InternalSetShaderResourceView<ShaderFrequency>(Resource, SRV, ResourceIndex);
	}

	D3D11RHI_API void ClearState();
	D3D11RHI_API void ConditionalClearShaderResource(FD3D11ViewableResource* Resource, bool bCheckBoundInputAssembler);
	D3D11RHI_API void ClearAllShaderResources();

	D3D11RHI_API EPixelFormat GetDisplayFormat(EPixelFormat InPixelFormat) const;

	FD3D11StateCache& GetStateCache() { return StateCache; }

protected:
	/** The global D3D interface. */
	TRefCountPtr<IDXGIFactory1> DXGIFactory1;
#if PLATFORM_WINDOWS
	TRefCountPtr<IDXGIFactory1> DXGIFactoryForDisplayList;
	TRefCountPtr<IDXGIFactory1> DXGIFactoryForResolutions;
#endif

	// Whether HDR is available from the particular DXGI factories available
	bool bDXGISupportsHDR;

	/** The global D3D device's immediate context */
	TRefCountPtr<FD3D11DeviceContext> Direct3DDeviceIMContext;

#if WITH_DX_PERF
	TRefCountPtr<ID3DUserDefinedAnnotation> Direct3DDeviceIMAnnotation;
#endif

#if NV_AFTERMATH
	UE::RHICore::Nvidia::Aftermath::D3D11::FCommandList AftermathHandle = nullptr;
#endif

	/** The global D3D device's immediate context */
	TRefCountPtr<FD3D11Device> Direct3DDevice;

	FD3D11StateCache StateCache;

	/** Tracks outstanding locks on each thread */
	FD3D11LockTracker LockTracker;
	FCriticalSection LockTrackerCS;

	/** A list of all viewport RHIs that have been created. */
	TArray<FD3D11Viewport*> Viewports;

	/** The feature level of the device. */
	D3D_FEATURE_LEVEL FeatureLevel;

	/**
	 * The context for the AMD AGS utility library.
	 * AGSContext does not implement AddRef/Release.
	 * Just use a bare pointer.
	 */
	AGSContext* AmdAgsContext;

	bool bNvAPIInitialized = false;

#if INTEL_EXTENSIONS
	// Context and functions for Intel extension framework utility library
	INTCExtensionContext* IntelExtensionContext;
	bool bIntelSupportsUAVOverlap;
#endif // INTEL_EXTENSIONS

	// set by UpdateMSAASettings(), get by GetMSAAQuality()
	// [SampleCount] = Quality, 0xffffffff if not supported
	uint32 AvailableMSAAQualities[DX_MAX_MSAA_COUNT + 1];

	/** A buffer in system memory containing all zeroes of the specified size. */
	void* ZeroBuffer;
	uint32 ZeroBufferSize;

	// Tracks the currently set state blocks.
	bool bCurrentDepthStencilStateIsReadOnly;

	// Current PSO Primitive Type
	EPrimitiveType PrimitiveType;

	TRefCountPtr<ID3D11RenderTargetView> CurrentRenderTargets[D3D11_SIMULTANEOUS_RENDER_TARGET_COUNT];
	TRefCountPtr<FD3D11UnorderedAccessView> CurrentUAVs[D3D11_PS_CS_UAV_REGISTER_COUNT];
	ID3D11UnorderedAccessView* UAVBound[D3D11_PS_CS_UAV_REGISTER_COUNT];
	uint32 UAVBindFirst;
	uint32 UAVBindCount;
	uint32 UAVSChanged;
	uint32 CurrentRTVOverlapMask;
	uint32 CurrentUAVMask;

	TRefCountPtr<ID3D11DepthStencilView> CurrentDepthStencilTarget;
	TRefCountPtr<FD3D11Texture> CurrentDepthTexture;
	FD3D11ViewableResource* CurrentResourcesBoundAsSRVs[SF_NumStandardFrequencies][D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT];
	FD3D11ViewableResource* CurrentResourcesBoundAsVBs[D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT];
	FD3D11ViewableResource* CurrentResourceBoundAsIB;
	int32 MaxBoundShaderResourcesIndex[SF_NumStandardFrequencies];
	int32 MaxBoundVertexBufferIndex;
	uint32 NumSimultaneousRenderTargets;
	uint32 NumUAVs;

	// Render queries that should be polled by the RHI thread.
	struct
	{
		FD3D11RenderQuery* First = nullptr;
		FD3D11RenderQuery* Last = nullptr;
	} ActiveQueries;
	friend class FD3D11RenderQuery;

	struct FProfiler
	{
		struct FFrame
		{
			struct FEndFrameQuery
			{
				FD3D11RenderQuery* Ptr = nullptr;

				FEndFrameQuery() = default;
				FEndFrameQuery(FEndFrameQuery&& Other)
					: Ptr(Other.Ptr)
				{
					Other.Ptr = nullptr;
				}

			} EndFrameQuery;

			UE::RHI::GPUProfiler::FEventStream EventStream;

		#if WITH_RHI_BREADCRUMBS
			FRHIBreadcrumbAllocatorArray BreadcrumbAllocators{};
		#endif

			FFrame()
				: EventStream(UE::RHI::GPUProfiler::FQueue(UE::RHI::GPUProfiler::FQueue::EType::Graphics, 0, 0))
			{}
		};

		FFrame Current;
		TQueue<TUniquePtr<FFrame>> Pending;

		TArray<FD3D11RenderQuery*> TimestampPool;
		TArray<FD3D11RenderQuery*> TimestampPoolEndFrame;

	} Profiler;

	template <typename TEventType, typename... TArgs>
	TEventType& EmplaceProfilerEvent(TArgs&&... Args)
	{
		return Profiler.Current.EventStream.Emplace<TEventType>(Forward<TArgs>(Args)...);
	}

	D3D11RHI_API FD3D11RenderQuery* InsertProfilerTimestamp(uint64* Target, bool bEndFrame);

	void FlushProfilerStats()
	{
		// Flush accumulated draw stats
		if (StatEvent)
		{
			EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FStats>() = StatEvent;
			StatEvent = {};
		}
	}

public:
	struct FTimestampCalibration
	{
		uint64 CPUTimestamp = 0;
		uint64 CPUFrequency = 0;

		uint64 GPUTimestamp = 0;
		uint64 GPUFrequency = 0;
	};

	D3D11RHI_API TOptional<FTimestampCalibration> CalibrateTimers();

protected:
	TOptional<FTimestampCalibration> TimestampCalibration;

	/** D3D11 defines a maximum of 14 constant buffers per shader stage. */
	enum { MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE = 14 };

	/** Track the currently bound uniform buffers. */
	FRHIUniformBuffer* BoundUniformBuffers[SF_NumStandardFrequencies][MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE];

	/** Bit array to track which uniform buffers have changed since the last draw call. */
	uint16 DirtyUniformBuffers[SF_NumStandardFrequencies];

	TArray<FRHIUniformBuffer*> StaticUniformBuffers;

	/** Tracks the current depth stencil access type. */
	FExclusiveDepthStencil CurrentDSVAccessType;

	/** When a new shader is set, we discard all old constants set for the previous shader. */
	bool bDiscardSharedConstants;

	/** A list of all D3D constant buffers RHIs that have been created. */
	TRefCountPtr<FD3D11ConstantBuffer> VSConstantBuffer;
	TRefCountPtr<FD3D11ConstantBuffer> PSConstantBuffer;
	TRefCountPtr<FD3D11ConstantBuffer> GSConstantBuffer;
	TRefCountPtr<FD3D11ConstantBuffer> CSConstantBuffer;

	/** A history of the most recently used bound shader states, used to keep transient bound shader states from being recreated for each use. */
	TGlobalResource< TBoundShaderStateHistory<10000> > BoundShaderStateHistory;
	FComputeShaderRHIRef CurrentComputeShader;

	FDisplayInformationArray DisplayList;

	HANDLE ExceptionHandlerHandle = INVALID_HANDLE_VALUE;

	bool bRenderDoc = false;

public:
	inline const FD3D11Adapter& GetAdapter() const { return Adapter; }

protected:
	FD3D11Adapter Adapter;
	FRHIMemoryStats MemoryStats;

	D3D11RHI_API FD3D11Texture* CreateTextureFromResource(bool bTextureArray, bool bCubeTexture, EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* TextureResource);

	/** Initializes the constant buffers.  Called once at RHI initialization time. */
	D3D11RHI_API void InitConstantBuffers();

	/** needs to be called before each draw call */
	D3D11RHI_API virtual void CommitNonComputeShaderConstants();

	/** needs to be called before each dispatch call */
	D3D11RHI_API virtual void CommitComputeShaderConstants();

	template <class ShaderType> void SetResourcesFromTables(const ShaderType* RESTRICT);

	D3D11RHI_API void CommitGraphicsResourceTables();
	D3D11RHI_API void CommitComputeResourceTables(FD3D11ComputeShader* ComputeShader);

	D3D11RHI_API void ValidateExclusiveDepthStencilAccess(FExclusiveDepthStencil Src) const;

	/** 
	 * Gets the best supported MSAA settings from the provided MSAA count to check against. 
	 * 
	 * @param PlatformFormat		The format of the texture being created 
	 * @param MSAACount				The MSAA count to check against. 
	 * @param OutBestMSAACount		The best MSAA count that is suppored.  Could be smaller than MSAACount if it is not supported 
	 * @param OutMSAAQualityLevels	The number MSAA quality levels for the best msaa count supported
	 */
	D3D11RHI_API void GetBestSupportedMSAASetting( DXGI_FORMAT PlatformFormat, uint32 MSAACount, uint32& OutBestMSAACount, uint32& OutMSAAQualityLevels );

	// shared code for different D3D11 devices (e.g. PC DirectX11 and XboxOne) called
	// after device creation and GRHISupportsAsyncTextureCreation was set and before resource init
	D3D11RHI_API void SetupAfterDeviceCreation();

	// called by SetupAfterDeviceCreation() when the device gets initialized
	D3D11RHI_API void UpdateMSAASettings();

	// @return 0xffffffff if not not supported
	D3D11RHI_API uint32 GetMaxMSAAQuality(uint32 SampleCount);

	D3D11RHI_API void CommitRenderTargetsAndUAVs();
	D3D11RHI_API void CommitRenderTargets(bool bClearUAVS);
	D3D11RHI_API void CommitUAVs();

	/**
	 * Cleanup the D3D device.
	 * This function must be called from the main game thread.
	 */
	D3D11RHI_API virtual void CleanupD3DDevice();

	D3D11RHI_API void ReleasePooledUniformBuffers();

	template<typename TPixelShader>
	static void ResolveTextureUsingShader(
		FD3D11DynamicRHI* const This,
		FD3D11Texture* const SourceTexture,
		FD3D11Texture* const DestTexture,
		ID3D11RenderTargetView* const DestTextureRTV,
		ID3D11DepthStencilView* const DestTextureDSV,
		D3D11_TEXTURE2D_DESC const& ResolveTargetDesc,
		FResolveRect const& SourceRect,
		FResolveRect const& DestRect,
		typename TPixelShader::FParameter const PixelShaderParameter
		);

	/**
	* Returns a pointer to a texture resource that can be used for CPU reads.
	* Note: the returned resource could be the original texture or a new temporary texture.
	* @param TextureRHI - Source texture to create a staging texture from.
	* @param InRect - rectangle to 'stage'.
	* @param StagingRectOUT - parameter is filled with the rectangle to read from the returned texture.
	* @return The CPU readable Texture object.
	*/
	D3D11RHI_API TRefCountPtr<ID3D11Texture2D> GetStagingTexture(FRHITexture* TextureRHI,FIntRect InRect, FIntRect& OutRect, FReadSurfaceDataFlags InFlags);

	D3D11RHI_API void ReadSurfaceDataNoMSAARaw(FRHITexture* TextureRHI,FIntRect Rect,TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags);

	D3D11RHI_API void ReadSurfaceDataMSAARaw(FRHITexture* TextureRHI, FIntRect Rect, TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags);

#if INTEL_EXTENSIONS
	D3D11RHI_API void StartIntelExtensions();
	D3D11RHI_API void StopIntelExtensions();
#endif // INTEL_EXTENSIONS

	bool bUAVOverlapEnabled = false;
	D3D11RHI_API void EnableUAVOverlap();
	D3D11RHI_API void DisableUAVOverlap();

	D3D11RHI_API bool SetupDisplayHDRMetaData();

	D3D11RHI_API void UpdateMemoryStats();
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Implements the D3D11RHI module as a dynamic RHI providing module. */
class FD3D11DynamicRHIModule : public IDynamicRHIModule
{
public:
	// IModuleInterface	
	virtual bool SupportsDynamicReloading() override { return false; }

	// IDynamicRHIModule
	virtual bool IsSupported() override;
	virtual bool IsSupported(ERHIFeatureLevel::Type RequestedFeatureLevel) override;
	virtual FDynamicRHI* CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num) override;

private:
	FD3D11Adapter ChosenAdapter;

	// set MaxSupportedFeatureLevel and ChosenAdapter
	void FindAdapter();
};

// 1d, 31 bit (uses the sign bit for internal use), O(n) where n is the amount of elements stored
// does not enforce any alignment
// unoccupied regions get compacted but occupied don't get compacted
class FRangeAllocator
{
public:

	struct FRange
	{
		// not valid
		FRange()
			: Start(0)
			, Size(0)
		{
			check(!IsValid());
		}

		void SetOccupied(int32 InStart, int32 InSize)
		{
			check(InStart >= 0);
			check(InSize > 0);

			Start = InStart;
			Size = InSize;
			check(IsOccupied());
		}

		void SetUnOccupied(int32 InStart, int32 InSize)
		{
			check(InStart >= 0);
			check(InSize > 0);

			Start = InStart;
			Size = -InSize;
			check(!IsOccupied());
		}

		bool IsValid() { return Size != 0; }

		bool IsOccupied() const { return Size > 0; }
		uint32 ComputeSize() const { return (Size > 0) ? Size : -Size; }

		// @apram InSize can be <0 to remove from the size
		void ExtendUnoccupied(int32 InSize) { check(!IsOccupied()); Size -= InSize; }

		void MakeOccupied(int32 InSize) { check(InSize > 0); check(!IsOccupied()); Size = InSize; }
		void MakeUnOccupied() { check(IsOccupied()); Size = -Size; }

		bool operator==(const FRange& rhs) const { return Start == rhs.Start && Size == rhs.Size; }

		int32 GetStart() { return Start; }
		int32 GetEnd() { return Start + ComputeSize(); }

	private:
		// in bytes
		int32 Start;
		// in bytes, 0:not valid, <0:unoccupied, >0:occupied
		int32 Size;
	};
public:

	// constructor
	FRangeAllocator(uint32 TotalSize)
	{
		FRange NewRange;

		NewRange.SetUnOccupied(0, TotalSize);

		Entries.Add(NewRange);
	}

	// specified range must be non occupied
	void OccupyRange(FRange InRange)
	{
		check(InRange.IsValid());
		check(InRange.IsOccupied());

		for(uint32 i = 0, Num = Entries.Num(); i < Num; ++i)
		{
			FRange& ref = Entries[i];

			if(!ref.IsOccupied())
			{
				int32 OverlapSize = ref.GetEnd() - InRange.GetStart();

				if(OverlapSize > 0)
				{
					int32 FrontCutSize = InRange.GetStart() - ref.GetStart();

					// there is some front part we cut off
					if(FrontCutSize > 0)
					{
						FRange NewFrontRange;

						NewFrontRange.SetUnOccupied(InRange.GetStart(), ref.ComputeSize() - FrontCutSize);

						ref.SetUnOccupied(ref.GetStart(), FrontCutSize);

						++i;

						// remaining is added behind the found element
						Entries.Insert(NewFrontRange, i);

						// don't access ref or Num any more - Entries[] might be reallocated
					}

					check(Entries[i].GetStart() == InRange.GetStart());

					int32 BackCutSize = Entries[i].ComputeSize() - InRange.ComputeSize();

					// otherwise the range was already occupied or not enough space was left (internal error)
					check(BackCutSize >= 0);

					// there is some back part we cut off
					if(BackCutSize > 0)
					{
						FRange NewBackRange;

						NewBackRange.SetUnOccupied(Entries[i].GetStart() + InRange.ComputeSize(), BackCutSize);

						Entries.Insert(NewBackRange, i + 1);
					}

					Entries[i] = InRange;
					return;
				}
			}
		}
	}

	// @param InSize >0
	FRange AllocRange(uint32 InSize)//, uint32 Alignment)
	{
		check(InSize > 0);

		for(uint32 i = 0, Num = Entries.Num(); i < Num; ++i)
		{
			FRange& ref = Entries[i];

			if(!ref.IsOccupied())
			{
				uint32 RefSize = ref.ComputeSize();

				// take the first fitting one - later we could optimize for minimal fragmentation
				if(RefSize >= InSize)
				{
					ref.MakeOccupied(InSize);

					FRange Ret = ref;

					if(RefSize > InSize)
					{
						FRange NewRange;

						NewRange.SetUnOccupied(ref.GetEnd(), RefSize - InSize);

						// remaining is added behind the found element
						Entries.Insert(NewRange, i + 1);
					}
					return Ret;
				}
			}
		}

		// nothing found
		return FRange();
	}

	// @param In needs to be what was returned by AllocRange()
	void ReleaseRange(FRange In)
	{
		int32 Index = Entries.Find(In);

		check(Index != INDEX_NONE);

		FRange& refIndex = Entries[Index];

		refIndex.MakeUnOccupied();

		Compacten(Index);
	}

	// for debugging
	uint32 GetNumEntries() const { return Entries.Num(); }

	// for debugging
	uint32 ComputeUnoccupiedSize() const
	{
		uint32 Ret = 0;

		for(uint32 i = 0, Num = Entries.Num(); i < Num; ++i)
		{
			const FRange& ref = Entries[i];

			if(!ref.IsOccupied())
			{
				uint32 RefSize = ref.ComputeSize();

				Ret += RefSize;
			}
		}

		return Ret;
	}

private:
	// compact unoccupied ranges
	void Compacten(uint32 StartIndex)
	{
		check(!Entries[StartIndex].IsOccupied());

		if(StartIndex && !Entries[StartIndex-1].IsOccupied())
		{
			// Seems we can combine with the element before,
			// searching further is not needed as we assume the buffer was compact before the last change.
			--StartIndex;
		}

		uint32 ElementsToRemove = 0;
		uint32 SizeGained = 0;

		for(uint32 i = StartIndex + 1, Num = Entries.Num(); i < Num; ++i)
		{
			FRange& ref = Entries[i];

			if(!ref.IsOccupied())
			{
				++ElementsToRemove;
				SizeGained += ref.ComputeSize();
			}
			else
			{
				break;
			}
		}

		if(ElementsToRemove)
		{
			Entries.RemoveAt(StartIndex + 1, ElementsToRemove, EAllowShrinking::No);
			Entries[StartIndex].ExtendUnoccupied(SizeGained);
		}
	}

public:
	static void Test()
	{
		// testing code
#if !UE_BUILD_SHIPPING
		{
			// create
			FRangeAllocator A(10);
			check(A.GetNumEntries() == 1);
			check(A.ComputeUnoccupiedSize() == 10);

			// successfully alloc
			FRangeAllocator::FRange a = A.AllocRange(3);
			check(a.GetStart() == 0);
			check(a.GetEnd() == 3);
			check(a.IsOccupied());
			check(A.GetNumEntries() == 2);
			check(A.ComputeUnoccupiedSize() == 7);

			// successfully alloc
			FRangeAllocator::FRange b = A.AllocRange(4);
			check(b.GetStart() == 3);
			check(b.GetEnd() == 7);
			check(b.IsOccupied());
			check(A.GetNumEntries() == 3);
			check(A.ComputeUnoccupiedSize() == 3);

			// failed alloc
			FRangeAllocator::FRange c = A.AllocRange(4);
			check(!c.IsValid());
			check(!c.IsOccupied());
			check(A.GetNumEntries() == 3);
			check(A.ComputeUnoccupiedSize() == 3);

			// successfully alloc
			FRangeAllocator::FRange d = A.AllocRange(3);
			check(d.GetStart() == 7);
			check(d.GetEnd() == 10);
			check(d.IsOccupied());
			check(A.GetNumEntries() == 3);
			check(A.ComputeUnoccupiedSize() == 0);

			A.ReleaseRange(b);
			check(A.GetNumEntries() == 3);
			check(A.ComputeUnoccupiedSize() == 4);

			A.ReleaseRange(a);
			check(A.GetNumEntries() == 2);
			check(A.ComputeUnoccupiedSize() == 7);

			A.ReleaseRange(d);
			check(A.GetNumEntries() == 1);
			check(A.ComputeUnoccupiedSize() == 10);

			// we are back to a clean start

			FRangeAllocator::FRange e = A.AllocRange(10);
			check(e.GetStart() == 0);
			check(e.GetEnd() == 10);
			check(e.IsOccupied());
			check(A.GetNumEntries() == 1);
			check(A.ComputeUnoccupiedSize() == 0);

			A.ReleaseRange(e);
			check(A.GetNumEntries() == 1);
			check(A.ComputeUnoccupiedSize() == 10);

			// we are back to a clean start

			// create define range we want to block out
			FRangeAllocator::FRange f;
			f.SetOccupied(2, 4);
			A.OccupyRange(f);
			check(A.GetNumEntries() == 3);
			check(A.ComputeUnoccupiedSize() == 6);

			FRangeAllocator::FRange g = A.AllocRange(2);
			check(g.GetStart() == 0);
			check(g.GetEnd() == 2);
			check(g.IsOccupied());
			check(A.GetNumEntries() == 3);
			check(A.ComputeUnoccupiedSize() == 4);

			FRangeAllocator::FRange h = A.AllocRange(4);
			check(h.GetStart() == 6);
			check(h.GetEnd() == 10);
			check(h.IsOccupied());
			check(A.GetNumEntries() == 3);
			check(A.ComputeUnoccupiedSize() == 0);
		}
#endif // !UE_BUILD_SHIPPING
	}

private:

	// ordered from small to large (for efficient compactening)
	TArray<FRange> Entries;
};

extern D3D11RHI_API FD3D11DynamicRHI*	GD3D11RHI;
