// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11RenderTarget.cpp: D3D render target implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "BatchedElements.h"
#include "ScreenRendering.h"
#include "RHIStaticStates.h"
#include "ResolveShader.h"
#include "PipelineStateCache.h"
#include "Math/PackedVector.h"
#include "RHISurfaceDataConversion.h"
#include "RHICore.h"

static inline DXGI_FORMAT ConvertTypelessToUnorm(DXGI_FORMAT Format)
{
	// prefer DXGIUtilities::FindSharedResourceFormat ?
	//	or something? lots of these mappers in DXGIUtilities already

	// required to prevent 
	// D3D11: ERROR: ID3D11DeviceContext::ResolveSubresource: The Format (0x1b, R8G8B8A8_TYPELESS) is never able to resolve multisampled resources. [ RESOURCE_MANIPULATION ERROR #294: DEVICE_RESOLVESUBRESOURCE_FORMAT_INVALID ]
	// D3D11: **BREAK** enabled for the previous D3D11 message, which was: [ RESOURCE_MANIPULATION ERROR #294: DEVICE_RESOLVESUBRESOURCE_FORMAT_INVALID ]
	switch (Format)
	{
		case DXGI_FORMAT_R8G8B8A8_TYPELESS:
			return DXGI_FORMAT_R8G8B8A8_UNORM;

		case DXGI_FORMAT_B8G8R8A8_TYPELESS:
			return DXGI_FORMAT_B8G8R8A8_UNORM;

		default:
			return Format;
	}
}

static FResolveRect GetDefaultRect(const FResolveRect& Rect,uint32 DefaultWidth,uint32 DefaultHeight)
{
	if (Rect.X1 >= 0 && Rect.X2 >= 0 && Rect.Y1 >= 0 && Rect.Y2 >= 0)
	{
		return Rect;
	}
	else
	{
		return FResolveRect(0,0,DefaultWidth,DefaultHeight);
	}
}

template<typename TPixelShader>
void FD3D11DynamicRHI::ResolveTextureUsingShader(
	FD3D11DynamicRHI* const This,
	FD3D11Texture* const SourceTexture,
	FD3D11Texture* const DestTexture,
	ID3D11RenderTargetView* const DestTextureRTV,
	ID3D11DepthStencilView* const DestTextureDSV,
	D3D11_TEXTURE2D_DESC const& ResolveTargetDesc,
	FResolveRect const& SourceRect,
	FResolveRect const& DestRect,
	typename TPixelShader::FParameter const PixelShaderParameter
	)
{
	// Save the current viewport so that it can be restored
	D3D11_VIEWPORT SavedViewport;
	uint32 NumSavedViewports = 1;
	This->StateCache.GetViewports(&NumSavedViewports, &SavedViewport);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;

	// No alpha blending, no depth tests or writes, no stencil tests or writes, no backface culling.
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

	// Make sure the destination is not bound as a shader resource.
	if (DestTexture)
	{
		This->ConditionalClearShaderResource(DestTexture, false);
	}

	// Determine if the entire destination surface is being resolved to.
	// If the entire surface is being resolved to, then it means we can clear it and signal the driver that it can discard
	// the surface's previous contents, which breaks dependencies between frames when using alternate-frame SLI.
	const bool bClearDestTexture =
			DestRect.X1 == 0
		&&	DestRect.Y1 == 0
		&&	DestRect.X2 == ResolveTargetDesc.Width
		&&	DestRect.Y2 == ResolveTargetDesc.Height;
	
	const bool bDepthStencil = ResolveTargetDesc.BindFlags & D3D11_BIND_DEPTH_STENCIL;

	//we may change rendertargets and depth state behind the RHI's back here.
	//save off this original state to restore it.
	FExclusiveDepthStencil OriginalDSVAccessType     = This->CurrentDSVAccessType;
	TRefCountPtr<FD3D11Texture> OriginalDepthTexture = This->CurrentDepthTexture;

	{
		TRHICommandList_RecursiveHazardous<FD3D11DynamicRHI> RHICmdList(This);
		if (bDepthStencil)
		{
			RHICmdList.RunOnContext([bClearDestTexture, DestTextureDSV](auto& Context)
			{
				// Clear the destination texture.
				if (bClearDestTexture)
				{
					Context.Direct3DDeviceIMContext->ClearDepthStencilView(DestTextureDSV,D3D11_CLEAR_DEPTH | D3D11_CLEAR_STENCIL,0,0);
				}

				//hack this to  pass validation in SetDepthStencil state since we are directly changing targets with a call to OMSetRenderTargets later.
				Context.CurrentDSVAccessType = FExclusiveDepthStencil::DepthWrite_StencilWrite;
			});

			check(DestTexture);
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();
			GraphicsPSOInit.DepthStencilTargetFormat = DestTexture->GetFormat();

			RHICmdList.BeginRenderPass(FRHIRenderPassInfo(DestTexture, EDepthStencilTargetActions::LoadDepthStencil_StoreDepthStencil), TEXT(""));
		}
		else
		{
			RHICmdList.RunOnContext([bClearDestTexture, DestTextureRTV](auto& Context)
			{
				// Clear the destination texture.
				if (bClearDestTexture)
				{
					FLinearColor ClearColor(0,0,0,0);
					Context.Direct3DDeviceIMContext->ClearRenderTargetView(DestTextureRTV,(float*)&ClearColor);
				}
			});

			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			RHICmdList.BeginRenderPass(FRHIRenderPassInfo(DestTexture, ERenderTargetActions::Load_Store), TEXT(""));
		}

		RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (float)ResolveTargetDesc.Width, (float)ResolveTargetDesc.Height, 1.0f);

		// Set the vertex and pixel shader
		auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FResolveVS> ResolveVertexShader(ShaderMap);
		TShaderMapRef<TPixelShader> ResolvePixelShader(ShaderMap);

		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = ResolveVertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ResolvePixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

		RHICmdList.RunOnContext([DestTexture](auto& Context)
		{
			Context.CurrentDepthTexture = DestTexture;
		});
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
		RHICmdList.SetBlendFactor(FLinearColor::White);

		SetShaderParametersLegacyVS(RHICmdList, ResolveVertexShader, SourceRect, DestRect, ResolveTargetDesc.Width, ResolveTargetDesc.Height);
		SetShaderParametersLegacyPS(RHICmdList, ResolvePixelShader, PixelShaderParameter);

		// Set the source texture.
		const uint32 TextureIndex = ResolvePixelShader->UnresolvedSurface.GetBaseIndex();

		if (SourceTexture)
		{
			RHICmdList.RunOnContext([SourceTexture, TextureIndex](FD3D11DynamicRHI& Context)
			{
				Context.SetShaderResourceView<SF_Pixel>(SourceTexture, SourceTexture->GetShaderResourceView(), TextureIndex);
			});
		}

		RHICmdList.DrawPrimitive(0, 2, 1);

		RHICmdList.EndRenderPass();
	}

	if (SourceTexture)
	{
		This->ConditionalClearShaderResource(SourceTexture, false);
	}

	// Reset saved render targets
	This->CommitRenderTargetsAndUAVs();

	// Reset saved viewport
	This->RHISetMultipleViewports(1, (FViewportBounds*)&SavedViewport);

	//reset DSVAccess.
	This->CurrentDSVAccessType = OriginalDSVAccessType;
	This->CurrentDepthTexture = OriginalDepthTexture;
}

// Only supports the formats that are supported by ConvertRAWSurfaceDataToFColor()
static uint32 D3D11RT_ComputeBytesPerPixel(DXGI_FORMAT Format)
{
	uint32 BytesPerPixel = 0;

	switch(Format)
	{
		case DXGI_FORMAT_R8G8_TYPELESS:
		case DXGI_FORMAT_R8G8_UNORM:
		case DXGI_FORMAT_R8G8_UINT:
		case DXGI_FORMAT_R8G8_SNORM:
		case DXGI_FORMAT_R8G8_SINT:
		case DXGI_FORMAT_R16_TYPELESS:
		case DXGI_FORMAT_R16_FLOAT:
		case DXGI_FORMAT_D16_UNORM:
		case DXGI_FORMAT_R16_UNORM:
		case DXGI_FORMAT_R16_UINT:
		case DXGI_FORMAT_R16_SNORM:
		case DXGI_FORMAT_R16_SINT:
		case DXGI_FORMAT_B5G6R5_UNORM:
		case DXGI_FORMAT_B5G5R5A1_UNORM:
			BytesPerPixel = 2;
			break;
		case DXGI_FORMAT_B8G8R8A8_TYPELESS:
		case DXGI_FORMAT_B8G8R8A8_UNORM:
		case DXGI_FORMAT_R8G8B8A8_TYPELESS:
		case DXGI_FORMAT_R8G8B8A8_UNORM:
		case DXGI_FORMAT_R24G8_TYPELESS:
		case DXGI_FORMAT_R10G10B10A2_UNORM:
		case DXGI_FORMAT_R11G11B10_FLOAT:
		case DXGI_FORMAT_R16G16_UNORM:
		case DXGI_FORMAT_R32_UINT:
		case DXGI_FORMAT_R32_TYPELESS:
		case DXGI_FORMAT_R32_FLOAT:
		case DXGI_FORMAT_B8G8R8A8_UNORM_SRGB:
		case DXGI_FORMAT_B8G8R8X8_TYPELESS:
		case DXGI_FORMAT_B8G8R8X8_UNORM_SRGB:
		case DXGI_FORMAT_B8G8R8X8_UNORM:
		case DXGI_FORMAT_R10G10B10A2_TYPELESS:
		case DXGI_FORMAT_R10G10B10A2_UINT:
		case DXGI_FORMAT_R8G8B8A8_UNORM_SRGB:
		case DXGI_FORMAT_R8G8B8A8_UINT:
		case DXGI_FORMAT_R8G8B8A8_SNORM:
		case DXGI_FORMAT_R8G8B8A8_SINT:
		case DXGI_FORMAT_R16G16_TYPELESS:
		case DXGI_FORMAT_R16G16_FLOAT:
		case DXGI_FORMAT_R16G16_UINT:
		case DXGI_FORMAT_R16G16_SNORM:
		case DXGI_FORMAT_R16G16_SINT:
		case DXGI_FORMAT_D32_FLOAT:
		case DXGI_FORMAT_R32_SINT:
			BytesPerPixel = 4;
			break;
		case DXGI_FORMAT_R16G16B16A16_FLOAT:
		case DXGI_FORMAT_R16G16B16A16_UNORM:
		case DXGI_FORMAT_R16G16B16A16_TYPELESS:
		case DXGI_FORMAT_R16G16B16A16_UINT:
		case DXGI_FORMAT_R16G16B16A16_SNORM:
		case DXGI_FORMAT_R16G16B16A16_SINT:
		case DXGI_FORMAT_R32G32_TYPELESS:
		case DXGI_FORMAT_R32G32_FLOAT:
		case DXGI_FORMAT_R32G32_UINT:
		case DXGI_FORMAT_R32G32_SINT:
			BytesPerPixel = 8;
			break;
		// Changing Depth Buffers to 32 bit on Dingo as D24S8 is actually implemented as a 32 bit buffer in the hardware
		case DXGI_FORMAT_R32G8X24_TYPELESS:
		case DXGI_FORMAT_D32_FLOAT_S8X24_UINT:
		case DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS:
			BytesPerPixel = 8;
			break;
		case DXGI_FORMAT_R8_TYPELESS:
		case DXGI_FORMAT_R8_UNORM:
		case DXGI_FORMAT_R8_UINT:
		case DXGI_FORMAT_R8_SNORM:
		case DXGI_FORMAT_R8_SINT:
		case DXGI_FORMAT_A8_UNORM:
		case DXGI_FORMAT_R1_UNORM:
			BytesPerPixel = 1;
			break;
		case DXGI_FORMAT_R32G32B32A32_UINT:
		case DXGI_FORMAT_R32G32B32A32_TYPELESS:
		case DXGI_FORMAT_R32G32B32A32_FLOAT:
		case DXGI_FORMAT_R32G32B32A32_SINT:
			BytesPerPixel = 16;
			break;

		default:
			// format not supported yet
			check(false);
			break;
	}

	// this function is superceded by DXGIUtilities, delete me ??
	check(BytesPerPixel == UE::DXGIUtilities::GetFormatSizeInBytes(Format) );

	return BytesPerPixel;
}

TRefCountPtr<ID3D11Texture2D> FD3D11DynamicRHI::GetStagingTexture(FRHITexture* TextureRHI,FIntRect InRect, FIntRect& StagingRectOUT, FReadSurfaceDataFlags InFlags)
{
	FD3D11Texture* Texture = ResourceCast(TextureRHI);

	D3D11_TEXTURE2D_DESC SourceDesc; 
	Texture->GetD3D11Texture2D()->GetDesc(&SourceDesc);
	
	bool bRequiresTempStagingTexture = SourceDesc.Usage != D3D11_USAGE_STAGING; 
	if(bRequiresTempStagingTexture == false)
	{
		// Returning the same texture is considerably faster than creating and copying to
		// a new staging texture as we do not have to wait for the GPU pipeline to catch up
		// to the staging texture preparation work.
		StagingRectOUT = InRect;
		return Texture->GetD3D11Texture2D();
	}

	// a temporary staging texture is needed.
	int32 SizeX = InRect.Width();
	int32 SizeY = InRect.Height();
	// Read back the surface data in the defined rect
	D3D11_BOX Rect;
	Rect.left = InRect.Min.X;
	Rect.top = InRect.Min.Y;
	Rect.right = InRect.Max.X;
	Rect.bottom = InRect.Max.Y;
	Rect.back = 1;
	Rect.front = 0;

	// create a temp 2d texture to copy render target to
	D3D11_TEXTURE2D_DESC Desc;
	ZeroMemory( &Desc, sizeof( D3D11_TEXTURE2D_DESC ) );
	Desc.Width = SizeX;
	Desc.Height = SizeY;
	Desc.MipLevels = 1;
	Desc.ArraySize = 1;
	Desc.Format = SourceDesc.Format;
	Desc.SampleDesc.Count = 1;
	Desc.SampleDesc.Quality = 0;
	Desc.Usage = D3D11_USAGE_STAGING;
	Desc.BindFlags = 0;
	Desc.CPUAccessFlags = D3D11_CPU_ACCESS_READ;
	Desc.MiscFlags = 0;
	TRefCountPtr<ID3D11Texture2D> TempTexture2D;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateTexture2D(&Desc,NULL,TempTexture2D.GetInitReference()), Direct3DDevice);

	// Staging rectangle is now the whole surface.
	StagingRectOUT.Min = FIntPoint::ZeroValue;
	StagingRectOUT.Max = FIntPoint(SizeX,SizeY);

	// Copy the data to a staging resource.
	uint32 Subresource = 0;
	if( SourceDesc.MiscFlags == D3D11_RESOURCE_MISC_TEXTURECUBE )
	{
		uint32 D3DFace = GetD3D11CubeFace(InFlags.GetCubeFace());
		Subresource = D3D11CalcSubresource(InFlags.GetMip(), InFlags.GetArrayIndex() * 6 + D3DFace, TextureRHI->GetNumMips());
	}
	else
	{
		const bool bIsTextureArray = Texture->GetDesc().IsTextureArray();
		Subresource = D3D11CalcSubresource(InFlags.GetMip(), bIsTextureArray ? InFlags.GetArrayIndex() : 0, TextureRHI->GetNumMips());
	}

	D3D11_BOX* RectPtr = NULL; // API prefers NULL for entire texture.
	if(Rect.left != 0 || Rect.top != 0 || Rect.right != SourceDesc.Width || Rect.bottom != SourceDesc.Height)
	{
		// ..Sub rectangle required, use the D3D11_BOX.
		RectPtr = &Rect;
	}

	Direct3DDeviceIMContext->CopySubresourceRegion(TempTexture2D,0,0,0,0,Texture->GetResource(),Subresource,RectPtr);

	return TempTexture2D;
}

void FD3D11DynamicRHI::ReadSurfaceDataNoMSAARaw(FRHITexture* TextureRHI,FIntRect InRect,TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags)
{
	checkf(InRect.Width() <= TextureRHI->GetSizeXYZ().X >> InFlags.GetMip(), TEXT("Provided rect width (%d), must be smaller or equal to the texture size requested Mip (%d)"), InRect.Width(), TextureRHI->GetSizeXYZ().X >> InFlags.GetMip());
	checkf(InRect.Height() <= TextureRHI->GetSizeXYZ().Y >> InFlags.GetMip(), TEXT("Provided rect height (%d), must be smaller or equal to the texture size requested Mip (%d)"), InRect.Height(), TextureRHI->GetSizeXYZ().Y >> InFlags.GetMip());

	FD3D11Texture* Texture = ResourceCast(TextureRHI);

	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();

	// Check the format of the surface
	D3D11_TEXTURE2D_DESC TextureDesc;
	Texture->GetD3D11Texture2D()->GetDesc(&TextureDesc);
	
	uint32 BytesPerPixel = D3D11RT_ComputeBytesPerPixel(TextureDesc.Format);
	
	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY * BytesPerPixel);

	bool bIsUsingTempStagingTexture = TextureDesc.Usage != D3D11_USAGE_STAGING;
	FIntRect StagingRect;
	TRefCountPtr<ID3D11Texture2D> TempTexture2D = GetStagingTexture(TextureRHI, InRect, StagingRect, InFlags);

	// Lock the staging resource.
	uint32 MappedSubresource = bIsUsingTempStagingTexture ? 0 : InFlags.GetMip();
	D3D11_MAPPED_SUBRESOURCE LockedRect;
	VERIFYD3D11RESULT_EX(Direct3DDeviceIMContext->Map(TempTexture2D, MappedSubresource, D3D11_MAP_READ,0,&LockedRect), Direct3DDevice);

	uint32 BytesPerLine = BytesPerPixel * InRect.Width();
	uint8* DestPtr = OutData.GetData();
	uint8* SrcPtr = (uint8*)LockedRect.pData + StagingRect.Min.X * BytesPerPixel +  StagingRect.Min.Y * LockedRect.RowPitch;
	for(uint32 Y = 0; Y < SizeY; Y++)
	{
		memcpy(DestPtr, SrcPtr, BytesPerLine);
		DestPtr += BytesPerLine;
		SrcPtr += LockedRect.RowPitch;
	}

	Direct3DDeviceIMContext->Unmap(TempTexture2D, MappedSubresource);
}

void FD3D11DynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI,FIntRect InRect,TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	if (!ensure(TextureRHI))
	{
		OutData.SetNumUninitialized(InRect.Width() * InRect.Height());
		FMemory::Memzero(OutData.GetData(), OutData.Num() * sizeof(FColor));
		return;
	}

	TArray<uint8> OutDataRaw;

	FD3D11Texture* Texture = ResourceCast(TextureRHI);

	// Check the format of the surface
	D3D11_TEXTURE2D_DESC TextureDesc;
	Texture->GetD3D11Texture2D()->GetDesc(&TextureDesc);

	check(TextureDesc.SampleDesc.Count >= 1);

	if(TextureDesc.SampleDesc.Count == 1)
	{
		ReadSurfaceDataNoMSAARaw(TextureRHI, InRect, OutDataRaw, InFlags);
	}
	else
	{
		ReadSurfaceDataMSAARaw(TextureRHI, InRect, OutDataRaw, InFlags);
	}

	const uint32 SizeX = InRect.Width() * TextureDesc.SampleDesc.Count;
	const uint32 SizeY = InRect.Height();

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY);

	uint32 BytesPerPixel = D3D11RT_ComputeBytesPerPixel(TextureDesc.Format);
	uint32 SrcPitch = SizeX * BytesPerPixel;
	
	// switching on the EPixelFormat is risky if the mapping is not what you expect
	//	verify against TextureDesc.Format

	EPixelFormat Format = TextureRHI->GetFormat();
	check( ConvertTypelessToUnorm((DXGI_FORMAT)GPixelFormats[Format].PlatformFormat) == ConvertTypelessToUnorm(TextureDesc.Format) );
	check( GPixelFormats[Format].BlockBytes == D3D11RT_ComputeBytesPerPixel(TextureDesc.Format) );

	// ConvertDXGIToFColor switches on the hardware format, not the EPixelFormat :

	if ( ! ConvertDXGIToFColor(TextureDesc.Format, SizeX, SizeY, OutDataRaw.GetData(), SrcPitch, OutData.GetData(), InFlags) )
	{
		checkf(0, TEXT("Unsupported surface format!"));
		OutData.Empty();
	}
}

void FD3D11DynamicRHI::ReadSurfaceDataMSAARaw(FRHITexture* TextureRHI,FIntRect InRect,TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags)
{
	FD3D11Texture* Texture = ResourceCast(TextureRHI);

	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();
	
	// Check the format of the surface
	D3D11_TEXTURE2D_DESC TextureDesc;
	Texture->GetD3D11Texture2D()->GetDesc(&TextureDesc);

	uint32 BytesPerPixel = D3D11RT_ComputeBytesPerPixel(TextureDesc.Format);

	const uint32 NumSamples = TextureDesc.SampleDesc.Count;

	// Read back the surface data from the define rect
	D3D11_BOX	Rect;
	Rect.left	= InRect.Min.X;
	Rect.top	= InRect.Min.Y;
	Rect.right	= InRect.Max.X;
	Rect.bottom	= InRect.Max.Y;
	Rect.back = 1;
	Rect.front = 0;

	// Create a non-MSAA render target to resolve individual samples of the source surface to.
	D3D11_TEXTURE2D_DESC NonMSAADesc;
	ZeroMemory( &NonMSAADesc, sizeof( D3D11_TEXTURE2D_DESC ) );
	NonMSAADesc.Width = SizeX;
	NonMSAADesc.Height = SizeY;
	NonMSAADesc.MipLevels = 1;
	NonMSAADesc.ArraySize = 1;
	NonMSAADesc.Format = TextureDesc.Format;
	NonMSAADesc.SampleDesc.Count = 1;
	NonMSAADesc.SampleDesc.Quality = 0;
	NonMSAADesc.Usage = D3D11_USAGE_DEFAULT;
	NonMSAADesc.BindFlags = D3D11_BIND_RENDER_TARGET;
	NonMSAADesc.CPUAccessFlags = 0;
	NonMSAADesc.MiscFlags = 0;
	TRefCountPtr<ID3D11Texture2D> NonMSAATexture2D;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateTexture2D(&NonMSAADesc,NULL,NonMSAATexture2D.GetInitReference()), Direct3DDevice);

	TRefCountPtr<ID3D11RenderTargetView> NonMSAARTV;
	D3D11_RENDER_TARGET_VIEW_DESC RTVDesc;
	FMemory::Memset(&RTVDesc,0,sizeof(RTVDesc));

	// typeless is not supported, similar code might be needed for other typeless formats
	RTVDesc.Format = ConvertTypelessToUnorm(NonMSAADesc.Format);

	RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2D;
	RTVDesc.Texture2D.MipSlice = 0;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateRenderTargetView(NonMSAATexture2D,&RTVDesc,NonMSAARTV.GetInitReference()), Direct3DDevice);

	// Create a CPU-accessible staging texture to copy the resolved sample data to.
	TRefCountPtr<ID3D11Texture2D> StagingTexture2D;
	D3D11_TEXTURE2D_DESC StagingDesc;
	ZeroMemory( &StagingDesc, sizeof( D3D11_TEXTURE2D_DESC ) );
	StagingDesc.Width = SizeX;
	StagingDesc.Height = SizeY;
	StagingDesc.MipLevels = 1;
	StagingDesc.ArraySize = 1;
	StagingDesc.Format = TextureDesc.Format;
	StagingDesc.SampleDesc.Count = 1;
	StagingDesc.SampleDesc.Quality = 0;
	StagingDesc.Usage = D3D11_USAGE_STAGING;
	StagingDesc.BindFlags = 0;
	StagingDesc.CPUAccessFlags = D3D11_CPU_ACCESS_READ;
	StagingDesc.MiscFlags = 0;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateTexture2D(&StagingDesc,NULL,StagingTexture2D.GetInitReference()), Direct3DDevice);

	// Determine the subresource index for cubemaps.
	uint32 Subresource = 0;
	if (TextureDesc.MiscFlags == D3D11_RESOURCE_MISC_TEXTURECUBE)
	{
		uint32 D3DFace = GetD3D11CubeFace(InFlags.GetCubeFace());
		Subresource = D3D11CalcSubresource(InFlags.GetMip(), InFlags.GetArrayIndex() * 6 + D3DFace, TextureRHI->GetNumMips());
	}
	else
	{
		const bool bIsTextureArray = Texture->GetDesc().IsTextureArray();
		Subresource = D3D11CalcSubresource(InFlags.GetMip(), bIsTextureArray ? InFlags.GetArrayIndex() : 0, TextureRHI->GetNumMips());
	}
	
	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY * NumSamples * BytesPerPixel);

	// Can be optimized by doing all subsamples into a large enough rendertarget in one pass (multiple draw calls)
	for(uint32 SampleIndex = 0;SampleIndex < NumSamples;++SampleIndex)
	{
		// Resolve the sample to the non-MSAA render target.
		ResolveTextureUsingShader<FResolveSingleSamplePS>(
			this,
			Texture,
			NULL,
			NonMSAARTV,
			NULL,
			NonMSAADesc,
			FResolveRect(InRect.Min.X, InRect.Min.Y, InRect.Max.X, InRect.Max.Y),
			FResolveRect(0,0,SizeX,SizeY),
			SampleIndex
			);

		// Copy the resolved sample data to the staging texture.
		Direct3DDeviceIMContext->CopySubresourceRegion(StagingTexture2D,0,0,0,0,NonMSAATexture2D,Subresource,&Rect);

		// Lock the staging texture.
		D3D11_MAPPED_SUBRESOURCE LockedRect;
		VERIFYD3D11RESULT_EX(Direct3DDeviceIMContext->Map(StagingTexture2D,0,D3D11_MAP_READ,0,&LockedRect), Direct3DDevice);

		// Read the data out of the buffer, could be optimized
		for(int32 Y = InRect.Min.Y; Y < InRect.Max.Y; Y++)
		{
			uint8* SrcPtr = (uint8*)LockedRect.pData + (Y - InRect.Min.Y) * LockedRect.RowPitch + InRect.Min.X * BytesPerPixel;
			uint8* DestPtr = &OutData[(Y - InRect.Min.Y) * SizeX * NumSamples * BytesPerPixel + SampleIndex * BytesPerPixel];

			for(int32 X = InRect.Min.X; X < InRect.Max.X; X++)
			{
				for(uint32 i = 0; i < BytesPerPixel; ++i)
				{
					*DestPtr++ = *SrcPtr++;
				}

				DestPtr += (NumSamples - 1) * BytesPerPixel;
			}
		}

		Direct3DDeviceIMContext->Unmap(StagingTexture2D,0);
	}
}

void FD3D11DynamicRHI::RHIMapStagingSurface(FRHITexture* TextureRHI, FRHIGPUFence* FenceRHI, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex)
{
	ID3D11Resource* Resource = ResourceCast(TextureRHI)->GetResource();

	DXGI_FORMAT Format = (DXGI_FORMAT)GPixelFormats[TextureRHI->GetDesc().Format].PlatformFormat;
	uint32 BytesPerPixel = D3D11RT_ComputeBytesPerPixel(Format);

	D3D11_MAPPED_SUBRESOURCE LockedRect;
	VERIFYD3D11RESULT_EX(Direct3DDeviceIMContext->Map(Resource, 0, D3D11_MAP_READ, 0, &LockedRect), Direct3DDevice);

	OutData = LockedRect.pData;
	OutWidth = LockedRect.RowPitch / BytesPerPixel;
	OutHeight = LockedRect.DepthPitch / LockedRect.RowPitch;

	check(OutData);
}

void FD3D11DynamicRHI::RHIUnmapStagingSurface(FRHITexture* TextureRHI, uint32 GPUIndex)
{
	ID3D11Resource* Resource = ResourceCast(TextureRHI)->GetResource();
	Direct3DDeviceIMContext->Unmap(Resource, 0);
}

void FD3D11DynamicRHI::RHIReadSurfaceFloatData(FRHITexture* TextureRHI,FIntRect InRect,TArray<FFloat16Color>& OutData,ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex)
{
	FD3D11Texture* Texture = ResourceCast(TextureRHI);

	uint32 SizeX = InRect.Width();
	uint32 SizeY = InRect.Height();

	// Check the format of the surface
	D3D11_TEXTURE2D_DESC TextureDesc;
	Texture->GetD3D11Texture2D()->GetDesc(&TextureDesc);

	// only supports exactly RGBA16F textures
	if ( ! ensure(TextureDesc.Format == GPixelFormats[PF_FloatRGBA].PlatformFormat) )
	{
		checkf(0, TEXT("Unsupported surface format!"));
		OutData.Empty();
		return;
	}

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY);

	// Read back the surface data from defined rect
	D3D11_BOX	Rect;
	Rect.left	= InRect.Min.X;
	Rect.top	= InRect.Min.Y;
	Rect.right	= InRect.Max.X;
	Rect.bottom	= InRect.Max.Y;
	Rect.back = 1;
	Rect.front = 0;

	// create a temp 2d texture to copy render target to
	D3D11_TEXTURE2D_DESC Desc;
	ZeroMemory( &Desc, sizeof( D3D11_TEXTURE2D_DESC ) );
	Desc.Width = SizeX;
	Desc.Height = SizeY;
	Desc.MipLevels = 1;
	Desc.ArraySize = 1;
	Desc.Format = TextureDesc.Format;
	Desc.SampleDesc.Count = 1;
	Desc.SampleDesc.Quality = 0;
	Desc.Usage = D3D11_USAGE_STAGING;
	Desc.BindFlags = 0;
	Desc.CPUAccessFlags = D3D11_CPU_ACCESS_READ;
	Desc.MiscFlags = 0;
	TRefCountPtr<ID3D11Texture2D> TempTexture2D;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateTexture2D(&Desc,NULL,TempTexture2D.GetInitReference()), Direct3DDevice);

	// Copy the data to a staging resource.
	uint32 Subresource = 0;
	if( TextureDesc.MiscFlags == D3D11_RESOURCE_MISC_TEXTURECUBE )
	{
		uint32 D3DFace = GetD3D11CubeFace(CubeFace);
		Subresource = D3D11CalcSubresource(MipIndex, ArrayIndex * 6 + D3DFace, TextureDesc.MipLevels);
	}
	else
	{
		const bool bIsTextureArray = TextureRHI->GetTexture2DArray() != nullptr;
		Subresource = D3D11CalcSubresource(MipIndex, bIsTextureArray ? ArrayIndex : 0, TextureDesc.MipLevels);
	}
	Direct3DDeviceIMContext->CopySubresourceRegion(TempTexture2D,0,0,0,0,Texture->GetResource(),Subresource,&Rect);

	// Lock the staging resource.
	D3D11_MAPPED_SUBRESOURCE LockedRect;
	VERIFYD3D11RESULT_EX(Direct3DDeviceIMContext->Map(TempTexture2D,0,D3D11_MAP_READ,0,&LockedRect), Direct3DDevice);

	for(int32 Y = InRect.Min.Y; Y < InRect.Max.Y; Y++)
	{
		FFloat16Color* SrcPtr = (FFloat16Color*)((uint8*)LockedRect.pData + (Y - InRect.Min.Y) * LockedRect.RowPitch);
		int32 Index = (Y - InRect.Min.Y) * SizeX;
		check(Index + ((int32)SizeX - 1) < OutData.Num());
		FFloat16Color* DestColor = &OutData[Index];
		FFloat16* DestPtr = (FFloat16*)(DestColor);
		FMemory::Memcpy(DestPtr,SrcPtr,SizeX * sizeof(FFloat16) * 4);
	}

	Direct3DDeviceIMContext->Unmap(TempTexture2D,0);
}

void FD3D11DynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	TArray<uint8> OutDataRaw;

	FD3D11Texture* Texture = ResourceCast(TextureRHI);

	// Check the format of the surface
	D3D11_TEXTURE2D_DESC TextureDesc;
	Texture->GetD3D11Texture2D()->GetDesc(&TextureDesc);

	check(TextureDesc.SampleDesc.Count >= 1);

	if (TextureDesc.SampleDesc.Count == 1)
	{
		ReadSurfaceDataNoMSAARaw(TextureRHI, InRect, OutDataRaw, InFlags);
	}
	else
	{
		ReadSurfaceDataMSAARaw(TextureRHI, InRect, OutDataRaw, InFlags);
	}

	const uint32 SizeX = InRect.Width() * TextureDesc.SampleDesc.Count;
	const uint32 SizeY = InRect.Height();

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY);

	uint32 BytesPerPixel = D3D11RT_ComputeBytesPerPixel(TextureDesc.Format);
	uint32 SrcPitch = SizeX * BytesPerPixel;
	EPixelFormat Format = TextureRHI->GetFormat();

	check( ConvertTypelessToUnorm((DXGI_FORMAT)GPixelFormats[Format].PlatformFormat) == ConvertTypelessToUnorm(TextureDesc.Format) );

	// switching on the EPixelFormat is risky if the mapping is not what you expect
	//	verify against TextureDesc.Format

	check( GPixelFormats[Format].BlockBytes == D3D11RT_ComputeBytesPerPixel(TextureDesc.Format) );

	if ( ! ConvertRAWSurfaceDataToFLinearColor(Format, SizeX, SizeY, OutDataRaw.GetData(), SrcPitch, OutData.GetData(), InFlags) )
	{
		checkf(0, TEXT("Unsupported surface format!"));
		OutData.Empty();
	}
}

void FD3D11DynamicRHI::RHIRead3DSurfaceFloatData(FRHITexture* TextureRHI,FIntRect InRect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData)
{
	FD3D11Texture* Texture = ResourceCast(TextureRHI);

	uint32 SizeX = InRect.Width();
	uint32 SizeY = InRect.Height();
	uint32 SizeZ = ZMinMax.Y - ZMinMax.X;

	// Check the format of the surface
	D3D11_TEXTURE3D_DESC TextureDesc;
	Texture->GetD3D11Texture3D()->GetDesc(&TextureDesc);

	bool bIsRGBAFmt = TextureDesc.Format == GPixelFormats[PF_FloatRGBA].PlatformFormat;
	bool bIsR16FFmt = TextureDesc.Format == GPixelFormats[PF_R16F].PlatformFormat;	
	bool bIsR32FFmt = TextureDesc.Format == GPixelFormats[PF_R32_FLOAT].PlatformFormat;
	if ( ! ensure(bIsRGBAFmt || bIsR16FFmt || bIsR32FFmt) )
	{
		OutData.Empty();
		return;
	}

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY * SizeZ);

	// Read back the surface data from defined rect
	D3D11_BOX	Rect;
	Rect.left	= InRect.Min.X;
	Rect.top	= InRect.Min.Y;
	Rect.right	= InRect.Max.X;
	Rect.bottom	= InRect.Max.Y;
	Rect.back = ZMinMax.Y;
	Rect.front = ZMinMax.X;

	// create a temp 2d texture to copy render target to
	D3D11_TEXTURE3D_DESC Desc;
	ZeroMemory( &Desc, sizeof( D3D11_TEXTURE3D_DESC ) );
	Desc.Width = SizeX;
	Desc.Height = SizeY;
	Desc.Depth = SizeZ;
	Desc.MipLevels = 1;
	Desc.Format = TextureDesc.Format;
	Desc.Usage = D3D11_USAGE_STAGING;
	Desc.BindFlags = 0;
	Desc.CPUAccessFlags = D3D11_CPU_ACCESS_READ;
	Desc.MiscFlags = 0;
	TRefCountPtr<ID3D11Texture3D> TempTexture3D;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateTexture3D(&Desc,NULL,TempTexture3D.GetInitReference()), Direct3DDevice);

	// Copy the data to a staging resource.
	uint32 Subresource = 0;
	Direct3DDeviceIMContext->CopySubresourceRegion(TempTexture3D,0,0,0,0,Texture->GetResource(),Subresource,&Rect);

	// Lock the staging resource.
	D3D11_MAPPED_SUBRESOURCE LockedRect;
	VERIFYD3D11RESULT_EX(Direct3DDeviceIMContext->Map(TempTexture3D,0,D3D11_MAP_READ,0,&LockedRect), Direct3DDevice);

	// Read the data out of the buffer
	if (bIsRGBAFmt)
	{
		// Texture data is RGBA16F
		for (int32 Z = ZMinMax.X; Z < ZMinMax.Y; ++Z)
		{
			for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; ++Y)
			{
				const FFloat16Color* SrcPtr = (const FFloat16Color*)((const uint8*)LockedRect.pData + (Y - InRect.Min.Y) * LockedRect.RowPitch + (Z - ZMinMax.X) * LockedRect.DepthPitch);
				int32 Index = (Y - InRect.Min.Y) * SizeX + (Z - ZMinMax.X) * SizeX * SizeY;
				check(Index < OutData.Num());
				FFloat16Color* DestPtr = &OutData[Index];
				FMemory::Memcpy(DestPtr, SrcPtr, SizeX * sizeof(FFloat16Color));
			}
		}
	}
	else if (bIsR16FFmt)
	{
		// Texture data is R16F
		for (int32 Z = ZMinMax.X; Z < ZMinMax.Y; ++Z)
		{
			for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; ++Y)
			{
				const FFloat16* SrcPtr = (const FFloat16*)((const uint8*)LockedRect.pData + (Y - InRect.Min.Y) * LockedRect.RowPitch + (Z - ZMinMax.X) * LockedRect.DepthPitch);
				for (int32 X = InRect.Min.X; X < InRect.Max.X; ++X)
				{
					int32 Index = (Y - InRect.Min.Y) * SizeX + (Z - ZMinMax.X) * SizeX * SizeY + X;
					check(Index < OutData.Num());
					OutData[Index].R = SrcPtr[X];
					OutData[Index].A = FFloat16(1.0f); // ensure full alpha (as if you sampled on GPU)
				}
			}
		}
	}
	else if (bIsR32FFmt)
	{
		// Texture data is R32F
		for (int32 Z = ZMinMax.X; Z < ZMinMax.Y; ++Z)
		{
			for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; ++Y)
			{
				const float* SrcPtr = (const float*)((const uint8*)LockedRect.pData + (Y - InRect.Min.Y) * LockedRect.RowPitch + (Z - ZMinMax.X) * LockedRect.DepthPitch);
				for (int32 X = InRect.Min.X; X < InRect.Max.X; ++X)
				{
					int32 Index = (Y - InRect.Min.Y) * SizeX + (Z - ZMinMax.X) * SizeX * SizeY + X;
					check(Index < OutData.Num());
					OutData[Index].R = FFloat16(SrcPtr[X]);
					OutData[Index].A = FFloat16(1.0f);
				}
			}
		}
	}
	else
	{
		// unsupported format; checked for this earlier
		check(0);
	}

	Direct3DDeviceIMContext->Unmap(TempTexture3D,0);
}

void FD3D11DynamicRHI::RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName)
{
	FRHISetRenderTargetsInfo RTInfo;
	InInfo.ConvertToRenderTargetsInfo(RTInfo);
	SetRenderTargetsAndClear(RTInfo);

	RenderPassInfo = InInfo;
}

void FD3D11DynamicRHI::RHIEndRenderPass()
{
	UE::RHICore::ResolveRenderPassTargets(RenderPassInfo, [this](UE::RHICore::FResolveTextureInfo Info)
	{
		ResolveTexture(Info);
	});

	FRHIRenderTargetView RTV(nullptr, ERenderTargetLoadAction::ENoAction);
	FRHIDepthRenderTargetView DepthRTV(nullptr, ERenderTargetLoadAction::ENoAction, ERenderTargetStoreAction::ENoAction);
	SetRenderTargets(1, &RTV, &DepthRTV);
}

void FD3D11DynamicRHI::ResolveTexture(UE::RHICore::FResolveTextureInfo Info)
{
	FD3D11Texture* SourceTexture      = ResourceCast(Info.SourceTexture);
	const FRHITextureDesc& SourceDesc = SourceTexture->GetDesc();

	FD3D11Texture* DestTexture        = ResourceCast(Info.DestTexture);
	const FRHITextureDesc& DestDesc   = DestTexture->GetDesc();

	if (SourceDesc.Format == PF_DepthStencil)
	{
		D3D11_TEXTURE2D_DESC ResolveTargetDesc;
		DestTexture->GetD3D11Texture2D()->GetDesc(&ResolveTargetDesc);

		ResolveTextureUsingShader<FResolveDepthPS>(
			this,
			SourceTexture,
			DestTexture,
			DestTexture->GetRenderTargetView(0, -1),
			DestTexture->GetDepthStencilView(FExclusiveDepthStencil::DepthWrite_StencilWrite),
			ResolveTargetDesc,
			GetDefaultRect(Info.ResolveRect, SourceDesc.Extent.X, SourceDesc.Extent.Y),
			GetDefaultRect(Info.ResolveRect, DestDesc.Extent.X, DestDesc.Extent.Y),
			FDummyResolveParameter()
		);
	}
	else
	{
		const DXGI_FORMAT DestFormatTypeless = ConvertTypelessToUnorm((DXGI_FORMAT)GPixelFormats[DestDesc.Format].PlatformFormat);

		int32 ArraySliceBegin = Info.ArraySlice;
		int32 ArraySliceEnd   = Info.ArraySlice + 1;

		if (Info.ArraySlice < 0)
		{
			ArraySliceBegin = 0;
			ArraySliceEnd   = SourceDesc.ArraySize;
		}

		for (int32 ArraySlice = ArraySliceBegin; ArraySlice < ArraySliceEnd; ArraySlice++)
		{
			int32 DestSubresource   = D3D11CalcSubresource(Info.MipLevel, ArraySlice, DestDesc.NumMips);
			int32 SourceSubresource = D3D11CalcSubresource(Info.MipLevel, ArraySlice, SourceDesc.NumMips);

			Direct3DDeviceIMContext->ResolveSubresource(DestTexture->GetResource(), DestSubresource, SourceTexture->GetResource(), SourceSubresource, DestFormatTypeless);
		}
	}
}