// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D11RHIPrivate.h"

void FD3D11ViewableResource::UpdateLinkedViews()
{
	for (FD3D11View* LinkedView = LinkedViews; LinkedView; LinkedView = LinkedView->Next())
	{
		LinkedView->UpdateView();
	}
}

// -----------------------------------------------------------------------------------------------------
//
//                                       FD3D11ShaderResourceView                                       
//
// -----------------------------------------------------------------------------------------------------

FD3D11ShaderResourceView::FD3D11ShaderResourceView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
	: FRHIShaderResourceView(Resource, ViewDesc)
{
	RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		UpdateView();
	});
}

void FD3D11ShaderResourceView::UpdateView()
{
	ID3D11Resource* D3DResource = nullptr;
	D3D11_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};

	if (IsBuffer())
	{
		FD3D11Buffer* Buffer = FD3D11DynamicRHI::ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.SRV.GetViewInfo(Buffer);

		if (!Info.bNullView)
		{
			D3DResource = Buffer->Resource;
			SRVDesc.Format = UE::DXGIUtilities::FindShaderResourceFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat), false);

			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Typed:
			case FRHIViewDesc::EBufferType::Structured:
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_BUFFER;
				SRVDesc.Buffer.FirstElement = Info.OffsetInBytes / Info.StrideInBytes;
				SRVDesc.Buffer.NumElements  = Info.NumElements;
				break;

			case FRHIViewDesc::EBufferType::Raw:
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_BUFFEREX;
				SRVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
				SRVDesc.BufferEx.FirstElement = Info.OffsetInBytes / Info.StrideInBytes;
				SRVDesc.BufferEx.NumElements  = Info.NumElements;
				SRVDesc.BufferEx.Flags        = D3D11_BUFFEREX_SRV_FLAG_RAW;
				break;

			default:
				checkNoEntry(); // unsupported
				break;
			}
		}
	}
	else
	{
		FD3D11Texture* Texture = FD3D11DynamicRHI::ResourceCast(GetTexture());
		D3DResource = Texture->GetResource();

		FRHITextureDesc const& TextureDesc = Texture->GetDesc();
		auto const Info = ViewDesc.Texture.SRV.GetViewInfo(Texture);

		SRVDesc.Format = UE::DXGIUtilities::FindShaderResourceFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat), Info.bSRGB);

		FRHIViewDesc::EDimension ViewDimension = UE::RHICore::AdjustViewInfoDimensionForNarrowing(Info, TextureDesc);
		switch (ViewDimension)
		{
		case FRHIViewDesc::EDimension::Texture2D:
			if (TextureDesc.NumSamples > 1)
			{
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DMS;
			}
			else
			{
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2D;
				SRVDesc.Texture2D.MostDetailedMip = Info.MipRange.First;
				SRVDesc.Texture2D.MipLevels       = Info.MipRange.Num;
			}
			break;

		case FRHIViewDesc::EDimension::Texture2DArray:
			if (TextureDesc.NumSamples > 1)
			{
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DMSARRAY;
				SRVDesc.Texture2DMSArray.FirstArraySlice = Info.ArrayRange.First;
				SRVDesc.Texture2DMSArray.ArraySize       = Info.ArrayRange.Num;
			}
			else
			{
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DARRAY;
				SRVDesc.Texture2DArray.FirstArraySlice = Info.ArrayRange.First;
				SRVDesc.Texture2DArray.ArraySize       = Info.ArrayRange.Num;
				SRVDesc.Texture2DArray.MostDetailedMip = Info.MipRange.First;
				SRVDesc.Texture2DArray.MipLevels       = Info.MipRange.Num;
			}
			break;

		case FRHIViewDesc::EDimension::Texture3D:
			SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE3D;
			SRVDesc.Texture3D.MostDetailedMip = Info.MipRange.First;
			SRVDesc.Texture3D.MipLevels       = Info.MipRange.Num;
			break;

		case FRHIViewDesc::EDimension::TextureCube:
			SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURECUBE;
			SRVDesc.TextureCube.MostDetailedMip = Info.MipRange.First;
			SRVDesc.TextureCube.MipLevels       = Info.MipRange.Num;
			break;

		case FRHIViewDesc::EDimension::TextureCubeArray:
			SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURECUBEARRAY;
			SRVDesc.TextureCubeArray.MostDetailedMip  = Info.MipRange.First;
			SRVDesc.TextureCubeArray.MipLevels        = Info.MipRange.Num;
			SRVDesc.TextureCubeArray.First2DArrayFace = Info.ArrayRange.First * 6;
			SRVDesc.TextureCubeArray.NumCubes         = Info.ArrayRange.Num;
			break;

		default:
			checkNoEntry();
			break;
		}
	}

	View = nullptr;
	if (D3DResource)
	{
		ID3D11Device* Device = FD3D11DynamicRHI::Get().GetDevice();
		VERIFYD3D11RESULT_EX(Device->CreateShaderResourceView(D3DResource, &SRVDesc, View.GetInitReference()), Device);
	}
}

FD3D11ViewableResource* FD3D11ShaderResourceView::GetBaseResource() const
{
	return IsBuffer()
		? static_cast<FD3D11ViewableResource*>(FD3D11DynamicRHI::ResourceCast(GetBuffer()))
		: static_cast<FD3D11ViewableResource*>(FD3D11DynamicRHI::ResourceCast(GetTexture()));
}



// -----------------------------------------------------------------------------------------------------
//
//                                            RHI Functions                                             
//
// -----------------------------------------------------------------------------------------------------

FShaderResourceViewRHIRef FD3D11DynamicRHI::RHICreateShaderResourceView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	return new FD3D11ShaderResourceView(RHICmdList, Resource, ViewDesc);
}
