// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Shaders.cpp: D3D shader RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "Serialization/MemoryReader.h"
#include "RHICoreShader.h"

#if WITH_NVAPI
	#include "nvapi.h"
#endif

template <typename TShaderType>
static inline void ReadShaderOptionalData(FShaderCodeReader& InShaderCode, TShaderType& OutShader)
{
	auto PackedResourceCounts = InShaderCode.FindOptionalData<FShaderCodePackedResourceCounts>();
	check(PackedResourceCounts);

	OutShader.UAVMask = 0;
	if (auto ResourceMasks = InShaderCode.FindOptionalData<FShaderCodeResourceMasks>())
	{
		OutShader.UAVMask = ResourceMasks->UAVMask;
	}
	
	OutShader.bShaderNeedsGlobalConstantBuffer = EnumHasAnyFlags(PackedResourceCounts->UsageFlags, EShaderResourceUsageFlags::GlobalUniformBuffer);
#if RHI_INCLUDE_SHADER_DEBUG_DATA
	OutShader.Debug.ShaderName = InShaderCode.FindOptionalData(FShaderCodeName::Key);

	int32 UniformBufferTableSize = 0;
	const uint8* UniformBufferData = InShaderCode.FindOptionalDataAndSize(FShaderCodeUniformBuffers::Key, UniformBufferTableSize);
	if (UniformBufferData && UniformBufferTableSize > 0)
	{
		FBufferReader UBReader((void*)UniformBufferData, UniformBufferTableSize, false);
		TArray<FString> Names;
		UBReader << Names;
		check(OutShader.Debug.UniformBufferNames.Num() == 0);
		for (int32 Index = 0; Index < Names.Num(); ++Index)
		{
			OutShader.Debug.UniformBufferNames.Add(FName(*Names[Index]));
		}
	}
#endif
	int32 VendorExtensionTableSize = 0;
	auto* VendorExtensionData = InShaderCode.FindOptionalDataAndSize(FShaderCodeVendorExtension::Key, VendorExtensionTableSize);
	if (VendorExtensionData && VendorExtensionTableSize > 0)
	{
		FBufferReader Ar((void*)VendorExtensionData, VendorExtensionTableSize, false);
		Ar << OutShader.VendorExtensions;
	}
	OutShader.bShaderNeedsGlobalConstantBuffer = EnumHasAnyFlags(PackedResourceCounts->UsageFlags, EShaderResourceUsageFlags::GlobalUniformBuffer);

	int32 IsSm6ShaderSize = 1;
	const uint8* IsSm6Shader = InShaderCode.FindOptionalData(EShaderOptionalDataKey::ShaderModel6, IsSm6ShaderSize);
	OutShader.bIsSm6Shader = IsSm6Shader && IsSm6ShaderSize && *IsSm6Shader;

	UE::RHICore::SetupShaderCodeValidationData(&OutShader, InShaderCode);
}

static bool ApplyVendorExtensions(ID3D11Device* Direct3DDevice, EShaderFrequency Frequency, const FD3D11ShaderData* ShaderData, bool& OutNeedsReset)
{
	if (ShaderData->bIsSm6Shader)
	{
		return false;
	}

	bool IsValidHardwareExtension = true;

	for (const FShaderCodeVendorExtension& Extension : ShaderData->VendorExtensions)
	{
		if (Extension.VendorId == EGpuVendorId::Nvidia)
		{
			if (!IsRHIDeviceNVIDIA())
			{
				IsValidHardwareExtension = false;
				break;
			}

#if WITH_NVAPI
			// https://developer.nvidia.com/unlocking-gpu-intrinsics-hlsl
			if (Extension.Parameter.Type == EShaderParameterType::UAV)
			{
				NvAPI_D3D11_SetNvShaderExtnSlot(Direct3DDevice, Extension.Parameter.BaseIndex);
				OutNeedsReset = true;
			}
#endif
		}
		else if (Extension.VendorId == EGpuVendorId::Amd)
		{
			if (!IsRHIDeviceAMD())
			{
				IsValidHardwareExtension = false;
				break;
			}
			// TODO: https://github.com/GPUOpen-LibrariesAndSDKs/AGS_SDK/blob/master/ags_lib/hlsl/ags_shader_intrinsics_dx11.hlsl
		}
		else if (Extension.VendorId == EGpuVendorId::Intel)
		{
			if (!IsRHIDeviceIntel())
			{
				IsValidHardwareExtension = false;
				break;
			}
			// TODO: https://github.com/intel/intel-graphics-compiler/blob/master/inc/IntelExtensions.hlsl
		}
	}

	return IsValidHardwareExtension;
}

static void ResetVendorExtensions(ID3D11Device* Direct3DDevice)
{
#if WITH_NVAPI
	if (IsRHIDeviceNVIDIA())
	{
		NvAPI_D3D11_SetNvShaderExtnSlot(Direct3DDevice, ~uint32(0));
	}
#endif
}

FVertexShaderRHIRef FD3D11DynamicRHI::RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	FShaderCodeReader ShaderCode(CreateShaderDesc.Code);
	FD3D11VertexShader* Shader = new FD3D11VertexShader;

	FMemoryReaderView Ar( CreateShaderDesc.Code, true );
	Shader->SerializeShaderResourceTable(Ar);
	int32 Offset = Ar.Tell();

	TArrayView<const uint8> ActualCode = ShaderCode.GetOffsetShaderCode(Offset);

	ReadShaderOptionalData(ShaderCode, *Shader);

	bool bNeedsReset = false;
	if (ApplyVendorExtensions(Direct3DDevice, SF_Vertex, Shader, bNeedsReset))
	{
		VERIFYD3D11SHADERRESULT(Direct3DDevice->CreateVertexShader(ActualCode.GetData(), ActualCode.Num(), nullptr, Shader->Resource.GetInitReference()), Shader, Direct3DDevice);
		if (bNeedsReset)
		{
			ResetVendorExtensions(Direct3DDevice);
		}
		UE::RHICore::InitStaticUniformBufferSlots(Shader);
	}
	
	// TEMP
	Shader->Code = CreateShaderDesc.Code;
	Shader->Offset = Offset;

	return Shader;
}

FGeometryShaderRHIRef FD3D11DynamicRHI::RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc)
{ 
	FShaderCodeReader ShaderCode(CreateShaderDesc.Code);
	FD3D11GeometryShader* Shader = new FD3D11GeometryShader;

	FMemoryReaderView Ar( CreateShaderDesc.Code, true );
	Shader->SerializeShaderResourceTable(Ar);
	int32 Offset = Ar.Tell();

	TArrayView<const uint8> ActualCode = ShaderCode.GetOffsetShaderCode(Offset);

	ReadShaderOptionalData(ShaderCode, *Shader);

	bool bNeedsReset = false;
	if (ApplyVendorExtensions(Direct3DDevice, SF_Geometry, Shader, bNeedsReset))
	{
		VERIFYD3D11SHADERRESULT(Direct3DDevice->CreateGeometryShader(ActualCode.GetData(), ActualCode.Num(), nullptr, Shader->Resource.GetInitReference()), Shader, Direct3DDevice);
		if (bNeedsReset)
		{
			ResetVendorExtensions(Direct3DDevice);
		}
		UE::RHICore::InitStaticUniformBufferSlots(Shader);
	}

	return Shader;
}

FPixelShaderRHIRef FD3D11DynamicRHI::RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	FShaderCodeReader ShaderCode(CreateShaderDesc.Code);
	FD3D11PixelShader* Shader = new FD3D11PixelShader;

	FMemoryReaderView Ar( CreateShaderDesc.Code, true );
	Shader->SerializeShaderResourceTable(Ar);
	int32 Offset = Ar.Tell();

	TArrayView<const uint8> ActualCode = ShaderCode.GetOffsetShaderCode(Offset);

	ReadShaderOptionalData(ShaderCode, *Shader);

	bool bNeedsReset = false;
	if (ApplyVendorExtensions(Direct3DDevice, SF_Pixel, Shader, bNeedsReset))
	{
		VERIFYD3D11SHADERRESULT(Direct3DDevice->CreatePixelShader(ActualCode.GetData(), ActualCode.Num(), nullptr, Shader->Resource.GetInitReference()), Shader, Direct3DDevice);
		if (bNeedsReset)
		{
			ResetVendorExtensions(Direct3DDevice);
		}
		UE::RHICore::InitStaticUniformBufferSlots(Shader);
	}

	return Shader;
}

FComputeShaderRHIRef FD3D11DynamicRHI::RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc)
{ 
	FShaderCodeReader ShaderCode(CreateShaderDesc.Code);
	FD3D11ComputeShader* Shader = new FD3D11ComputeShader;

	FMemoryReaderView Ar( CreateShaderDesc.Code, true );
	Shader->SerializeShaderResourceTable(Ar);
	int32 Offset = Ar.Tell();

	TArrayView<const uint8> ActualCode = ShaderCode.GetOffsetShaderCode(Offset);

	ReadShaderOptionalData(ShaderCode, *Shader);

	bool bNeedsReset = false;
	if (ApplyVendorExtensions(Direct3DDevice, SF_Compute, Shader, bNeedsReset))
	{
		VERIFYD3D11SHADERRESULT(Direct3DDevice->CreateComputeShader(ActualCode.GetData(), ActualCode.Num(), nullptr, Shader->Resource.GetInitReference()), Shader, Direct3DDevice);
		if (bNeedsReset)
		{
			ResetVendorExtensions(Direct3DDevice);
		}
		UE::RHICore::InitStaticUniformBufferSlots(Shader);
	}

	return Shader;
}

void FD3D11DynamicRHI::RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) 
{ 
	check(Count > 0);
	check(Data);

	// structures are chosen to be directly mappable
	D3D11_VIEWPORT* D3DData = (D3D11_VIEWPORT*)Data;

	StateCache.SetViewports(Count, D3DData);
}

FD3D11BoundShaderState::FD3D11BoundShaderState(
	FRHIVertexDeclaration* InVertexDeclarationRHI,
	FRHIVertexShader* InVertexShaderRHI,
	FRHIPixelShader* InPixelShaderRHI,
	FRHIGeometryShader* InGeometryShaderRHI,
	ID3D11Device* Direct3DDevice
	):
	CacheLink(InVertexDeclarationRHI,InVertexShaderRHI,InPixelShaderRHI,InGeometryShaderRHI,this)
{
	INC_DWORD_STAT(STAT_D3D11NumBoundShaderState);

	FD3D11VertexDeclaration* InVertexDeclaration = FD3D11DynamicRHI::ResourceCast(InVertexDeclarationRHI);
	FD3D11VertexShader* InVertexShader = FD3D11DynamicRHI::ResourceCast(InVertexShaderRHI);
	FD3D11PixelShader* InPixelShader = FD3D11DynamicRHI::ResourceCast(InPixelShaderRHI);
	FD3D11GeometryShader* InGeometryShader = FD3D11DynamicRHI::ResourceCast(InGeometryShaderRHI);

	// Create an input layout for this combination of vertex declaration and vertex shader.
	D3D11_INPUT_ELEMENT_DESC NullInputElement;
	FMemory::Memzero(&NullInputElement,sizeof(D3D11_INPUT_ELEMENT_DESC));

	FShaderCodeReader VertexShaderCode(InVertexShader->Code);

	if (InVertexDeclaration == nullptr)
	{
		InputLayout = nullptr;
	}
	else
	{
		FMemory::Memcpy(StreamStrides, InVertexDeclaration->StreamStrides, sizeof(StreamStrides));

		VERIFYD3D11RESULT_EX(
		Direct3DDevice->CreateInputLayout(
			InVertexDeclaration && InVertexDeclaration->VertexElements.Num() ? InVertexDeclaration->VertexElements.GetData() : &NullInputElement,
			InVertexDeclaration ? InVertexDeclaration->VertexElements.Num() : 0,
			&InVertexShader->Code[ InVertexShader->Offset ],			// TEMP ugly
			VertexShaderCode.GetActualShaderCodeSize() - InVertexShader->Offset,
			InputLayout.GetInitReference()
			),
		Direct3DDevice
		);
	}

	VertexShader = InVertexShader->Resource;
	PixelShader = InPixelShader ? InPixelShader->Resource : nullptr;
	GeometryShader = InGeometryShader ? InGeometryShader->Resource : nullptr;

	FMemory::Memzero(&bShaderNeedsGlobalConstantBuffer,sizeof(bShaderNeedsGlobalConstantBuffer));

	bShaderNeedsGlobalConstantBuffer[SF_Vertex] = InVertexShader->bShaderNeedsGlobalConstantBuffer;
	bShaderNeedsGlobalConstantBuffer[SF_Pixel] = InPixelShader ? InPixelShader->bShaderNeedsGlobalConstantBuffer : false;
	bShaderNeedsGlobalConstantBuffer[SF_Geometry] = InGeometryShader ? InGeometryShader->bShaderNeedsGlobalConstantBuffer : false;

	static_assert(UE_ARRAY_COUNT(bShaderNeedsGlobalConstantBuffer) == SF_NumStandardFrequencies, "EShaderFrequency size should match with array count of bShaderNeedsGlobalConstantBuffer.");
}

FD3D11BoundShaderState::~FD3D11BoundShaderState()
{
	DEC_DWORD_STAT(STAT_D3D11NumBoundShaderState);
}

/**
* Creates a bound shader state instance which encapsulates a decl, vertex shader, and pixel shader
* @param VertexDeclaration - existing vertex decl
* @param StreamStrides - optional stream strides
* @param VertexShader - existing vertex shader
* @param PixelShader - existing pixel shader
* @param GeometryShader - existing geometry shader
*/
FBoundShaderStateRHIRef FD3D11DynamicRHI::RHICreateBoundShaderState(
	FRHIVertexDeclaration* VertexDeclarationRHI,
	FRHIVertexShader* VertexShaderRHI,
	FRHIPixelShader* PixelShaderRHI,
	FRHIGeometryShader* GeometryShaderRHI
	)
{
	check(IsInRenderingThread() || IsInRHIThread());

	SCOPE_CYCLE_COUNTER(STAT_D3D11CreateBoundShaderStateTime);

	checkf(GIsRHIInitialized && Direct3DDeviceIMContext,(TEXT("Bound shader state RHI resource was created without initializing Direct3D first")));

	// Check for an existing bound shader state which matches the parameters
	FCachedBoundShaderStateLink* CachedBoundShaderStateLink = GetCachedBoundShaderState(
		VertexDeclarationRHI,
		VertexShaderRHI,
		PixelShaderRHI,
		GeometryShaderRHI
		);
	if(CachedBoundShaderStateLink)
	{
		// If we've already created a bound shader state with these parameters, reuse it.
		return CachedBoundShaderStateLink->BoundShaderState;
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_D3D11NewBoundShaderStateTime);
		return new FD3D11BoundShaderState(VertexDeclarationRHI,VertexShaderRHI,PixelShaderRHI,GeometryShaderRHI,Direct3DDevice);
	}
}
