// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11State.cpp: D3D state implementation.
=============================================================================*/

#include "D3D11State.h"
#include "D3D11RHIPrivate.h"
#include "RHIUtilities.h"

// Cache of Sampler States; we store pointers to both as we don't want the TMap to be artificially
// modifying ref counts if not needed; so we manage that ourselves
static TMap<ID3D11SamplerState*, FD3D11SamplerState*> GSamplerStateCache;
static FCriticalSection GSamplerStateCacheCS;


void EmptyD3DSamplerStateCache()
{
	FScopeLock Lock(&GSamplerStateCacheCS);

	for (auto Iter = GSamplerStateCache.CreateIterator(); Iter; ++Iter )
	{
		auto* State = Iter.Value();
		// Manually release
		State->Release();
	}

	GSamplerStateCache.Empty();
}

static D3D11_TEXTURE_ADDRESS_MODE TranslateAddressMode(ESamplerAddressMode AddressMode)
{
	switch(AddressMode)
	{
	case AM_Clamp: return D3D11_TEXTURE_ADDRESS_CLAMP;
	case AM_Mirror: return D3D11_TEXTURE_ADDRESS_MIRROR;
	case AM_Border: return D3D11_TEXTURE_ADDRESS_BORDER;
	default: return D3D11_TEXTURE_ADDRESS_WRAP;
	};
}

static D3D11_CULL_MODE TranslateCullMode(ERasterizerCullMode CullMode)
{
	switch(CullMode)
	{
	case CM_CW: return D3D11_CULL_BACK;
	case CM_CCW: return D3D11_CULL_FRONT;
	default: return D3D11_CULL_NONE;
	};
}

static D3D11_FILL_MODE TranslateFillMode(ERasterizerFillMode FillMode)
{
	switch(FillMode)
	{
	case FM_Wireframe: return D3D11_FILL_WIREFRAME;
	default: return D3D11_FILL_SOLID;
	};
}

static D3D11_COMPARISON_FUNC TranslateCompareFunction(ECompareFunction CompareFunction)
{
	switch(CompareFunction)
	{
	case CF_Less: return D3D11_COMPARISON_LESS;
	case CF_LessEqual: return D3D11_COMPARISON_LESS_EQUAL;
	case CF_Greater: return D3D11_COMPARISON_GREATER;
	case CF_GreaterEqual: return D3D11_COMPARISON_GREATER_EQUAL;
	case CF_Equal: return D3D11_COMPARISON_EQUAL;
	case CF_NotEqual: return D3D11_COMPARISON_NOT_EQUAL;
	case CF_Never: return D3D11_COMPARISON_NEVER;
	default: return D3D11_COMPARISON_ALWAYS;
	};
}

static D3D11_COMPARISON_FUNC TranslateSamplerCompareFunction(ESamplerCompareFunction SamplerComparisonFunction)
{
	switch(SamplerComparisonFunction)
	{
	case SCF_Less: return D3D11_COMPARISON_LESS;
	case SCF_Never: 
	default: return D3D11_COMPARISON_NEVER;
	};
}

static D3D11_STENCIL_OP TranslateStencilOp(EStencilOp StencilOp)
{
	switch(StencilOp)
	{
	case SO_Zero: return D3D11_STENCIL_OP_ZERO;
	case SO_Replace: return D3D11_STENCIL_OP_REPLACE;
	case SO_SaturatedIncrement: return D3D11_STENCIL_OP_INCR_SAT;
	case SO_SaturatedDecrement: return D3D11_STENCIL_OP_DECR_SAT;
	case SO_Invert: return D3D11_STENCIL_OP_INVERT;
	case SO_Increment: return D3D11_STENCIL_OP_INCR;
	case SO_Decrement: return D3D11_STENCIL_OP_DECR;
	default: return D3D11_STENCIL_OP_KEEP;
	};
}

static D3D11_BLEND_OP TranslateBlendOp(EBlendOperation BlendOp)
{
	switch(BlendOp)
	{
	case BO_Subtract: return D3D11_BLEND_OP_SUBTRACT;
	case BO_Min: return D3D11_BLEND_OP_MIN;
	case BO_Max: return D3D11_BLEND_OP_MAX;
	case BO_ReverseSubtract: return D3D11_BLEND_OP_REV_SUBTRACT;
	default: return D3D11_BLEND_OP_ADD;
	};
}

static D3D11_BLEND TranslateBlendFactor(EBlendFactor BlendFactor)
{
	switch(BlendFactor)
	{
	case BF_One: return D3D11_BLEND_ONE;
	case BF_SourceColor: return D3D11_BLEND_SRC_COLOR;
	case BF_InverseSourceColor: return D3D11_BLEND_INV_SRC_COLOR;
	case BF_SourceAlpha: return D3D11_BLEND_SRC_ALPHA;
	case BF_InverseSourceAlpha: return D3D11_BLEND_INV_SRC_ALPHA;
	case BF_DestAlpha: return D3D11_BLEND_DEST_ALPHA;
	case BF_InverseDestAlpha: return D3D11_BLEND_INV_DEST_ALPHA;
	case BF_DestColor: return D3D11_BLEND_DEST_COLOR;
	case BF_InverseDestColor: return D3D11_BLEND_INV_DEST_COLOR;
	case BF_ConstantBlendFactor: return D3D11_BLEND_BLEND_FACTOR;
	case BF_InverseConstantBlendFactor: return D3D11_BLEND_INV_BLEND_FACTOR;
	case BF_Source1Color: return D3D11_BLEND_SRC1_COLOR;
	case BF_InverseSource1Color: return D3D11_BLEND_INV_SRC1_COLOR;
	case BF_Source1Alpha: return D3D11_BLEND_SRC1_ALPHA;
	case BF_InverseSource1Alpha: return D3D11_BLEND_INV_SRC1_ALPHA;
	
	default: return D3D11_BLEND_ZERO;
	};
}

FSamplerStateRHIRef FD3D11DynamicRHI::RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer)
{
	D3D11_SAMPLER_DESC SamplerDesc;
	FMemory::Memzero(&SamplerDesc,sizeof(D3D11_SAMPLER_DESC));

	SamplerDesc.AddressU = TranslateAddressMode(Initializer.AddressU);
	SamplerDesc.AddressV = TranslateAddressMode(Initializer.AddressV);
	SamplerDesc.AddressW = TranslateAddressMode(Initializer.AddressW);
	SamplerDesc.MipLODBias = Initializer.MipBias;
	SamplerDesc.MaxAnisotropy = ComputeAnisotropyRT(Initializer.MaxAnisotropy);
	SamplerDesc.MinLOD = Initializer.MinMipLevel;
	SamplerDesc.MaxLOD = Initializer.MaxMipLevel;

	// Determine whether we should use one of the comparison modes
	const bool bComparisonEnabled = Initializer.SamplerComparisonFunction != SCF_Never;
	switch(Initializer.Filter)
	{
	case SF_AnisotropicLinear:
	case SF_AnisotropicPoint:
		if (SamplerDesc.MaxAnisotropy == 1)
		{
			SamplerDesc.Filter = bComparisonEnabled ? D3D11_FILTER_COMPARISON_MIN_MAG_MIP_LINEAR : D3D11_FILTER_MIN_MAG_MIP_LINEAR;
		}
		else
		{
			// D3D11 doesn't allow using point filtering for mip filter when using anisotropic filtering
			SamplerDesc.Filter = bComparisonEnabled ? D3D11_FILTER_COMPARISON_ANISOTROPIC : D3D11_FILTER_ANISOTROPIC;
		}
		
		break;
	case SF_Trilinear:
        SamplerDesc.Filter = bComparisonEnabled ? D3D11_FILTER_COMPARISON_MIN_MAG_MIP_LINEAR : D3D11_FILTER_MIN_MAG_MIP_LINEAR;
		break;
	case SF_Bilinear:
		SamplerDesc.Filter = bComparisonEnabled ? D3D11_FILTER_COMPARISON_MIN_MAG_LINEAR_MIP_POINT : D3D11_FILTER_MIN_MAG_LINEAR_MIP_POINT;
		break;
	case SF_Point:
		SamplerDesc.Filter = bComparisonEnabled ? D3D11_FILTER_COMPARISON_MIN_MAG_MIP_POINT : D3D11_FILTER_MIN_MAG_MIP_POINT;
		break;
	}
	const FLinearColor LinearBorderColor = FColor(Initializer.BorderColor);
	SamplerDesc.BorderColor[0] = LinearBorderColor.R;
	SamplerDesc.BorderColor[1] = LinearBorderColor.G;
	SamplerDesc.BorderColor[2] = LinearBorderColor.B;
	SamplerDesc.BorderColor[3] = LinearBorderColor.A;
	SamplerDesc.ComparisonFunc = TranslateSamplerCompareFunction(Initializer.SamplerComparisonFunction);

	// D3D11 will return the same pointer if the particular state description was already created
	TRefCountPtr<ID3D11SamplerState> SamplerStateHandle;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateSamplerState(&SamplerDesc, SamplerStateHandle.GetInitReference()), Direct3DDevice);

	FScopeLock Lock(&GSamplerStateCacheCS);
	FD3D11SamplerState** Found = GSamplerStateCache.Find(SamplerStateHandle);
	if (Found)
	{
		return *Found;
	}

	FD3D11SamplerState* SamplerState = new FD3D11SamplerState;
	SamplerState->Resource = SamplerStateHandle;
	// Manually add reference as we control the creation/destructions
	SamplerState->AddRef();
	GSamplerStateCache.Add(SamplerStateHandle.GetReference(), SamplerState);
	return SamplerState;
}

FRasterizerStateRHIRef FD3D11DynamicRHI::RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer)
{
	D3D11_RASTERIZER_DESC RasterizerDesc;
	FMemory::Memzero(&RasterizerDesc,sizeof(D3D11_RASTERIZER_DESC));

	RasterizerDesc.CullMode = TranslateCullMode(Initializer.CullMode);
	RasterizerDesc.FillMode = TranslateFillMode(Initializer.FillMode);
	RasterizerDesc.SlopeScaledDepthBias = Initializer.SlopeScaleDepthBias;
	RasterizerDesc.FrontCounterClockwise = true;
	RasterizerDesc.DepthBias = FMath::FloorToInt(Initializer.DepthBias * (float)(1 << 24));
	RasterizerDesc.DepthClipEnable = Initializer.DepthClipMode == ERasterizerDepthClipMode::DepthClip;
	RasterizerDesc.MultisampleEnable = Initializer.bAllowMSAA;
	RasterizerDesc.ScissorEnable = true;

	FD3D11RasterizerState* RasterizerState = new FD3D11RasterizerState;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateRasterizerState(&RasterizerDesc,RasterizerState->Resource.GetInitReference()), Direct3DDevice);
	return RasterizerState;
}

FDepthStencilStateRHIRef FD3D11DynamicRHI::RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer)
{
	FD3D11DepthStencilState* DepthStencilState = new FD3D11DepthStencilState;

	D3D11_DEPTH_STENCIL_DESC DepthStencilDesc;
	FMemory::Memzero(&DepthStencilDesc,sizeof(D3D11_DEPTH_STENCIL_DESC));

	// depth part
	DepthStencilDesc.DepthEnable = Initializer.DepthTest != CF_Always || Initializer.bEnableDepthWrite;
	DepthStencilDesc.DepthWriteMask = Initializer.bEnableDepthWrite ? D3D11_DEPTH_WRITE_MASK_ALL : D3D11_DEPTH_WRITE_MASK_ZERO;
	DepthStencilDesc.DepthFunc = TranslateCompareFunction(Initializer.DepthTest);

	// stencil part
	DepthStencilDesc.StencilEnable = Initializer.bEnableFrontFaceStencil || Initializer.bEnableBackFaceStencil;
	DepthStencilDesc.StencilReadMask = Initializer.StencilReadMask;
	DepthStencilDesc.StencilWriteMask = Initializer.StencilWriteMask;
	DepthStencilDesc.FrontFace.StencilFunc = TranslateCompareFunction(Initializer.FrontFaceStencilTest);
	DepthStencilDesc.FrontFace.StencilFailOp = TranslateStencilOp(Initializer.FrontFaceStencilFailStencilOp);
	DepthStencilDesc.FrontFace.StencilDepthFailOp = TranslateStencilOp(Initializer.FrontFaceDepthFailStencilOp);
	DepthStencilDesc.FrontFace.StencilPassOp = TranslateStencilOp(Initializer.FrontFacePassStencilOp);
	if( Initializer.bEnableBackFaceStencil )
	{
		DepthStencilDesc.BackFace.StencilFunc = TranslateCompareFunction(Initializer.BackFaceStencilTest);
		DepthStencilDesc.BackFace.StencilFailOp = TranslateStencilOp(Initializer.BackFaceStencilFailStencilOp);
		DepthStencilDesc.BackFace.StencilDepthFailOp = TranslateStencilOp(Initializer.BackFaceDepthFailStencilOp);
		DepthStencilDesc.BackFace.StencilPassOp = TranslateStencilOp(Initializer.BackFacePassStencilOp);
	}
	else
	{
		DepthStencilDesc.BackFace = DepthStencilDesc.FrontFace;
	}

	const bool bStencilOpIsKeep =
		   Initializer.FrontFaceStencilFailStencilOp == SO_Keep
		&& Initializer.FrontFaceDepthFailStencilOp == SO_Keep
		&& Initializer.FrontFacePassStencilOp == SO_Keep
		&& Initializer.BackFaceStencilFailStencilOp == SO_Keep
		&& Initializer.BackFaceDepthFailStencilOp == SO_Keep
		&& Initializer.BackFacePassStencilOp == SO_Keep;

	const bool bMayWriteStencil = Initializer.StencilWriteMask != 0 && !bStencilOpIsKeep;
	DepthStencilState->AccessType.SetDepthStencilWrite(Initializer.bEnableDepthWrite, bMayWriteStencil);

	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateDepthStencilState(&DepthStencilDesc,DepthStencilState->Resource.GetInitReference()), Direct3DDevice);
	return DepthStencilState;
}

FBlendStateRHIRef FD3D11DynamicRHI::RHICreateBlendState(const FBlendStateInitializerRHI& Initializer)
{
	D3D11_BLEND_DESC BlendDesc;
	FMemory::Memzero(&BlendDesc,sizeof(D3D11_BLEND_DESC));

	BlendDesc.AlphaToCoverageEnable = Initializer.bUseAlphaToCoverage;
	BlendDesc.IndependentBlendEnable = Initializer.bUseIndependentRenderTargetBlendStates;

	static_assert(MaxSimultaneousRenderTargets <= D3D11_SIMULTANEOUS_RENDER_TARGET_COUNT, "Too many MRTs.");
	for(uint32 RenderTargetIndex = 0;RenderTargetIndex < MaxSimultaneousRenderTargets;++RenderTargetIndex)
	{
		const FBlendStateInitializerRHI::FRenderTarget& RenderTargetInitializer = Initializer.RenderTargets[RenderTargetIndex];
		D3D11_RENDER_TARGET_BLEND_DESC& RenderTarget = BlendDesc.RenderTarget[RenderTargetIndex];
		RenderTarget.BlendEnable = 
			RenderTargetInitializer.ColorBlendOp != BO_Add || RenderTargetInitializer.ColorDestBlend != BF_Zero || RenderTargetInitializer.ColorSrcBlend != BF_One ||
			RenderTargetInitializer.AlphaBlendOp != BO_Add || RenderTargetInitializer.AlphaDestBlend != BF_Zero || RenderTargetInitializer.AlphaSrcBlend != BF_One;
		RenderTarget.BlendOp = TranslateBlendOp(RenderTargetInitializer.ColorBlendOp);
		RenderTarget.SrcBlend = TranslateBlendFactor(RenderTargetInitializer.ColorSrcBlend);
		RenderTarget.DestBlend = TranslateBlendFactor(RenderTargetInitializer.ColorDestBlend);
		RenderTarget.BlendOpAlpha = TranslateBlendOp(RenderTargetInitializer.AlphaBlendOp);
		RenderTarget.SrcBlendAlpha = TranslateBlendFactor(RenderTargetInitializer.AlphaSrcBlend);
		RenderTarget.DestBlendAlpha = TranslateBlendFactor(RenderTargetInitializer.AlphaDestBlend);
		RenderTarget.RenderTargetWriteMask = 
		  ((RenderTargetInitializer.ColorWriteMask & CW_RED)   ? D3D11_COLOR_WRITE_ENABLE_RED   : 0)
		| ((RenderTargetInitializer.ColorWriteMask & CW_GREEN) ? D3D11_COLOR_WRITE_ENABLE_GREEN : 0)
		| ((RenderTargetInitializer.ColorWriteMask & CW_BLUE)  ? D3D11_COLOR_WRITE_ENABLE_BLUE  : 0)
		| ((RenderTargetInitializer.ColorWriteMask & CW_ALPHA) ? D3D11_COLOR_WRITE_ENABLE_ALPHA : 0);
	}
	
	FD3D11BlendState* BlendState = new FD3D11BlendState;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateBlendState(&BlendDesc,BlendState->Resource.GetInitReference()), Direct3DDevice);
	return BlendState;
}
