// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of Device Context State Caching to improve draw
//	thread performance by removing redundant device context calls.

//-----------------------------------------------------------------------------
//	Include Files
//-----------------------------------------------------------------------------
#include "D3D11StateCache.h"
#include "D3D11RHIPrivate.h"
#include "D3D11StateCache.h"

#if D3D11_ALLOW_STATE_CACHE && D3D11_STATE_CACHE_RUNTIME_TOGGLE

// Default the state caching system to on.
bool GD3D11SkipStateCaching = false;

// A self registering exec helper to check for the TOGGLESTATECACHE command.
class FD3D11ToggleStateCacheExecHelper : public FSelfRegisteringExec
{
	virtual bool Exec( class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
	{
		if (FParse::Command(&Cmd, TEXT("TOGGLESTATECACHE")))
		{
			GD3D11SkipStateCaching = !GD3D11SkipStateCaching;
			Ar.Log(FString::Printf(TEXT("D3D11 State Caching: %s"), GD3D11SkipStateCaching ? TEXT("OFF") : TEXT("ON")));
			return true;
		}
		return false;
	}
};
static FD3D11ToggleStateCacheExecHelper GD3D11ToggleStateCacheExecHelper;

#endif	// D3D11_ALLOW_STATE_CACHE && D3D11_STATE_CACHE_RUNTIME_TOGGLE

#if D3D11_ALLOW_STATE_CACHE && D3D11_STATE_CACHE_DEBUG && DO_CHECK

template <EShaderFrequency ShaderFrequency>
void FD3D11StateCacheBase::VerifySamplerStates()
{
	ID3D11SamplerState* SamplerStates[D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT];
	switch (ShaderFrequency)
	{
	case SF_Vertex:		Direct3DDeviceIMContext->VSGetSamplers(0, D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT, SamplerStates); break;
	case SF_Geometry:	Direct3DDeviceIMContext->GSGetSamplers(0, D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT, SamplerStates); break;
	case SF_Pixel:		Direct3DDeviceIMContext->PSGetSamplers(0, D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT, SamplerStates); break;
	case SF_Compute:	Direct3DDeviceIMContext->CSGetSamplers(0, D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT, SamplerStates); break;
	}

	for (uint32 Index = 0; Index < D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT; Index++)
	{
		checkf(SamplerStates[Index] == CurrentSamplerStates[ShaderFrequency][Index], TEXT("Dangling bound SamplerState, try running with -d3debug to track it down."));

		if (SamplerStates[Index])
		{
			SamplerStates[Index]->Release();
		}
	}
}

template <EShaderFrequency ShaderFrequency>
void FD3D11StateCacheBase::VerifyConstantBuffers()
{
	ID3D11Buffer* Buffers[D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT];

	switch (ShaderFrequency)
	{
	case SF_Vertex:		Direct3DDeviceIMContext->VSGetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Buffers); break;
	case SF_Geometry:	Direct3DDeviceIMContext->GSGetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Buffers); break;
	case SF_Pixel:		Direct3DDeviceIMContext->PSGetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Buffers); break;
	case SF_Compute:	Direct3DDeviceIMContext->CSGetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Buffers); break;
	}

	for (uint32 Index = 0; Index < D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT; Index++)
	{
		checkf(Buffers[Index] == CurrentConstantBuffers[ShaderFrequency][Index].Buffer, TEXT("Dangling bound Constant Buffer, try running with -d3debug to track it down."));
		if (Buffers[Index])
		{
			Buffers[Index]->Release();
		}
	}
}

template <EShaderFrequency ShaderFrequency>
void FD3D11StateCacheBase::VerifyShaderResourceViews()
{
	ID3D11ShaderResourceView* Views[D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT];

	switch (ShaderFrequency)
	{
	case SF_Vertex:		Direct3DDeviceIMContext->VSGetShaderResources(0, D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT, Views); break;
	case SF_Geometry:	Direct3DDeviceIMContext->GSGetShaderResources(0, D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT, Views); break;
	case SF_Pixel:		Direct3DDeviceIMContext->PSGetShaderResources(0, D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT, Views); break;
	case SF_Compute:	Direct3DDeviceIMContext->CSGetShaderResources(0, D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT, Views); break;
	}

	for (uint32 Index = 0; Index < D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT; Index++)
	{
		checkf(Views[Index] == CurrentShaderResourceViews[ShaderFrequency][Index], TEXT("Dangling bound SRV, try running with -d3debug to track it down."));

		if (Views[Index])
		{
			Views[Index]->Release();
		}
	}
}


// verification to be called before state cache call
// stack crawl upon check() failure will tell you if state 
// corruption occurred before or after the state cache call
void FD3D11StateCacheBase::VerifyCacheStatePre()
{
	VerifyCacheState();
}

//verification to be called after state cache call
void FD3D11StateCacheBase::VerifyCacheStatePost()
{
	VerifyCacheState();
}

void FD3D11StateCacheBase::VerifyCacheState()
{
	if (!Direct3DDeviceIMContext)
	{
		return;
	}

	// Verify Shader States
	{
		TRefCountPtr<ID3D11VertexShader> VertexShader;
		TRefCountPtr<ID3D11GeometryShader> GeometryShader;
		TRefCountPtr<ID3D11PixelShader> PixelShader;
		TRefCountPtr<ID3D11ComputeShader> ComputeShader;

		Direct3DDeviceIMContext->VSGetShader(VertexShader.GetInitReference(), nullptr, nullptr);
		Direct3DDeviceIMContext->GSGetShader(GeometryShader.GetInitReference(), nullptr, nullptr);
		Direct3DDeviceIMContext->PSGetShader(PixelShader.GetInitReference(), nullptr, nullptr);
		Direct3DDeviceIMContext->CSGetShader(ComputeShader.GetInitReference(), nullptr, nullptr);

		check(VertexShader.GetReference() == CurrentVertexShader);
		check(GeometryShader.GetReference() == CurrentGeometryShader);
		check(PixelShader.GetReference() == CurrentPixelShader);
		check(ComputeShader.GetReference() == CurrentComputeShader);
	}

	// Verify Depth Stencil State
	{
		TRefCountPtr<ID3D11DepthStencilState> DepthStencilState;
		uint32 StencilRef;

		Direct3DDeviceIMContext->OMGetDepthStencilState(DepthStencilState.GetInitReference(), &StencilRef);

		check(DepthStencilState.GetReference() == CurrentDepthStencilState);
		check(StencilRef == CurrentReferenceStencil);
	}

	// Verify Rasterizer State
	{
		TRefCountPtr<ID3D11RasterizerState> RasterizerState;

		Direct3DDeviceIMContext->RSGetState(RasterizerState.GetInitReference());

		check(RasterizerState.GetReference() == CurrentRasterizerState);
	}

	// Verify Blend State
	{
		TRefCountPtr<ID3D11BlendState> BlendState;
		float BlendFactor[4];
		uint32 SampleMask;

		Direct3DDeviceIMContext->OMGetBlendState(BlendState.GetInitReference(), BlendFactor, &SampleMask);

		check(BlendState.GetReference() == CurrentBlendState);
		check(FMemory::Memcmp(BlendFactor, CurrentBlendFactor, sizeof(CurrentBlendFactor)) == 0);
		check(SampleMask == CurrentBlendSampleMask);
	}

	// Verify Viewport state
	{
		D3D11_VIEWPORT vp[D3D11_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE];
		uint32 numVP = CurrentNumberOfViewports;
		Direct3DDeviceIMContext->RSGetViewports(&numVP,&vp[0]);
		check(numVP == CurrentNumberOfViewports);
		check( FMemory::Memcmp( &vp, &CurrentViewport[0],sizeof(D3D11_VIEWPORT) * CurrentNumberOfViewports) ==0);
	}
	
	// Verify Input Layout
	{
		TRefCountPtr<ID3D11InputLayout> InputLayout;
		Direct3DDeviceIMContext->IAGetInputLayout(InputLayout.GetInitReference());
		checkf(InputLayout.GetReference() == CurrentInputLayout, TEXT("Dangling bound Input Layout, try running with -d3debug to track it down."));
	}

	// Verify Sampler States
	{
		VerifySamplerStates<SF_Vertex>();
		VerifySamplerStates<SF_Geometry>();
		VerifySamplerStates<SF_Pixel>();
		VerifySamplerStates<SF_Compute>();
	}

	// Verify Vertex Buffers
	{
		ID3D11Buffer* VertexBuffers[D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT];
		uint32 Strides[D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT];
		uint32 Offsets[D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT];

		Direct3DDeviceIMContext->IAGetVertexBuffers(0, D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT, VertexBuffers, Strides, Offsets);

		for (uint32 Index = 0; Index < D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT; Index++)
		{
			check(VertexBuffers[Index] == CurrentVertexBuffers[Index].VertexBuffer);
			check(Strides[Index] == CurrentVertexBuffers[Index].Stride);
			check(Offsets[Index] == CurrentVertexBuffers[Index].Offset);
			if (VertexBuffers[Index])
			{
				VertexBuffers[Index]->Release();
			}
		}
	}

	// Verify Index Buffer
	{
		TRefCountPtr<ID3D11Buffer> IndexBuffer;
		DXGI_FORMAT Format;
		uint32 Offset;

		Direct3DDeviceIMContext->IAGetIndexBuffer(IndexBuffer.GetInitReference(), &Format, &Offset);

		check(IndexBuffer.GetReference() == CurrentIndexBuffer);
		check(Format == CurrentIndexFormat);
		check(Offset == CurrentIndexOffset);
	}

	// Verify Primitive Topology
	{
		D3D11_PRIMITIVE_TOPOLOGY PrimitiveTopology;
		Direct3DDeviceIMContext->IAGetPrimitiveTopology(&PrimitiveTopology);
		check(PrimitiveTopology == CurrentPrimitiveTopology);
	}

	// Verify Constant Buffers
	{
		((FD3D11StateCache*)this)->VerifyConstantBuffers<SF_Vertex>();
		((FD3D11StateCache*)this)->VerifyConstantBuffers<SF_Geometry>();
		((FD3D11StateCache*)this)->VerifyConstantBuffers<SF_Pixel>();
		((FD3D11StateCache*)this)->VerifyConstantBuffers<SF_Compute>();
	}

	// Verify Shader Resource Views
	{
		VerifyShaderResourceViews<SF_Vertex>();
		VerifyShaderResourceViews<SF_Geometry>();
		VerifyShaderResourceViews<SF_Pixel>();
		VerifyShaderResourceViews<SF_Compute>();
	}
}
#endif	// D3D11_ALLOW_STATE_CACHE && D3D11_STATE_CACHE_DEBUG && DO_CHECK

void FD3D11StateCacheBase::ClearState()
{
	if (Direct3DDeviceIMContext)
	{
		Direct3DDeviceIMContext->ClearState();
	}

#if D3D11_ALLOW_STATE_CACHE
	// Shader Resource View State Cache
	for (uint32 ShaderFrequency = 0; ShaderFrequency < SF_NumStandardFrequencies; ShaderFrequency++)
	{
		for (uint32 Index = 0; Index < D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT; Index++)
		{
			if(CurrentShaderResourceViews[ShaderFrequency][Index])
			{
				CurrentShaderResourceViews[ShaderFrequency][Index]->Release();
				CurrentShaderResourceViews[ShaderFrequency][Index] = NULL;
			}
		}
	}

	// Rasterizer State Cache
	CurrentRasterizerState = nullptr;

	// Depth Stencil State Cache
	CurrentReferenceStencil = 0;
	CurrentDepthStencilState = nullptr;
	bDepthBoundsEnabled = false;
	DepthBoundsMin = 0.0f;
	DepthBoundsMax = 1.0f;

	// Shader Cache
	CurrentVertexShader = nullptr;
	CurrentGeometryShader = nullptr;
	CurrentPixelShader = nullptr;
	CurrentComputeShader = nullptr;

	// Blend State Cache
	CurrentBlendFactor[0] = 1.0f;
	CurrentBlendFactor[1] = 1.0f;
	CurrentBlendFactor[2] = 1.0f;
	CurrentBlendFactor[3] = 1.0f;

	FMemory::Memset( &CurrentViewport[0], 0, sizeof(D3D11_VIEWPORT) * D3D11_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE );
	CurrentNumberOfViewports = 0;

	CurrentBlendSampleMask = 0xffffffff;
	CurrentBlendState = nullptr;

	CurrentInputLayout = nullptr;

	FMemory::Memzero(CurrentVertexBuffers, sizeof(CurrentVertexBuffers));
	FMemory::Memzero(CurrentSamplerStates, sizeof(CurrentSamplerStates));

	CurrentIndexBuffer = nullptr;
	CurrentIndexFormat = DXGI_FORMAT_UNKNOWN;

	CurrentIndexOffset = 0;
	CurrentPrimitiveTopology = D3D11_PRIMITIVE_TOPOLOGY_UNDEFINED;

	for (uint32 Frequency = 0; Frequency < SF_NumStandardFrequencies; Frequency++)
	{
		for (uint32 Index = 0; Index < D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT; Index++)
		{
			CurrentConstantBuffers[Frequency][Index].Buffer = nullptr;
			CurrentConstantBuffers[Frequency][Index].FirstConstant = 0;
			CurrentConstantBuffers[Frequency][Index].NumConstants = D3D11_REQ_CONSTANT_BUFFER_ELEMENT_COUNT;
		}
	}

#endif	// D3D11_ALLOW_STATE_CACHE
}
