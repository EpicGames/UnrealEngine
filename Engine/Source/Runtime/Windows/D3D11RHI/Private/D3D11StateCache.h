// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of Device Context State Caching to improve draw
//	thread performance by removing redundant device context calls.

#pragma once

#include "D3D11StateCachePrivate.h"

typedef FD3D11StateCacheBase FD3D11StateCache;
