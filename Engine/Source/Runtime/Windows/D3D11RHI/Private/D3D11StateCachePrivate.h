// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of Device Context State Caching to improve draw
//	thread performance by removing redundant device context calls.

#pragma once

#include "RHIDefinitions.h"
#include "Windows/D3D11ThirdParty.h"

//-----------------------------------------------------------------------------
//	Configuration
//-----------------------------------------------------------------------------

// If set, enables the D3D11 state caching system.
#define D3D11_ALLOW_STATE_CACHE 1

// If set, includes a runtime toggle console command for debugging D3D11 state caching.
// ("TOGGLESTATECACHE")
#define D3D11_STATE_CACHE_RUNTIME_TOGGLE 0

// If set, includes a cache state verification check.
// After each state set call, the cached state is compared against the actual state of the ID3D11DeviceContext.
// This is *very slow* and should only be enabled to debug the state caching system.
#ifndef D3D11_STATE_CACHE_DEBUG
#define D3D11_STATE_CACHE_DEBUG 0
#endif

//-----------------------------------------------------------------------------
//	
//-----------------------------------------------------------------------------

// Keep set state functions inline to reduce call overhead
#define D3D11_STATE_CACHE_INLINE FORCEINLINE
//#define DO_CHECK 1
//#define D3D11_STATE_CACHE_DEBUG 1


#if D3D11_ALLOW_STATE_CACHE && D3D11_STATE_CACHE_DEBUG && DO_CHECK
	#define D3D11_STATE_CACHE_VERIFY(...) VerifyCacheState()
	#define D3D11_STATE_CACHE_VERIFY_PRE(...) VerifyCacheStatePre()
	#define D3D11_STATE_CACHE_VERIFY_POST(...) VerifyCacheStatePost()
#else
	#define D3D11_STATE_CACHE_VERIFY(...)
	#define D3D11_STATE_CACHE_VERIFY_PRE(...)
	#define D3D11_STATE_CACHE_VERIFY_POST(...)
#endif

#if D3D11_ALLOW_STATE_CACHE && D3D11_STATE_CACHE_RUNTIME_TOGGLE
extern bool GD3D11SkipStateCaching;
#else
static const bool GD3D11SkipStateCaching = false;
#endif

//-----------------------------------------------------------------------------
//	FD3D11StateCache Class Definition
//-----------------------------------------------------------------------------
class FD3D11StateCacheBase
{
public:
	bool bDepthBoundsEnabled = false;
	float DepthBoundsMin = 0.0f;
	float DepthBoundsMax = 1.0f;

protected:
	ID3D11DeviceContext* Direct3DDeviceIMContext;

#if D3D11_ALLOW_STATE_CACHE
	// Shader Resource Views Cache
	ID3D11ShaderResourceView* CurrentShaderResourceViews[SF_NumStandardFrequencies][D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT];

	// Rasterizer State Cache
	ID3D11RasterizerState* CurrentRasterizerState;

	// Depth Stencil State Cache
	uint32 CurrentReferenceStencil;
	ID3D11DepthStencilState* CurrentDepthStencilState;

	// Shader Cache
	ID3D11VertexShader* CurrentVertexShader;
	ID3D11GeometryShader* CurrentGeometryShader;
	ID3D11PixelShader* CurrentPixelShader;
	ID3D11ComputeShader* CurrentComputeShader;

	// Blend State Cache
	float CurrentBlendFactor[4];
	uint32 CurrentBlendSampleMask;
	ID3D11BlendState* CurrentBlendState;
	
	// Viewport
	uint32			CurrentNumberOfViewports;
	D3D11_VIEWPORT CurrentViewport[D3D11_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE];


	// Vertex Buffer State
	struct FD3D11VertexBufferState
	{
		ID3D11Buffer* VertexBuffer;
		uint32 Stride;
		uint32 Offset;
	} CurrentVertexBuffers[D3D11_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT];

	// Index Buffer State
	ID3D11Buffer* CurrentIndexBuffer;
	DXGI_FORMAT CurrentIndexFormat;
	uint32 CurrentIndexOffset;

	// Primitive Topology State
	D3D11_PRIMITIVE_TOPOLOGY CurrentPrimitiveTopology;

	// Input Layout State
	ID3D11InputLayout* CurrentInputLayout;

	uint16 StreamStrides[MaxVertexElementCount];

	// Sampler State
	ID3D11SamplerState* CurrentSamplerStates[SF_NumStandardFrequencies][D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT];

	// Constant Buffer State
	struct FD3D11ConstantBufferState
	{
		ID3D11Buffer* Buffer;
		uint32 FirstConstant;
		uint32 NumConstants;
	} CurrentConstantBuffers[SF_NumStandardFrequencies][D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT];

	bool bAlwaysSetIndexBuffers;

#endif

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void InternalSetShaderResourceView(uint32 ResourceIndex, ID3D11ShaderResourceView*& SRV)
	{
		// Set the SRV we have been given (or null).
		CA_SUPPRESS(6326);
		switch (ShaderFrequency)
		{
		case SF_Vertex:		Direct3DDeviceIMContext->VSSetShaderResources(ResourceIndex, 1, &SRV); break;
		case SF_Geometry:	Direct3DDeviceIMContext->GSSetShaderResources(ResourceIndex, 1, &SRV); break;
		case SF_Pixel:		Direct3DDeviceIMContext->PSSetShaderResources(ResourceIndex, 1, &SRV); break;
		case SF_Compute:	Direct3DDeviceIMContext->CSSetShaderResources(ResourceIndex, 1, &SRV); break;
		}
	}

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void InternalSetSamplerState(uint32 SamplerIndex, ID3D11SamplerState*& SamplerState)
	{
		CA_SUPPRESS(6326);
		switch (ShaderFrequency)
		{
		case SF_Vertex:		Direct3DDeviceIMContext->VSSetSamplers(SamplerIndex, 1, &SamplerState); break;
		case SF_Geometry:	Direct3DDeviceIMContext->GSSetSamplers(SamplerIndex, 1, &SamplerState); break;
		case SF_Pixel:		Direct3DDeviceIMContext->PSSetSamplers(SamplerIndex, 1, &SamplerState); break;
		case SF_Compute:	Direct3DDeviceIMContext->CSSetSamplers(SamplerIndex, 1, &SamplerState); break;
		}
	}

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void InternalSetSetConstantBuffer(uint32 SlotIndex, ID3D11Buffer*& ConstantBuffer)
	{
		CA_SUPPRESS(6326);
		switch (ShaderFrequency)
		{
		case SF_Vertex:		Direct3DDeviceIMContext->VSSetConstantBuffers(SlotIndex, 1, &ConstantBuffer); break;
		case SF_Geometry:	Direct3DDeviceIMContext->GSSetConstantBuffers(SlotIndex, 1, &ConstantBuffer); break;
		case SF_Pixel:		Direct3DDeviceIMContext->PSSetConstantBuffers(SlotIndex, 1, &ConstantBuffer); break;
		case SF_Compute:	Direct3DDeviceIMContext->CSSetConstantBuffers(SlotIndex, 1, &ConstantBuffer); break;
		}
	}

	D3D11_STATE_CACHE_INLINE void InternalSetIndexBuffer(ID3D11Buffer* IndexBuffer, DXGI_FORMAT Format, uint32 Offset)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();

		if ( bAlwaysSetIndexBuffers || (CurrentIndexBuffer != IndexBuffer || CurrentIndexFormat != Format || CurrentIndexOffset != Offset) || GD3D11SkipStateCaching)
		{
			CurrentIndexBuffer = IndexBuffer;
			CurrentIndexFormat = Format;
			CurrentIndexOffset = Offset;
			Direct3DDeviceIMContext->IASetIndexBuffer(IndexBuffer, Format, Offset);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->IASetIndexBuffer(IndexBuffer, Format, Offset);
#endif
	}

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void InternalSetShaderResourceView(ID3D11ShaderResourceView*& SRV, uint32 ResourceIndex)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		check(ResourceIndex < ARRAYSIZE(CurrentShaderResourceViews[ShaderFrequency]));
		if ((CurrentShaderResourceViews[ShaderFrequency][ResourceIndex] != SRV) || GD3D11SkipStateCaching)
		{
			if(SRV)
			{
				SRV->AddRef();
			}
			if(CurrentShaderResourceViews[ShaderFrequency][ResourceIndex])
			{
				CurrentShaderResourceViews[ShaderFrequency][ResourceIndex]->Release();
			}
			CurrentShaderResourceViews[ShaderFrequency][ResourceIndex] = SRV;
			InternalSetShaderResourceView<ShaderFrequency>(ResourceIndex, SRV);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else	// !D3D11_ALLOW_STATE_CACHE
		InternalSetShaderResourceView<ShaderFrequency>(ResourceIndex, SRV);
#endif
	}

	D3D11_STATE_CACHE_INLINE void InternalSetStreamSource(ID3D11Buffer* VertexBuffer, uint32 StreamIndex, uint32 Stride, uint32 Offset)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		check(StreamIndex < ARRAYSIZE(CurrentVertexBuffers));
		FD3D11VertexBufferState& Slot = CurrentVertexBuffers[StreamIndex];
		if ((Slot.VertexBuffer != VertexBuffer || Slot.Offset != Offset || Slot.Stride != Stride) || GD3D11SkipStateCaching)
		{
			Slot.VertexBuffer = VertexBuffer;
			Slot.Offset = Offset;
			Slot.Stride = Stride;
			Direct3DDeviceIMContext->IASetVertexBuffers(StreamIndex, 1, &VertexBuffer, &Stride, &Offset);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->IASetVertexBuffers(StreamIndex, 1, &VertexBuffer, &Stride, &Offset);
#endif
	}

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void InternalSetSamplerState(ID3D11SamplerState* SamplerState, uint32 SamplerIndex)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		check(SamplerIndex < ARRAYSIZE(CurrentSamplerStates[ShaderFrequency]));
		if ((CurrentSamplerStates[ShaderFrequency][SamplerIndex] != SamplerState) || GD3D11SkipStateCaching)
		{
			CurrentSamplerStates[ShaderFrequency][SamplerIndex] = SamplerState;
			InternalSetSamplerState<ShaderFrequency>(SamplerIndex, SamplerState);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		InternalSetSamplerState<ShaderFrequency>(SamplerIndex, SamplerState);
#endif
	}

public:
	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void ClearConstantBuffers()
	{
#if D3D11_ALLOW_STATE_CACHE
		FMemory::Memzero(CurrentConstantBuffers[ShaderFrequency]);
#endif
		ID3D11Buffer* Empty[D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT] = {0};
		CA_SUPPRESS(6326);
		switch (ShaderFrequency)
		{
		case SF_Vertex:		Direct3DDeviceIMContext->VSSetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Empty); break;
		case SF_Geometry:	Direct3DDeviceIMContext->GSSetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Empty); break;
		case SF_Pixel:		Direct3DDeviceIMContext->PSSetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Empty); break;
		case SF_Compute:	Direct3DDeviceIMContext->CSSetConstantBuffers(0, D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, Empty); break;
		}
	}

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void SetShaderResourceView(ID3D11ShaderResourceView* SRV, uint32 ResourceIndex)
	{
		InternalSetShaderResourceView<ShaderFrequency>(SRV, ResourceIndex);
	}

	D3D11_STATE_CACHE_INLINE void SetViewport(D3D11_VIEWPORT Viewport)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentNumberOfViewports != 1 || FMemory::Memcmp(&CurrentViewport[0],&Viewport, sizeof(D3D11_VIEWPORT))) || GD3D11SkipStateCaching)
		{
			FMemory::Memcpy(&CurrentViewport[0], &Viewport, sizeof(D3D11_VIEWPORT));
			CurrentNumberOfViewports = 1;
			Direct3DDeviceIMContext->RSSetViewports(1,&Viewport);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else	// !D3D11_ALLOW_STATE_CACHE
		Direct3DDeviceIMContext->RSSetViewports(1,&Viewport);
#endif	// D3D11_ALLOW_STATE_CACHE
	}

	D3D11_STATE_CACHE_INLINE void SetViewports(uint32 Count, D3D11_VIEWPORT* Viewports)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentNumberOfViewports != Count || FMemory::Memcmp(&CurrentViewport[0], Viewports, sizeof(D3D11_VIEWPORT) * Count)) || GD3D11SkipStateCaching)
		{
			FMemory::Memcpy(&CurrentViewport[0], Viewports, sizeof(D3D11_VIEWPORT) * Count);
			CurrentNumberOfViewports = Count;
			Direct3DDeviceIMContext->RSSetViewports(Count, Viewports);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else	// !D3D11_ALLOW_STATE_CACHE
		Direct3DDeviceIMContext->RSSetViewports(Count, Viewports);
#endif	// D3D11_ALLOW_STATE_CACHE
	}

	D3D11_STATE_CACHE_INLINE void GetViewport(D3D11_VIEWPORT *Viewport)
	{
#if D3D11_ALLOW_STATE_CACHE
		check(Viewport);
		FMemory::Memcpy(Viewport,&CurrentViewport,sizeof(D3D11_VIEWPORT));
#else	// !D3D11_ALLOW_STATE_CACHE
		uint32 one = 1;
		Direct3DDeviceIMContext->RSGetViewports(&one,Viewport);
#endif	// D3D11_ALLOW_STATE_CACHE
	}
	
	D3D11_STATE_CACHE_INLINE void GetViewports(uint32* Count, D3D11_VIEWPORT *Viewports)
	{
#if D3D11_ALLOW_STATE_CACHE 
		check (*Count);
		if (Viewports) //NULL is legal if you just want count
		{
			//as per d3d spec
			int32 StorageSizeCount = (int32)(*Count);
			int32 CopyCount = FMath::Min(FMath::Min(StorageSizeCount, (int32)CurrentNumberOfViewports), D3D11_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE);
			if (CopyCount > 0)
			{
				FMemory::Memcpy(Viewports, &CurrentViewport[0], sizeof(D3D11_VIEWPORT) * CopyCount);
			}
			//remaining viewports in supplied array must be set to zero
			if (StorageSizeCount > CopyCount)
			{
				FMemory::Memset(&Viewports[CopyCount], 0, sizeof(D3D11_VIEWPORT) * (StorageSizeCount - CopyCount));
			}
		}
		*Count = CurrentNumberOfViewports;

#else	// !D3D11_ALLOW_STATE_CACHE
		Direct3DDeviceIMContext->RSGetViewports(Count, Viewports);
#endif	// D3D11_ALLOW_STATE_CACHE
	}

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void SetSamplerState(ID3D11SamplerState* SamplerState, uint32 SamplerIndex)
	{
		InternalSetSamplerState<ShaderFrequency>(SamplerState, SamplerIndex);
	}

	template <EShaderFrequency ShaderFrequency>
	D3D11_STATE_CACHE_INLINE void SetConstantBuffer(ID3D11Buffer* ConstantBuffer, uint32 SlotIndex)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		FD3D11ConstantBufferState& Current = CurrentConstantBuffers[ShaderFrequency][SlotIndex];
		if ((Current.Buffer != ConstantBuffer || Current.FirstConstant != 0 || Current.NumConstants != D3D11_REQ_CONSTANT_BUFFER_ELEMENT_COUNT) || GD3D11SkipStateCaching)
		{
			Current.Buffer = ConstantBuffer;
			Current.FirstConstant = 0;
			Current.NumConstants = D3D11_REQ_CONSTANT_BUFFER_ELEMENT_COUNT;
			InternalSetSetConstantBuffer<ShaderFrequency>(SlotIndex, ConstantBuffer);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		InternalSetSetConstantBuffer<ShaderFrequency>(SlotIndex, ConstantBuffer);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetRasterizerState(ID3D11RasterizerState* State)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentRasterizerState != State) || GD3D11SkipStateCaching)
		{
			CurrentRasterizerState = State;
			Direct3DDeviceIMContext->RSSetState(State);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->RSSetState(State);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetBlendState(ID3D11BlendState* State, const float BlendFactor[4], uint32 SampleMask)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentBlendState != State || CurrentBlendSampleMask != SampleMask || FMemory::Memcmp(CurrentBlendFactor, BlendFactor, sizeof(CurrentBlendFactor))) || GD3D11SkipStateCaching)
		{
			CurrentBlendState = State;
			CurrentBlendSampleMask = SampleMask;
			FMemory::Memcpy(CurrentBlendFactor, BlendFactor, sizeof(CurrentBlendFactor));
			Direct3DDeviceIMContext->OMSetBlendState(State, BlendFactor, SampleMask);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->OMSetBlendState(State, BlendFactor, SampleMask);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetBlendFactor(const float BlendFactor[4], uint32 SampleMask)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentBlendSampleMask != SampleMask || FMemory::Memcmp(CurrentBlendFactor, BlendFactor, sizeof(CurrentBlendFactor))) || GD3D11SkipStateCaching)
		{
			CurrentBlendSampleMask = SampleMask;
			FMemory::Memcpy(CurrentBlendFactor, BlendFactor, sizeof(CurrentBlendFactor));
			Direct3DDeviceIMContext->OMSetBlendState(CurrentBlendState, BlendFactor, SampleMask);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->OMSetBlendState(CurrentBlendState, BlendFactor, SampleMask);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetDepthStencilState(ID3D11DepthStencilState* State, uint32 RefStencil)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentDepthStencilState != State || CurrentReferenceStencil != RefStencil) || GD3D11SkipStateCaching)
		{
			CurrentDepthStencilState = State;
			CurrentReferenceStencil = RefStencil;
			Direct3DDeviceIMContext->OMSetDepthStencilState(State, RefStencil);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->OMSetDepthStencilState(State, RefStencil);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetStencilRef(uint32 RefStencil)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if (CurrentReferenceStencil != RefStencil || GD3D11SkipStateCaching)
		{
			CurrentReferenceStencil = RefStencil;
			Direct3DDeviceIMContext->OMSetDepthStencilState(CurrentDepthStencilState, RefStencil);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->OMSetDepthStencilState(CurrentDepthStencilState, RefStencil);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetVertexShader(ID3D11VertexShader* Shader)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentVertexShader != Shader) || GD3D11SkipStateCaching)
		{
			CurrentVertexShader = Shader;
			Direct3DDeviceIMContext->VSSetShader(Shader, nullptr, 0);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->VSSetShader(Shader, nullptr, 0);
#endif
	}

	D3D11_STATE_CACHE_INLINE void GetVertexShader(ID3D11VertexShader** VertexShader)
	{
#if D3D11_ALLOW_STATE_CACHE
		*VertexShader = CurrentVertexShader;
		if (CurrentVertexShader)
		{
			CurrentVertexShader->AddRef();
		}
#else
		Direct3DDeviceIMContext->VSGetShader(VertexShader, nullptr, nullptr);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetGeometryShader(ID3D11GeometryShader* Shader)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentGeometryShader != Shader) || GD3D11SkipStateCaching)
		{
			CurrentGeometryShader = Shader;
			Direct3DDeviceIMContext->GSSetShader(Shader, nullptr, 0);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->GSSetShader(Shader, nullptr, 0);
#endif
	}

	D3D11_STATE_CACHE_INLINE void GetGeometryShader(ID3D11GeometryShader** GeometryShader)
	{
#if D3D11_ALLOW_STATE_CACHE
		*GeometryShader = CurrentGeometryShader;
		if (CurrentGeometryShader)
		{
			CurrentGeometryShader->AddRef();
		}
#else
		Direct3DDeviceIMContext->GSGetShader(GeometryShader, nullptr, nullptr);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetPixelShader(ID3D11PixelShader* Shader)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentPixelShader != Shader) || GD3D11SkipStateCaching)
		{
			CurrentPixelShader = Shader;
			Direct3DDeviceIMContext->PSSetShader(Shader, nullptr, 0);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->PSSetShader(Shader, nullptr, 0);
#endif
	}

	D3D11_STATE_CACHE_INLINE void GetPixelShader(ID3D11PixelShader** PixelShader)
	{
#if D3D11_ALLOW_STATE_CACHE
		*PixelShader = CurrentPixelShader;
		if (CurrentPixelShader)
		{
			CurrentPixelShader->AddRef();
		}
#else
		Direct3DDeviceIMContext->PSGetShader(PixelShader, nullptr, nullptr);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetComputeShader(ID3D11ComputeShader* Shader)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentComputeShader != Shader) || GD3D11SkipStateCaching)
		{
			CurrentComputeShader = Shader;
			Direct3DDeviceIMContext->CSSetShader(Shader, nullptr, 0);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->CSSetShader(Shader, nullptr, 0);
#endif
	}

	D3D11_STATE_CACHE_INLINE void GetComputeShader(ID3D11ComputeShader** ComputeShader)
	{
#if D3D11_ALLOW_STATE_CACHE
		*ComputeShader = CurrentComputeShader;
		if (CurrentComputeShader)
		{
			CurrentComputeShader->AddRef();
		}
#else
		Direct3DDeviceIMContext->CSGetShader(ComputeShader, nullptr, nullptr);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetStreamStrides(const uint16* InStreamStrides)
	{
		FMemory::Memcpy(StreamStrides, InStreamStrides, sizeof(StreamStrides));
	}

	D3D11_STATE_CACHE_INLINE void SetInputLayout(ID3D11InputLayout* InputLayout)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentInputLayout != InputLayout) || GD3D11SkipStateCaching)
		{
			CurrentInputLayout = InputLayout;
			Direct3DDeviceIMContext->IASetInputLayout(InputLayout);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->IASetInputLayout(InputLayout);
#endif
	}

	D3D11_STATE_CACHE_INLINE void SetStreamSource(ID3D11Buffer* VertexBuffer, uint32 StreamIndex, uint32 Stride, uint32 Offset)
	{
		ensure(Stride == StreamStrides[StreamIndex]);
		InternalSetStreamSource(VertexBuffer, StreamIndex, Stride, Offset);
	}

	D3D11_STATE_CACHE_INLINE void SetStreamSource(ID3D11Buffer* VertexBuffer, uint32 StreamIndex, uint32 Offset)
	{
		InternalSetStreamSource(VertexBuffer, StreamIndex, StreamStrides[StreamIndex], Offset);
	}

public:

	D3D11_STATE_CACHE_INLINE void SetIndexBuffer(ID3D11Buffer* IndexBuffer, DXGI_FORMAT Format, uint32 Offset)
	{
		InternalSetIndexBuffer(IndexBuffer, Format, Offset);
	}

	D3D11_STATE_CACHE_INLINE void SetPrimitiveTopology(D3D11_PRIMITIVE_TOPOLOGY PrimitiveTopology)
	{
#if D3D11_ALLOW_STATE_CACHE
		D3D11_STATE_CACHE_VERIFY_PRE();
		if ((CurrentPrimitiveTopology != PrimitiveTopology) || GD3D11SkipStateCaching)
		{
			CurrentPrimitiveTopology = PrimitiveTopology;
			Direct3DDeviceIMContext->IASetPrimitiveTopology(PrimitiveTopology);
		}
		D3D11_STATE_CACHE_VERIFY_POST();
#else
		Direct3DDeviceIMContext->IASetPrimitiveTopology(PrimitiveTopology);
#endif
	}

	FD3D11StateCacheBase()
		: Direct3DDeviceIMContext(nullptr)
	{
		FMemory::Memzero(CurrentShaderResourceViews, sizeof(CurrentShaderResourceViews));
	}

	void Init(ID3D11DeviceContext* InDeviceContext, bool bInAlwaysSetIndexBuffers = false )
	{
		SetContext(InDeviceContext);
		
#if D3D11_ALLOW_STATE_CACHE
		bAlwaysSetIndexBuffers = bInAlwaysSetIndexBuffers;
#endif
	}

	virtual ~FD3D11StateCacheBase() = default;

	virtual D3D11_STATE_CACHE_INLINE void SetContext(ID3D11DeviceContext* InDeviceContext)
	{
		Direct3DDeviceIMContext = InDeviceContext;
		ClearState();
		D3D11_STATE_CACHE_VERIFY();
	}

	/** 
	 * Clears all D3D11 State, setting all input/output resource slots, shaders, input layouts,
	 * predications, scissor rectangles, depth-stencil state, rasterizer state, blend state,
	 * sampler state, and viewports to NULL
	 */
	virtual void ClearState();

#if D3D11_ALLOW_STATE_CACHE && D3D11_STATE_CACHE_DEBUG
protected:
	// Debug helper methods to verify cached state integrity.
	template <EShaderFrequency ShaderFrequency>
	void VerifySamplerStates();

	template <EShaderFrequency ShaderFrequency>
	void VerifyConstantBuffers();

	template <EShaderFrequency ShaderFrequency>
	void VerifyShaderResourceViews();

	void VerifyCacheStatePre();
	void VerifyCacheStatePost();
	void VerifyCacheState();
#endif
};
