// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Texture.cpp: D3D texture RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"

// For Depth Bounds Test interface
#include "Windows/AllowWindowsPlatformTypes.h"
#if WITH_NVAPI
	#include "nvapi.h"
#endif
#if WITH_AMD_AGS
	#include "amd_ags.h"
#endif
#include "Windows/HideWindowsPlatformTypes.h"

#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RHICoreStats.h"
#include "RHICoreTexture.h"
#include "RHIUtilities.h"
#include "RHICoreTextureInitializer.h"
#include "RHITextureUtils.h"

int64 FD3D11GlobalStats::GDedicatedVideoMemory = 0;
int64 FD3D11GlobalStats::GDedicatedSystemMemory = 0;
int64 FD3D11GlobalStats::GSharedSystemMemory = 0;
int64 FD3D11GlobalStats::GTotalGraphicsMemory = 0;


/*-----------------------------------------------------------------------------
	Texture allocator support.
-----------------------------------------------------------------------------*/

// Note: This function can be called from many different threads
// @param TextureSize >0 to allocate, <0 to deallocate
// @param b3D true:3D, false:2D or cube map
void UpdateD3D11TextureStats(FD3D11Texture& Texture, bool bAllocating)
{
	const FRHITextureDesc& TextureDesc = Texture.GetDesc();

	uint32 BindFlags;
	if (TextureDesc.IsTexture3D())
	{
		D3D11_TEXTURE3D_DESC Desc;
		Texture.GetD3D11Texture3D()->GetDesc(&Desc);
		BindFlags = Desc.BindFlags;
	}
	else
	{
		D3D11_TEXTURE2D_DESC Desc;
		Texture.GetD3D11Texture2D()->GetDesc(&Desc);
		BindFlags = Desc.BindFlags;
	}

	const uint64 TextureSize = Texture.GetMemorySize();

	const bool bOnlyStreamableTexturesInTexturePool = false;
	UE::RHICore::UpdateGlobalTextureStats(TextureDesc, TextureSize, bOnlyStreamableTexturesInTexturePool, bAllocating);

	if (bAllocating)
	{
		INC_DWORD_STAT(STAT_D3D11TexturesAllocated);
	}
	else
	{
		INC_DWORD_STAT(STAT_D3D11TexturesReleased);
	}

	// On Windows there is no way to hook into the low level d3d allocations and frees.
	// This means that we must manually add the tracking here.
	if (bAllocating)
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Texture.GetResource(), TextureSize, ELLMTag::GraphicsPlatform));
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default , Texture.GetResource(), TextureSize, ELLMTag::Textures));
		{
			LLM(UE_MEMSCOPE_DEFAULT(ELLMTag::Textures));
			MemoryTrace_Alloc((uint64)Texture.GetResource(), TextureSize, 1024, EMemoryTraceRootHeap::VideoMemory);
		}
	}
	else
	{
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Texture.GetResource()));
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default , Texture.GetResource()));
		MemoryTrace_Free((uint64)Texture.GetResource(), EMemoryTraceRootHeap::VideoMemory);
	}
}

FDynamicRHI::FRHICalcTextureSizeResult FD3D11DynamicRHI::RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex)
{
	// D3D11 does not provide a way to compute the actual driver/GPU specific in-memory size of a texture.
	// Fallback to the estimate based on the texture's dimensions / format etc.
	FDynamicRHI::FRHICalcTextureSizeResult Result;
	Result.Size = Desc.CalcMemorySizeEstimate(FirstMipIndex);
	Result.Align = 1;
	return Result;
}

void FD3D11DynamicRHI::RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats)
{
	UE::RHICore::FillBaselineTextureMemoryStats(OutStats);

	OutStats.DedicatedVideoMemory = FD3D11GlobalStats::GDedicatedVideoMemory;
    OutStats.DedicatedSystemMemory = FD3D11GlobalStats::GDedicatedSystemMemory;
    OutStats.SharedSystemMemory = FD3D11GlobalStats::GSharedSystemMemory;
	OutStats.TotalGraphicsMemory = FD3D11GlobalStats::GTotalGraphicsMemory ? FD3D11GlobalStats::GTotalGraphicsMemory : -1;

	OutStats.LargestContiguousAllocation = OutStats.StreamingMemorySize;
}

bool FD3D11DynamicRHI::RHIGetTextureMemoryVisualizeData( FColor* /*TextureData*/, int32 /*SizeX*/, int32 /*SizeY*/, int32 /*Pitch*/, int32 /*PixelSize*/ )
{
	// currently only implemented for console (Note: Keep this function for further extension. Talk to NiklasS for more info.)
	return false;
}

// Work around an issue with the WARP device & BC7
// Creating two views with different formats (DXGI_FORMAT_BC7_UNORM vs DXGI_FORMAT_BC7_UNORM_SRGB)
// will result in a crash inside d3d10warp.dll when creating the second view
void ApplyBC7SoftwareAdapterWorkaround(bool bSoftwareAdapter, D3D11_TEXTURE2D_DESC& Desc)
{
	if (bSoftwareAdapter)
	{
		bool bApplyWorkaround =	Desc.Format == DXGI_FORMAT_BC7_TYPELESS
							 && Desc.Usage == D3D11_USAGE_DEFAULT
							 && Desc.MipLevels == 1
							 && Desc.ArraySize == 1
							 && Desc.CPUAccessFlags == 0;

		if (bApplyWorkaround)
		{
			Desc.MiscFlags |= D3D11_RESOURCE_MISC_SHARED;
		}
	}
}

/** If true, guard texture creates with SEH to log more information about a driver crash we are seeing during texture streaming. */
#define GUARDED_TEXTURE_CREATES (!(UE_BUILD_SHIPPING || UE_BUILD_TEST || PLATFORM_COMPILER_CLANG))

/**
 * Creates a 2D texture optionally guarded by a structured exception handler.
 */
static void SafeCreateTexture2D(ID3D11Device* Direct3DDevice, int32 UEFormat, const D3D11_TEXTURE2D_DESC* TextureDesc, const D3D11_SUBRESOURCE_DATA* SubResourceData, ID3D11Texture2D** OutTexture2D, const TCHAR* DebugName)
{
#if GUARDED_TEXTURE_CREATES
	bool bDriverCrash = true;
	__try
	{
#endif // #if GUARDED_TEXTURE_CREATES
		VERIFYD3D11CREATETEXTURERESULT(
			Direct3DDevice->CreateTexture2D(TextureDesc,SubResourceData,OutTexture2D),
			UEFormat,
			TextureDesc->Width,
			TextureDesc->Height,
			TextureDesc->ArraySize,
			TextureDesc->Format,
			TextureDesc->MipLevels,
			TextureDesc->BindFlags,
			TextureDesc->Usage,
			TextureDesc->CPUAccessFlags,
			TextureDesc->MiscFlags,			
			TextureDesc->SampleDesc.Count,
			TextureDesc->SampleDesc.Quality,
			SubResourceData ? SubResourceData->pSysMem : nullptr,
			SubResourceData ? SubResourceData->SysMemPitch : 0,
			SubResourceData ? SubResourceData->SysMemSlicePitch : 0,
			Direct3DDevice,
			DebugName
			);
#if GUARDED_TEXTURE_CREATES
		bDriverCrash = false;
	}
	__finally
	{
		if (bDriverCrash)
		{
			UE_LOGF(LogD3D11RHI,Error,
				"Driver crashed while creating texture: %ux%ux%u %ls(0x%08x) with %u mips, PF_ %d",
				TextureDesc->Width,
				TextureDesc->Height,
				TextureDesc->ArraySize,
				UE::DXGIUtilities::GetFormatString(TextureDesc->Format),
				(uint32)TextureDesc->Format,
				TextureDesc->MipLevels,
				UEFormat
				);
		}
	}
#endif // #if GUARDED_TEXTURE_CREATES
}

static void SafeCreateTexture3D(FD3D11Device* Direct3DDevice, int32 UEFormat, const D3D11_TEXTURE3D_DESC* TextureDesc, const D3D11_SUBRESOURCE_DATA* SubResourceData, ID3D11Texture3D** OutTexture2D, const TCHAR* DebugName)
{
#if GUARDED_TEXTURE_CREATES
	bool bDriverCrash = true;
	__try
	{
#endif // #if GUARDED_TEXTURE_CREATES
		VERIFYD3D11CREATETEXTURERESULT(
			Direct3DDevice->CreateTexture3D(TextureDesc,SubResourceData,OutTexture2D),
			UEFormat,
			TextureDesc->Width,
			TextureDesc->Height,
			TextureDesc->Depth,
			TextureDesc->Format,
			TextureDesc->MipLevels,
			TextureDesc->BindFlags,
			TextureDesc->Usage,
			TextureDesc->CPUAccessFlags,
			TextureDesc->MiscFlags,			
			0,
			0,
			SubResourceData ? SubResourceData->pSysMem : nullptr,
			SubResourceData ? SubResourceData->SysMemPitch : 0,
			SubResourceData ? SubResourceData->SysMemSlicePitch : 0,
			Direct3DDevice,
			DebugName
			);
#if GUARDED_TEXTURE_CREATES
		bDriverCrash = false;
	}
	__finally
	{
		if (bDriverCrash)
		{
			UE_LOGF(LogD3D11RHI,Error,
				"Driver crashed while creating texture: %ux%ux%u %ls(0x%08x) with %u mips, PF_ %d",
				TextureDesc->Width,
				TextureDesc->Height,
				TextureDesc->Depth,
				UE::DXGIUtilities::GetFormatString(TextureDesc->Format),
				(uint32)TextureDesc->Format,
				TextureDesc->MipLevels,
				UEFormat
				);
		}
	}
#endif // #if GUARDED_TEXTURE_CREATES
}

FD3D11Texture* FD3D11DynamicRHI::BeginCreateTextureInternal(const FRHITextureCreateDesc& CreateDesc)
{
	FD3D11Texture* Texture = new FD3D11Texture(CreateDesc);
	return Texture;
}

inline uint32 GetMaxMSAAQuality(uint32 SampleCount)
{
	if (SampleCount <= DX_MAX_MSAA_COUNT)
	{
		return 0;
	}
	return 0xffffffff;
}

enum class ED3D11TextureCreateViewFlags
{
	None        = 0,
	SRV         = 1 << 0,
	RTV         = 1 << 1,
	DSV         = 1 << 2,
};
ENUM_CLASS_FLAGS(ED3D11TextureCreateViewFlags);

// Setup a D3D11_TEXTURE#D_DESC structure and return a set of flags telling the caller which view types should be created.
template<typename TDesc>
static ED3D11TextureCreateViewFlags SetupD3D11TextureCommonDesc(TDesc& D3D11Desc, const FRHITextureDesc& TextureDesc, DXGI_FORMAT PlatformResourceFormat)
{
	FMemory::Memzero(D3D11Desc);

	const bool                bTextureArray = TextureDesc.IsTextureArray();
	const bool                bCubeTexture = TextureDesc.IsTextureCube();
	const uint32              SizeX = TextureDesc.Extent.X;
	const uint32              SizeY = TextureDesc.Extent.Y;
	const uint32              SizeZ = bCubeTexture ? TextureDesc.ArraySize * 6 : TextureDesc.ArraySize;
	const EPixelFormat        Format = TextureDesc.Format;
	const uint32              NumMips = TextureDesc.NumMips;
	const uint32              NumSamples = TextureDesc.NumSamples;
	const ETextureCreateFlags Flags = TextureDesc.Flags;

	check(SizeX > 0 && SizeY > 0 && NumMips > 0);

	const bool bSRGB = EnumHasAnyFlags(Flags, ETextureCreateFlags::SRGB);

	uint32 CPUAccessFlags = 0;
	D3D11_USAGE TextureUsage = D3D11_USAGE_DEFAULT;
	bool bCreateShaderResource = true;

	uint32 ActualMSAAQuality = GetMaxMSAAQuality(NumSamples);
	check(ActualMSAAQuality != 0xffffffff);
	check(NumSamples == 1 || !EnumHasAnyFlags(Flags, ETextureCreateFlags::Shared));

	const bool bIsMultisampled = NumSamples > 1;

	if (EnumHasAnyFlags(Flags, ETextureCreateFlags::CPUReadback))
	{
		check(!EnumHasAnyFlags(Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::DepthStencilTargetable | ETextureCreateFlags::ShaderResource));

		CPUAccessFlags = D3D11_CPU_ACCESS_READ;
		TextureUsage = D3D11_USAGE_STAGING;
		bCreateShaderResource = false;
	}

	if (EnumHasAnyFlags(Flags, ETextureCreateFlags::CPUWritable))
	{
		CPUAccessFlags = D3D11_CPU_ACCESS_WRITE;
		TextureUsage = D3D11_USAGE_STAGING;
		bCreateShaderResource = false;
	}

	// Describe the texture.
	D3D11Desc.Width = SizeX;
	D3D11Desc.Height = SizeY;
	D3D11Desc.MipLevels = NumMips;
	D3D11Desc.Format = PlatformResourceFormat;
	D3D11Desc.Usage = TextureUsage;
	D3D11Desc.BindFlags = bCreateShaderResource ? D3D11_BIND_SHADER_RESOURCE : 0;
	D3D11Desc.CPUAccessFlags = CPUAccessFlags;
	D3D11Desc.MiscFlags = bCubeTexture ? D3D11_RESOURCE_MISC_TEXTURECUBE : 0;

	// NV12/P010 doesn't support SRV in NV12 format so don't create SRV for it.
	// Todo: add support for SRVs of underneath luminance & chrominance textures.
	if (Format == PF_NV12 || Format == PF_P010)
	{
		// This has to be set after the bind flags because it is valid to bind R8 or B8G8 to this
		// and creating a SRV afterward would fail because of the missing bind flags
		bCreateShaderResource = false;
	}

	if (EnumHasAnyFlags(Flags, ETextureCreateFlags::DisableSRVCreation))
	{
		bCreateShaderResource = false;
	}

	if (EnumHasAnyFlags(Flags, ETextureCreateFlags::Shared))
	{
		if (GCVarUseSharedKeyedMutex->GetInt() != 0)
		{
			D3D11Desc.MiscFlags |= D3D11_RESOURCE_MISC_SHARED_KEYEDMUTEX;
		}
		else
		{
			D3D11Desc.MiscFlags |= D3D11_RESOURCE_MISC_SHARED;
		}
	}

	// Set up the texture bind flags.
	bool bCreateRTV = false;
	bool bCreateDSV = false;

	if (EnumHasAnyFlags(Flags, ETextureCreateFlags::RenderTargetable))
	{
		check(!EnumHasAnyFlags(Flags, ETextureCreateFlags::DepthStencilTargetable | ETextureCreateFlags::ResolveTargetable));
		D3D11Desc.BindFlags |= D3D11_BIND_RENDER_TARGET;
		bCreateRTV = true;
	}
	else if (EnumHasAnyFlags(Flags, ETextureCreateFlags::DepthStencilTargetable))
	{
		check(!EnumHasAnyFlags(Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ResolveTargetable));
		D3D11Desc.BindFlags |= D3D11_BIND_DEPTH_STENCIL;
		bCreateDSV = true;
	}
	else if (EnumHasAnyFlags(Flags, ETextureCreateFlags::ResolveTargetable))
	{
		check(!EnumHasAnyFlags(Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::DepthStencilTargetable));
		if (Format == PF_DepthStencil || Format == PF_ShadowDepth || Format == PF_D24)
		{
			D3D11Desc.BindFlags |= D3D11_BIND_DEPTH_STENCIL;
			bCreateDSV = true;
		}
		else
		{
			D3D11Desc.BindFlags |= D3D11_BIND_RENDER_TARGET;
			bCreateRTV = true;
		}
	}
	// NV12 doesn't support RTV in NV12 format so don't create RTV for it.
	// Todo: add support for RTVs of underneath luminance & chrominance textures.
	if (Format == PF_NV12 || Format == PF_P010)
	{
		bCreateRTV = false;
	}

	if (EnumHasAnyFlags(Flags, ETextureCreateFlags::UAV))
	{
		D3D11Desc.BindFlags |= D3D11_BIND_UNORDERED_ACCESS;
	}

	if (bCreateDSV && !EnumHasAnyFlags(Flags, ETextureCreateFlags::ShaderResource))
	{
		D3D11Desc.BindFlags &= ~D3D11_BIND_SHADER_RESOURCE;
		bCreateShaderResource = false;
	}

	ED3D11TextureCreateViewFlags CreateViewFlags = ED3D11TextureCreateViewFlags::None;
	if (bCreateShaderResource)
	{
		EnumAddFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::SRV);
	}
	if (bCreateRTV)
	{
		EnumAddFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::RTV);
	}
	if (bCreateDSV)
	{
		EnumAddFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::DSV);
	}
	return CreateViewFlags;
}

TRefCountPtr<ID3D11RenderTargetView> CreateRTV(FD3D11Device* Direct3DDevice, ID3D11Resource* Resource, const FRHITextureDesc& TextureDesc, DXGI_FORMAT PlatformResourceFormat, uint32 MipIndex, uint32 SliceIndex, uint32 SliceCount)
{
	D3D11_RENDER_TARGET_VIEW_DESC RTVDesc;
	FMemory::Memzero(RTVDesc);

	RTVDesc.Format = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::SRGB));

	if (TextureDesc.IsTexture3D())
	{
		RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE3D;
		RTVDesc.Texture3D.MipSlice = MipIndex;
		RTVDesc.Texture3D.FirstWSlice = 0;
		RTVDesc.Texture3D.WSize = TextureDesc.Depth;
	}
	else if (TextureDesc.IsTextureArray() || TextureDesc.IsTextureCube())
	{
		if (TextureDesc.NumSamples > 1)
		{
			RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DMSARRAY;
			RTVDesc.Texture2DMSArray.FirstArraySlice = SliceIndex;
			RTVDesc.Texture2DMSArray.ArraySize = SliceCount;
		}
		else
		{
			RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DARRAY;
			RTVDesc.Texture2DArray.FirstArraySlice = SliceIndex;
			RTVDesc.Texture2DArray.ArraySize = SliceCount;
			RTVDesc.Texture2DArray.MipSlice = MipIndex;
		}
	}
	else
	{
		if (TextureDesc.NumSamples > 1)
		{
			RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DMS;
			// Nothing to set
		}
		else
		{
			RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2D;
			RTVDesc.Texture2D.MipSlice = MipIndex;
		}
	}

	TRefCountPtr<ID3D11RenderTargetView> RenderTargetView;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateRenderTargetView(Resource, &RTVDesc, RenderTargetView.GetInitReference()), Direct3DDevice);

	return RenderTargetView;
}

TRefCountPtr<ID3D11ShaderResourceView> CreateSRV(FD3D11Device* Direct3DDevice, ID3D11Resource* Resource, const FRHITextureDesc& TextureDesc, DXGI_FORMAT PlatformResourceFormat)
{
	const DXGI_FORMAT PlatformShaderResourceFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::SRGB));

	D3D11_SHADER_RESOURCE_VIEW_DESC SRVDesc;
	FMemory::Memzero(SRVDesc);

	SRVDesc.Format = PlatformShaderResourceFormat;

	switch (TextureDesc.Dimension)
	{
	case ETextureDimension::Texture2D:
		if (TextureDesc.NumSamples > 1)
		{
			SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DMS;
			// Nothing to set
		}
		else
		{
			SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2D;
			SRVDesc.Texture2D.MostDetailedMip = 0;
			SRVDesc.Texture2D.MipLevels = TextureDesc.NumMips;
		}
		break;
	case ETextureDimension::Texture2DArray:
		if (TextureDesc.NumSamples > 1)
		{
			SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DMSARRAY;
			SRVDesc.Texture2DMSArray.FirstArraySlice = 0;
			SRVDesc.Texture2DMSArray.ArraySize = TextureDesc.ArraySize;
		}
		else
		{
			SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DARRAY;
			SRVDesc.Texture2DArray.MostDetailedMip = 0;
			SRVDesc.Texture2DArray.MipLevels = TextureDesc.NumMips;
			SRVDesc.Texture2DArray.FirstArraySlice = 0;
			SRVDesc.Texture2DArray.ArraySize = TextureDesc.ArraySize;
		}
		break;
	case ETextureDimension::Texture3D:
		SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE3D;
		SRVDesc.Texture3D.MipLevels = TextureDesc.NumMips;
		SRVDesc.Texture3D.MostDetailedMip = 0;
		break;
	case ETextureDimension::TextureCube:
		SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURECUBE;
		SRVDesc.TextureCube.MostDetailedMip = 0;
		SRVDesc.TextureCube.MipLevels = TextureDesc.NumMips;
		break;
	case ETextureDimension::TextureCubeArray:
		SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURECUBEARRAY;
		SRVDesc.TextureCubeArray.MostDetailedMip = 0;
		SRVDesc.TextureCubeArray.MipLevels = TextureDesc.NumMips;
		SRVDesc.TextureCubeArray.First2DArrayFace = 0;
		SRVDesc.TextureCubeArray.NumCubes = TextureDesc.ArraySize;
		break;
	default:
		break;
	}

	TRefCountPtr<ID3D11ShaderResourceView> SRV;
	VERIFYD3D11RESULT_EX(Direct3DDevice->CreateShaderResourceView(Resource, &SRVDesc, SRV.GetInitReference()), Direct3DDevice);

	return SRV;
}

void FD3D11DynamicRHI::FinalizeCreateTexture2DInternal(FD3D11Texture* Texture, TConstArrayView<D3D11_SUBRESOURCE_DATA> InitialData)
{
	const FString DebugName = Texture->GetName().ToString();
	const FRHITextureDesc& TextureDesc = Texture->GetDesc();

	check(!TextureDesc.IsTexture3D());

	const bool                bTextureArray = TextureDesc.IsTextureArray();
	const bool                bCubeTexture  = TextureDesc.IsTextureCube();
	const uint32              SizeX         = TextureDesc.Extent.X;
	const uint32              SizeY         = TextureDesc.Extent.Y;
	const uint32              SizeZ         = bCubeTexture ? TextureDesc.ArraySize * 6 : TextureDesc.ArraySize;
	const EPixelFormat        Format        = TextureDesc.Format;
	const uint32              NumMips       = TextureDesc.NumMips;
	const uint32              NumSamples    = TextureDesc.NumSamples;
	const ETextureCreateFlags Flags         = TextureDesc.Flags;

	check(SizeX > 0 && SizeY > 0 && NumMips > 0);

	if (bCubeTexture)
	{
		checkf(SizeX <= GetMaxCubeTextureDimension(), TEXT("Requested cube texture size too large: %i, Max: %i, DebugName: '%s'"), SizeX, GetMaxCubeTextureDimension(), *DebugName);
		check(SizeX == SizeY);
	}
	else
	{
		checkf(SizeX <= GetMax2DTextureDimension(), TEXT("Requested texture2d x size too large: %i, Max: %i, DebugName: '%s'"), SizeX, GetMax2DTextureDimension(), *DebugName);
		checkf(SizeY <= GetMax2DTextureDimension(), TEXT("Requested texture2d y size too large: %i, Max: %i, DebugName: '%s'"), SizeY, GetMax2DTextureDimension(), *DebugName);
	}

	if (bTextureArray)
	{
		checkf(SizeZ <= GetMaxTextureArrayLayers(), TEXT("Requested texture array size too large: %i, Max: %i, DebugName: '%s'"), SizeZ, GetMaxTextureArrayLayers(), *DebugName);
	}

	SCOPE_CYCLE_COUNTER(STAT_D3D11CreateTextureTime);

	const DXGI_FORMAT PlatformResourceFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat(Format, Flags);

	const uint32 ActualMSAAQuality = GetMaxMSAAQuality(TextureDesc.NumSamples);
	check(ActualMSAAQuality != 0xffffffff);
	check(TextureDesc.NumSamples == 1 || !EnumHasAnyFlags(Flags, ETextureCreateFlags::Shared));

	// Describe the texture.
	D3D11_TEXTURE2D_DESC Desc;
	
	ED3D11TextureCreateViewFlags CreateViewFlags = SetupD3D11TextureCommonDesc(Desc, TextureDesc, PlatformResourceFormat);

	// Texture2D specific vars
	Desc.ArraySize = SizeZ;
	Desc.SampleDesc.Count = NumSamples;
	Desc.SampleDesc.Quality = ActualMSAAQuality;

	ApplyBC7SoftwareAdapterWorkaround(Adapter.bSoftwareAdapter, Desc);

	const D3D11_SUBRESOURCE_DATA* pSubresourceData = nullptr;
	if (InitialData.Num())
	{
		// Caller provided initial data.
		check(InitialData.Num() == NumMips * SizeZ);
		pSubresourceData = InitialData.GetData();
	}

	TRefCountPtr<ID3D11Texture2D> TextureResource;

#if INTEL_EXTENSIONS
	if (EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::Atomic64Compatible) && IsRHIDeviceIntel() && GRHIGlobals.SupportsAtomicUInt64)
	{
		INTC_D3D11_TEXTURE2D_DESC IntelDesc{};
		IntelDesc.EmulatedTyped64bitAtomics = true;
		IntelDesc.pD3D11Desc = &Desc;

		VERIFYD3D11RESULT(INTC_D3D11_CreateTexture2D(IntelExtensionContext, &IntelDesc, pSubresourceData, TextureResource.GetInitReference()));
	}
	else
#endif
	{
		SafeCreateTexture2D(Direct3DDevice, Format, &Desc, pSubresourceData, TextureResource.GetInitReference(), *DebugName);
	}

#if RHI_USE_RESOURCE_DEBUG_NAME
	if (Texture->GetName() != NAME_None)
	{
		SetD3D11ObjectName(TextureResource, DebugName);
	}
#endif

	TArray<TRefCountPtr<ID3D11RenderTargetView>> RenderTargetViews;
	bool bCreatedRTVPerSlice = false;

	if (EnumHasAnyFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::RTV))
	{
		if (EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::TargetArraySlicesIndependently) && (TextureDesc.IsTextureArray() || TextureDesc.IsTextureCube()))
		{
			bCreatedRTVPerSlice = true;

			for (uint32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
			{
				for (uint32 SliceIndex = 0; SliceIndex < Desc.ArraySize; SliceIndex++)
				{
					RenderTargetViews.Emplace(CreateRTV(Direct3DDevice, TextureResource, TextureDesc, PlatformResourceFormat, MipIndex, SliceIndex, 1));
				}
			}
		}
		else
		{
			for (uint32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
			{
				RenderTargetViews.Emplace(CreateRTV(Direct3DDevice, TextureResource, TextureDesc, PlatformResourceFormat, MipIndex, 0, Desc.ArraySize));
			}
		}
	}
	
	TRefCountPtr<ID3D11DepthStencilView> DepthStencilViews[FExclusiveDepthStencil::MaxIndex];

	if (EnumHasAnyFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::DSV))
	{
		const D3D11_DSV_DIMENSION DSVDimension =
			(bTextureArray || bCubeTexture)
			? (TextureDesc.NumSamples > 1 ? D3D11_DSV_DIMENSION_TEXTURE2DMSARRAY : D3D11_DSV_DIMENSION_TEXTURE2DARRAY)
			: (TextureDesc.NumSamples > 1 ? D3D11_DSV_DIMENSION_TEXTURE2DMS : D3D11_DSV_DIMENSION_TEXTURE2D);

		// Create a depth-stencil-view for the texture.
		CD3D11_DEPTH_STENCIL_VIEW_DESC DSVDesc(
			DSVDimension,
			UE::DXGIUtilities::FindDepthStencilFormat((DXGI_FORMAT)GPixelFormats[TextureDesc.Format].PlatformFormat),
			0, // MipSlice
			0, // FirstArraySlice
			Desc.ArraySize,
			0 // Flags
		);

		for (uint32 AccessType = 0; AccessType < FExclusiveDepthStencil::MaxIndex; ++AccessType)
		{
			// Create a read-only access views for the texture.
			// Read-only DSVs are not supported in Feature Level 10 so 
			// a dummy DSV is created in order reduce logic complexity at a higher-level.
			DSVDesc.Flags = (AccessType & FExclusiveDepthStencil::DepthRead_StencilWrite) ? D3D11_DSV_READ_ONLY_DEPTH : 0;
			if(UE::DXGIUtilities::HasStencilBits(DSVDesc.Format))
			{
				DSVDesc.Flags |= (AccessType & FExclusiveDepthStencil::DepthWrite_StencilRead) ? D3D11_DSV_READ_ONLY_STENCIL : 0;
			}
			VERIFYD3D11RESULT_EX(Direct3DDevice->CreateDepthStencilView(TextureResource,&DSVDesc,DepthStencilViews[AccessType].GetInitReference()), Direct3DDevice);
		}
	}
	check(IsValidRef(TextureResource));

	// Create a shader resource view for the texture.
	TRefCountPtr<ID3D11ShaderResourceView> ShaderResourceView;

	if (EnumHasAnyFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::SRV))
	{
		ShaderResourceView = CreateSRV(Direct3DDevice, TextureResource, TextureDesc, PlatformResourceFormat);
		check(IsValidRef(ShaderResourceView));
	}

	Texture->FinalizeCreation(
		TextureResource,
		ShaderResourceView,
		Desc.ArraySize,
		bCreatedRTVPerSlice,
		RenderTargetViews,
		DepthStencilViews
	);
}

void FD3D11DynamicRHI::FinalizeCreateTexture3DInternal(FD3D11Texture* Texture, TConstArrayView<D3D11_SUBRESOURCE_DATA> InitialData)
{
	const FString DebugName = Texture->GetName().ToString();
	const FRHITextureDesc& TextureDesc = Texture->GetDesc();

	check(TextureDesc.IsTexture3D());
	check(TextureDesc.ArraySize == 1);

	SCOPE_CYCLE_COUNTER(STAT_D3D11CreateTextureTime);

	// Set up the texture bind flags.
	check(!EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::DepthStencilTargetable | ETextureCreateFlags::ResolveTargetable));

	const DXGI_FORMAT PlatformResourceFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat(TextureDesc.Format, TextureDesc.Flags);

	// Describe the texture.
	D3D11_TEXTURE3D_DESC Desc;

	const ED3D11TextureCreateViewFlags CreateViewFlags = SetupD3D11TextureCommonDesc(Desc, TextureDesc, PlatformResourceFormat);

	// Texture3D specific vars
	Desc.Depth = TextureDesc.Depth;

	const D3D11_SUBRESOURCE_DATA* pSubresourceData = nullptr;
	if (InitialData.Num())
	{
		// Caller provided initial data.
		check(InitialData.Num() == TextureDesc.NumMips);
		pSubresourceData = InitialData.GetData();
	}

	TRefCountPtr<ID3D11Texture3D> TextureResource;
	SafeCreateTexture3D(Direct3DDevice, TextureDesc.Format, &Desc, pSubresourceData, TextureResource.GetInitReference(), *DebugName);

#if RHI_USE_RESOURCE_DEBUG_NAME
	if (Texture->GetName() != NAME_None)
	{
		SetD3D11ObjectName(TextureResource, DebugName);
	}
#endif

	// Create a shader resource view for the texture.
	TRefCountPtr<ID3D11ShaderResourceView> ShaderResourceView;

	if (EnumHasAnyFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::SRV))
	{
		ShaderResourceView = CreateSRV(Direct3DDevice, TextureResource, TextureDesc, PlatformResourceFormat);
	}

	TRefCountPtr<ID3D11RenderTargetView> RenderTargetView;

	if (EnumHasAnyFlags(CreateViewFlags, ED3D11TextureCreateViewFlags::RTV))
	{
		RenderTargetView = CreateRTV(Direct3DDevice, TextureResource, TextureDesc, PlatformResourceFormat, 0, 0, 1);
	}

	Texture->FinalizeCreation(
		TextureResource,
		ShaderResourceView,
		1,     // InRTVArraySize
		false, // bInCreatedRTVsPerSlice
		{ RenderTargetView },
		{}
	);
}

static const D3D11_SUBRESOURCE_DATA* FillSubresourceData(TArray<D3D11_SUBRESOURCE_DATA>& SubresourceData, const FRHITextureDesc& CreateDesc, TConstArrayView<uint8> InitialData)
{
	const D3D11_SUBRESOURCE_DATA* pSubresourceData = nullptr;

	if (InitialData.Num())
	{
		const FPixelFormatInfo& PixelFormat = GPixelFormats[CreateDesc.Format];

		const uint32 FaceCount = CreateDesc.IsTextureCube() ? 6 : 1;
		const uint32 ArrayCount = CreateDesc.ArraySize * FaceCount;
		const uint32 MipCount = CreateDesc.NumMips;

		// each mip of each array slice counts as a subresource
		SubresourceData.AddZeroed(MipCount * ArrayCount);

		uint32 SliceOffset = 0;
		for (uint32 ArraySliceIndex = 0; ArraySliceIndex < ArrayCount; ++ArraySliceIndex)
		{
			uint32 MipOffset = 0;
			for (uint32 MipIndex = 0; MipIndex < MipCount; ++MipIndex)
			{
				const uint32 DataOffset = SliceOffset + MipOffset;
				const uint32 SubResourceIndex = ArraySliceIndex * MipCount + MipIndex;

				const FUintVector3 BlockCounts = UE::RHITextureUtils::CalculateMipBlockCounts(CreateDesc, MipIndex, PixelFormat);

				const uint32 RowPitch = BlockCounts.X * PixelFormat.BlockBytes;
				const uint32 SlicePitch = BlockCounts.Y * RowPitch;

				SubresourceData[SubResourceIndex].pSysMem          = &InitialData[DataOffset];
				SubresourceData[SubResourceIndex].SysMemPitch      = RowPitch;
				SubresourceData[SubResourceIndex].SysMemSlicePitch = SlicePitch;

				MipOffset += SlicePitch * BlockCounts.Z;
			}

			SliceOffset += MipOffset;
		}

		pSubresourceData = SubresourceData.GetData();
	}

	return pSubresourceData;
}

FD3D11Texture* FD3D11DynamicRHI::FinalizeCreateTextureInternal(FD3D11Texture* Texture, const FRHITextureDesc& InDesc, TConstArrayView<uint8> InitialData)
{
	TArray<D3D11_SUBRESOURCE_DATA> SubresourceData;
	FillSubresourceData(SubresourceData, InDesc, InitialData);

	if (InDesc.IsTexture3D())
	{
		FinalizeCreateTexture3DInternal(Texture, TConstArrayView<D3D11_SUBRESOURCE_DATA>(SubresourceData));
	}
	else
	{
		FinalizeCreateTexture2DInternal(Texture, TConstArrayView<D3D11_SUBRESOURCE_DATA>(SubresourceData));
	}

	return Texture;
}

FD3D11Texture* FD3D11DynamicRHI::CreateTextureInternal(const FRHITextureCreateDesc& CreateDesc, TConstArrayView<uint8> InitialData)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);
	FD3D11Texture* Texture = BeginCreateTextureInternal(CreateDesc);
	return FinalizeCreateTextureInternal(Texture, CreateDesc, InitialData);
}

FRHITextureInitializer FD3D11DynamicRHI::RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	if (CreateDesc.InitAction == ERHITextureInitAction::Default)
	{
		return UE::RHICore::FDefaultTextureInitializer(RHICmdList, CreateTextureInternal(CreateDesc, {}));
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::BulkData)
	{
		check(CreateDesc.BulkData);

		FD3D11Texture* Texture = CreateTextureInternal(CreateDesc, CreateDesc.BulkData->GetBulkDataView<uint8>());

		// Discard the bulk data's contents.
		CreateDesc.BulkData->Discard();

		return UE::RHICore::FDefaultTextureInitializer(RHICmdList, Texture);
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::Initializer)
	{
		FD3D11Texture* Texture = BeginCreateTextureInternal(CreateDesc);

		const uint64 UploadMemorySize = UE::RHITextureUtils::CalculateTextureSize(CreateDesc);
		void* UploadMemory = FMemory::Malloc(UploadMemorySize, 16);

		return UE::RHICore::FDefaultLayoutTextureInitializer(RHICmdList, Texture, UploadMemory, UploadMemorySize,
			[this, Texture = TRefCountPtr<FD3D11Texture>(Texture), UploadMemory = UE::RHICore::FInitializerScopedMemory(UploadMemory), UploadMemorySize](FRHICommandListBase& RHICmdList) mutable
			{
				TConstArrayView<uint8> InitialData(reinterpret_cast<const uint8*>(UploadMemory.Pointer), UploadMemorySize);

				return TRefCountPtr<FRHITexture>(FinalizeCreateTextureInternal(Texture, Texture->GetDesc(), InitialData));
			});
	}

	return UE::RHICore::HandleUnknownTextureInitializerInitAction(RHICmdList, CreateDesc);
}

FTextureRHIRef FD3D11DynamicRHI::RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent)
{
	const FPixelFormatInfo& PixelFormat = GPixelFormats[Format];

	TArray<D3D11_SUBRESOURCE_DATA, TInlineAllocator<12>> SubresourceData;
	SubresourceData.SetNumUninitialized(NumMips);

	for (uint32 MipIndex = 0; MipIndex < NumInitialMips; ++MipIndex)
	{
		const uint32 NumBlocksX = UE::RHITextureUtils::CalculateMipBlockCount(SizeX, MipIndex, PixelFormat.BlockSizeX);
		const uint32 NumBlocksY = UE::RHITextureUtils::CalculateMipBlockCount(SizeY, MipIndex, PixelFormat.BlockSizeY);

		SubresourceData[MipIndex].pSysMem = InitialMipData[MipIndex];
		SubresourceData[MipIndex].SysMemPitch = NumBlocksX * PixelFormat.BlockBytes;
		SubresourceData[MipIndex].SysMemSlicePitch = NumBlocksX * NumBlocksY * PixelFormat.BlockBytes;
	}

	void* TempBuffer = ZeroBuffer;
	uint32 TempBufferSize = ZeroBufferSize;
	for (uint32 MipIndex = NumInitialMips; MipIndex < NumMips; ++MipIndex)
	{
		const uint32 NumBlocksX = UE::RHITextureUtils::CalculateMipBlockCount(SizeX, MipIndex, PixelFormat.BlockSizeX);
		const uint32 NumBlocksY = UE::RHITextureUtils::CalculateMipBlockCount(SizeY, MipIndex, PixelFormat.BlockSizeY);

		uint32 MipSize = NumBlocksX * NumBlocksY * PixelFormat.BlockBytes;

		if (MipSize > TempBufferSize)
		{
			UE_LOGF(LogD3D11RHI, Verbose, "Temp texture streaming buffer not large enough, needed %d bytes", MipSize);
			check(TempBufferSize == ZeroBufferSize);
			TempBufferSize = MipSize;
			TempBuffer = FMemory::Malloc(TempBufferSize);
			FMemory::Memzero(TempBuffer, TempBufferSize);
		}

		SubresourceData[MipIndex].pSysMem = TempBuffer;
		SubresourceData[MipIndex].SysMemPitch = NumBlocksX * PixelFormat.BlockBytes;
		SubresourceData[MipIndex].SysMemSlicePitch = MipSize;
	}

	const FRHITextureCreateDesc CreateDesc =
		FRHITextureCreateDesc::Create2D(DebugName, SizeX, SizeY, (EPixelFormat)Format)
		.SetClearValue(FClearValueBinding::None)
		.SetFlags(Flags)
		.SetNumMips(NumMips)
		.DetermineInititialState();

	FD3D11Texture* Texture = BeginCreateTextureInternal(CreateDesc);
	FinalizeCreateTexture2DInternal(Texture, SubresourceData);

	if (TempBufferSize != ZeroBufferSize)
	{
		FMemory::Free(TempBuffer);
	}

	OutCompletionEvent = nullptr;

	return Texture;
}

uint32 FD3D11DynamicRHI::RHIComputeMemorySize(FRHITexture* TextureRHI)
{
	if(!TextureRHI)
	{
		return 0;
	}
	
	FD3D11Texture* Texture = ResourceCast(TextureRHI);
	return Texture->GetMemorySize();
}

FRHILockTextureResult FD3D11Texture::Lock(FD3D11DynamicRHI* D3DRHI, const FRHILockTextureArgs& Arguments, bool bForceLockDeferred)
{
	check(!IsTexture3D()); // Only 2D texture locks are implemented

	SCOPE_CYCLE_COUNTER(STAT_D3D11LockTextureTime);

	const FRHITextureDesc& Desc = this->GetDesc();

	const uint32 ArrayIndex = UE::RHICore::GetLockArrayIndex(Desc, Arguments);

	// Calculate the subresource index corresponding to the specified mip-map.
	const uint32 Subresource = D3D11CalcSubresource(Arguments.MipIndex, ArrayIndex, Desc.NumMips);

	// Calculate the dimensions of the mip-map.
	const uint32 BlockSizeX = GPixelFormats[Desc.Format].BlockSizeX;
	const uint32 BlockSizeY = GPixelFormats[Desc.Format].BlockSizeY;
	const uint32 BlockBytes = GPixelFormats[Desc.Format].BlockBytes;

	const uint32 MipSizeX = FMath::Max(uint32(Desc.Extent.X) >> Arguments.MipIndex, BlockSizeX);
	const uint32 MipSizeY = FMath::Max(uint32(Desc.Extent.Y) >> Arguments.MipIndex, BlockSizeY);
	const uint32 NumBlocksX = (MipSizeX + BlockSizeX - 1) / BlockSizeX;
	const uint32 NumBlocksY = (MipSizeY + BlockSizeY - 1) / BlockSizeY;
	const uint32 MipBytes = NumBlocksX * NumBlocksY * BlockBytes;

	FRHILockTextureResult Result;

	FD3D11LockedData LockedData;
	if (Arguments.LockMode == RLM_WriteOnly)
	{
		if (!bForceLockDeferred && EnumHasAnyFlags(Desc.Flags, TexCreate_CPUWritable))
		{
			D3D11_MAPPED_SUBRESOURCE MappedTexture{};
			VERIFYD3D11RESULT_EX(D3DRHI->GetDeviceContext()->Map(GetResource(), Subresource, D3D11_MAP_WRITE, 0, &MappedTexture), D3DRHI->GetDevice());

			LockedData.SetData(MappedTexture.pData);
			LockedData.Pitch = MappedTexture.RowPitch;
		}
		else
		{
			// If we're writing to the texture, allocate a system memory buffer to receive the new contents.
			LockedData.AllocData(MipBytes);
			LockedData.Pitch = NumBlocksX * BlockBytes;
			LockedData.bLockDeferred = true;
		}
	}
	else
	{
		check(!bForceLockDeferred);
		// If we're reading from the texture, we create a staging resource, copy the texture contents to it, and map it.

		// Create the staging texture.
		D3D11_TEXTURE2D_DESC StagingTextureDesc{};
		GetD3D11Texture2D()->GetDesc(&StagingTextureDesc);

		StagingTextureDesc.Width = MipSizeX;
		StagingTextureDesc.Height = MipSizeY;
		StagingTextureDesc.MipLevels = 1;
		StagingTextureDesc.ArraySize = 1;
		StagingTextureDesc.Usage = D3D11_USAGE_STAGING;
		StagingTextureDesc.BindFlags = 0;
		StagingTextureDesc.CPUAccessFlags = D3D11_CPU_ACCESS_READ;
		StagingTextureDesc.MiscFlags = 0;

		const FString StagingTextureName = FString::Printf(TEXT("%s_Staging"), *GetName().ToString());

		TRefCountPtr<ID3D11Texture2D> StagingTexture;
		VERIFYD3D11CREATETEXTURERESULT(
			D3DRHI->GetDevice()->CreateTexture2D(&StagingTextureDesc, NULL, StagingTexture.GetInitReference()),
			Desc.Format,
			Desc.Extent.X,
			Desc.Extent.Y,
			this->GetSizeZ(),
			StagingTextureDesc.Format,
			1,
			0,
			StagingTextureDesc.Usage,
			StagingTextureDesc.CPUAccessFlags,
			StagingTextureDesc.MiscFlags,
			StagingTextureDesc.SampleDesc.Count,
			StagingTextureDesc.SampleDesc.Quality,
			nullptr,
			0,
			0,
			D3DRHI->GetDevice(),
			*StagingTextureName
		);
		LockedData.StagingResource = StagingTexture;

		// Copy the mip-map data from the real resource into the staging resource
		D3DRHI->GetDeviceContext()->CopySubresourceRegion(StagingTexture, 0, 0, 0, 0, GetResource(), Subresource, NULL);

		// Map the staging resource, and return the mapped address.
		D3D11_MAPPED_SUBRESOURCE MappedTexture{};
		VERIFYD3D11RESULT_EX(D3DRHI->GetDeviceContext()->Map(StagingTexture, 0, D3D11_MAP_READ, 0, &MappedTexture), D3DRHI->GetDevice());

		LockedData.SetData(MappedTexture.pData);
		LockedData.Pitch = MappedTexture.RowPitch;
	}

	// Add the lock to the outstanding lock list.
	if (!bForceLockDeferred)
	{
		D3DRHI->AddLockedData(FD3D11LockedKey(GetResource(), Subresource), LockedData);
	}
	else
	{
		RunOnRHIThread([this, Subresource, LockedData, D3DRHI]()
		{
			D3DRHI->AddLockedData(FD3D11LockedKey(GetResource(), Subresource), LockedData);
		});
	}

	Result.Data = LockedData.GetData();
	Result.ByteCount = MipBytes;
	Result.Stride = LockedData.Pitch;

	return Result;
}

void FD3D11Texture::Unlock(FD3D11DynamicRHI* D3DRHI, const FRHILockTextureArgs& Arguments)
{
	check(!IsTexture3D()); // Only 2D texture locks are implemented

	SCOPE_CYCLE_COUNTER(STAT_D3D11UnlockTextureTime);

	const FRHITextureDesc& Desc = this->GetDesc();
	const uint32 ArrayIndex = UE::RHICore::GetLockArrayIndex(Desc, Arguments);

	// Calculate the subresource index corresponding to the specified mip-map.
	const uint32 Subresource = D3D11CalcSubresource(Arguments.MipIndex, ArrayIndex, Desc.NumMips);

	// Find the object that is tracking this lock and remove it from outstanding list
	FD3D11LockedData LockedData;
	verifyf(D3DRHI->RemoveLockedData(FD3D11LockedKey(GetResource(), Subresource), LockedData), TEXT("Texture is not locked"));

	if (!LockedData.bLockDeferred && EnumHasAnyFlags(Desc.Flags, TexCreate_CPUWritable))
	{
		D3DRHI->GetDeviceContext()->Unmap(GetResource(), 0);
	}
	else if(!LockedData.StagingResource)
	{
		// If we're writing, we need to update the subresource
		D3DRHI->GetDeviceContext()->UpdateSubresource(GetResource(), Subresource, NULL, LockedData.GetData(), LockedData.Pitch, 0);
		LockedData.FreeData();
	}
	else
	{
		D3DRHI->GetDeviceContext()->Unmap(LockedData.StagingResource, 0);
	}
}

FRHILockTextureResult FD3D11DynamicRHI::RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	FD3D11Texture* Texture = ResourceCast(Arguments.Texture);

	if (ShouldNotEnqueueRHICommand())
	{
		ConditionalClearShaderResource(Texture, false);
		return Texture->Lock(this, Arguments, false /* bForceLockDeferred */);
	}

	if (Arguments.LockMode == RLM_ReadOnly)
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

		ConditionalClearShaderResource(Texture, false);
		return Texture->Lock(this, Arguments, false /* bForceLockDeferred */);
	}

	return Texture->Lock(this, Arguments, true /* bForceLockDeferred */);
}

void FD3D11DynamicRHI::RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	FD3D11Texture* Texture = ResourceCast(Arguments.Texture);

	if (ShouldNotEnqueueRHICommand())
	{
		Texture->Unlock(this, Arguments);
	}
	else
	{
		RunOnRHIThread([this, Texture, Arguments]()
		{
			Texture->Unlock(this, Arguments);
		});
	}
}

void FD3D11DynamicRHI::RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
{
	const FPixelFormatInfo& FormatInfo = GPixelFormats[TextureRHI->GetFormat()];

	check(UpdateRegion.Width  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.Height % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.DestX  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.DestY  % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.SrcX   % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.SrcY   % FormatInfo.BlockSizeY == 0);

	const uint32 SrcXInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcX,   FormatInfo.BlockSizeX);
	const uint32 SrcYInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcY,   FormatInfo.BlockSizeY);
	const uint32 WidthInBlocks  = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Width,  FormatInfo.BlockSizeX);
	const uint32 HeightInBlocks = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Height, FormatInfo.BlockSizeY);

	const void* UpdateMemory = SourceData + FormatInfo.BlockBytes * SrcXInBlocks + SourcePitch * SrcYInBlocks;
	uint32 UpdatePitch = SourcePitch;

	const bool bNeedStagingMemory = RHICmdList.IsTopOfPipe();
	if (bNeedStagingMemory)
	{
		const size_t SourceDataSizeInBlocks = static_cast<size_t>(WidthInBlocks) * static_cast<size_t>(HeightInBlocks);
		const size_t SourceDataSize = SourceDataSizeInBlocks * FormatInfo.BlockBytes;

		uint8* const StagingMemory = (uint8*)FMemory::Malloc(SourceDataSize);
		const size_t StagingPitch = static_cast<size_t>(WidthInBlocks) * FormatInfo.BlockBytes;

		const uint8* CopySrc = (const uint8*)UpdateMemory;
		uint8* CopyDst = (uint8*)StagingMemory;
		for (uint32 BlockRow = 0; BlockRow < HeightInBlocks; BlockRow++)
		{
			FMemory::Memcpy(CopyDst, CopySrc, WidthInBlocks * FormatInfo.BlockBytes);
			CopySrc += SourcePitch;
			CopyDst += StagingPitch;
		}

		UpdateMemory = StagingMemory;
		UpdatePitch = StagingPitch;
	}

	RHICmdList.EnqueueLambda([this, TextureRHI, MipIndex, UpdateRegion, UpdatePitch, UpdateMemory, bNeedStagingMemory] (FRHICommandListBase&)
	{
		FD3D11Texture* Texture = ResourceCast(TextureRHI);

		D3D11_BOX DestBox =
		{
			UpdateRegion.DestX,                      UpdateRegion.DestY,                       0,
			UpdateRegion.DestX + UpdateRegion.Width, UpdateRegion.DestY + UpdateRegion.Height, 1
		};

		Direct3DDeviceIMContext->UpdateSubresource(Texture->GetResource(), MipIndex, &DestBox, UpdateMemory, UpdatePitch, 0);

		if (bNeedStagingMemory)
		{
			FMemory::Free(const_cast<void*>(UpdateMemory));
		}
	});
}

void FD3D11DynamicRHI::RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI,uint32 MipIndex,const FUpdateTextureRegion3D& UpdateRegion,uint32 SourceRowPitch,uint32 SourceDepthPitch,const uint8* SourceData)
{
	const SIZE_T SourceDataSize = static_cast<SIZE_T>(SourceDepthPitch) * UpdateRegion.Depth;
	uint8* SourceDataCopy = (uint8*)FMemory::Malloc(SourceDataSize);
	FMemory::Memcpy(SourceDataCopy, SourceData, SourceDataSize);
	SourceData = SourceDataCopy;

	RHICmdList.EnqueueLambda([this, TextureRHI, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, SourceData] (FRHICommandListBase&)
	{
		FD3D11Texture* Texture = ResourceCast(TextureRHI);

		// The engine calls this with the texture size in the region. 
		// Some platforms like D3D11 needs that to be rounded up to the block size.
		const FPixelFormatInfo& Format = GPixelFormats[Texture->GetFormat()];
		const int32 NumBlockX = FMath::DivideAndRoundUp<int32>(UpdateRegion.Width, Format.BlockSizeX);
		const int32 NumBlockY = FMath::DivideAndRoundUp<int32>(UpdateRegion.Height, Format.BlockSizeY);

		D3D11_BOX DestBox =
		{
			UpdateRegion.DestX,                                 UpdateRegion.DestY,                                 UpdateRegion.DestZ,
			UpdateRegion.DestX + NumBlockX * Format.BlockSizeX, UpdateRegion.DestY + NumBlockY * Format.BlockSizeY, UpdateRegion.DestZ + UpdateRegion.Depth
		};

		Direct3DDeviceIMContext->UpdateSubresource(Texture->GetResource(), MipIndex, &DestBox, SourceData, SourceRowPitch, SourceDepthPitch);

		FMemory::Free((void*)SourceData);
	});
}

void FD3D11DynamicRHI::RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData)
{
	RHIUpdateTexture3D(RHICmdList, UpdateData.Texture, UpdateData.MipIndex, UpdateData.UpdateRegion, UpdateData.RowPitch, UpdateData.DepthPitch, UpdateData.Data);
	FMemory::Free(UpdateData.Data);
	UpdateData.Data = nullptr;
}

void FD3D11DynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	TextureRHI->SetName(Name);

	SetD3D11ResourceName(ResourceCast(TextureRHI), Name);
#endif
}

FD3D11Texture* FD3D11DynamicRHI::CreateTextureFromResource(bool bTextureArray, bool bCubeTexture, EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* TextureResource)
{
	D3D11_TEXTURE2D_DESC TextureDesc;
	TextureResource->GetDesc(&TextureDesc);

	const bool bSRGB = EnumHasAnyFlags(TexCreateFlags, TexCreate_SRGB);

	const DXGI_FORMAT PlatformResourceFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat(Format, TexCreateFlags);
	const DXGI_FORMAT PlatformShaderResourceFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, bSRGB);
	const DXGI_FORMAT PlatformRenderTargetFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, bSRGB);

	const bool bIsMultisampled = TextureDesc.SampleDesc.Count > 1;

	TRefCountPtr<ID3D11ShaderResourceView> ShaderResourceView;
	TArray<TRefCountPtr<ID3D11RenderTargetView> > RenderTargetViews;
	TRefCountPtr<ID3D11DepthStencilView> DepthStencilViews[FExclusiveDepthStencil::MaxIndex];

	bool bCreateRTV = (TextureDesc.BindFlags & D3D11_BIND_RENDER_TARGET) != 0;
	bool bCreateDSV = (TextureDesc.BindFlags & D3D11_BIND_DEPTH_STENCIL) != 0;
	bool bCreateShaderResource = (TextureDesc.BindFlags & D3D11_BIND_SHADER_RESOURCE) != 0;

	// DXGI_FORMAT_NV12 allows us to create RTV and SRV but only with other formats, so we should block creation here.
	// @todo: Should this be a check? Seems wrong to just silently change what the caller asked for.
	if (Format == PF_NV12 || Format == PF_P010)
	{
		bCreateRTV = false;
		bCreateShaderResource = false;
	}

	bool bCreatedRTVPerSlice = false;

	if(bCreateRTV)
	{
		// Create a render target view for each mip
		for (uint32 MipIndex = 0; MipIndex < TextureDesc.MipLevels; MipIndex++)
		{
			if (EnumHasAnyFlags(TexCreateFlags, TexCreate_TargetArraySlicesIndependently) && (bTextureArray || bCubeTexture))
			{
				bCreatedRTVPerSlice = true;

				for (uint32 SliceIndex = 0; SliceIndex < TextureDesc.ArraySize; SliceIndex++)
				{
					D3D11_RENDER_TARGET_VIEW_DESC RTVDesc;
					FMemory::Memzero(RTVDesc);

					RTVDesc.Format = PlatformRenderTargetFormat;

					if (bIsMultisampled)
					{
						RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DMSARRAY;
						RTVDesc.Texture2DMSArray.FirstArraySlice = SliceIndex;
						RTVDesc.Texture2DMSArray.ArraySize = 1;
					}
					else
					{
						RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DARRAY;
						RTVDesc.Texture2DArray.FirstArraySlice = SliceIndex;
						RTVDesc.Texture2DArray.ArraySize = 1;
						RTVDesc.Texture2DArray.MipSlice = MipIndex;
					}

					TRefCountPtr<ID3D11RenderTargetView> RenderTargetView;
					VERIFYD3D11RESULT_EX(Direct3DDevice->CreateRenderTargetView(TextureResource,&RTVDesc,RenderTargetView.GetInitReference()), Direct3DDevice);
					RenderTargetViews.Add(RenderTargetView);
				}
			}
			else
			{
				D3D11_RENDER_TARGET_VIEW_DESC RTVDesc;
				FMemory::Memzero(RTVDesc);

				RTVDesc.Format = PlatformRenderTargetFormat;

				if (bTextureArray || bCubeTexture)
				{
					if (bIsMultisampled)
					{
						RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DMSARRAY;
						RTVDesc.Texture2DMSArray.FirstArraySlice = 0;
						RTVDesc.Texture2DMSArray.ArraySize = TextureDesc.ArraySize;
					}
					else
					{
						RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DARRAY;
						RTVDesc.Texture2DArray.FirstArraySlice = 0;
						RTVDesc.Texture2DArray.ArraySize = TextureDesc.ArraySize;
						RTVDesc.Texture2DArray.MipSlice = MipIndex;
					}
				}
				else
				{
					if (bIsMultisampled)
					{
						RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2DMS;
						// Nothing to set
					}
					else
					{
						RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2D;
						RTVDesc.Texture2D.MipSlice = MipIndex;
					}
				}

				TRefCountPtr<ID3D11RenderTargetView> RenderTargetView;
				VERIFYD3D11RESULT_EX(Direct3DDevice->CreateRenderTargetView(TextureResource,&RTVDesc,RenderTargetView.GetInitReference()), Direct3DDevice);
				RenderTargetViews.Add(RenderTargetView);
			}
		}
	}

	if(bCreateDSV)
	{
		// Create a depth-stencil-view for the texture.
		D3D11_DEPTH_STENCIL_VIEW_DESC DSVDesc;
		FMemory::Memzero(DSVDesc);

		DSVDesc.Format = UE::DXGIUtilities::FindDepthStencilFormat(PlatformResourceFormat);

		if (bTextureArray || bCubeTexture)
		{
			if (bIsMultisampled)
			{
				DSVDesc.ViewDimension = D3D11_DSV_DIMENSION_TEXTURE2DMSARRAY;
				DSVDesc.Texture2DMSArray.FirstArraySlice = 0;
				DSVDesc.Texture2DMSArray.ArraySize = TextureDesc.ArraySize;
			}
			else
			{
				DSVDesc.ViewDimension = D3D11_DSV_DIMENSION_TEXTURE2DARRAY;
				DSVDesc.Texture2DArray.FirstArraySlice = 0;
				DSVDesc.Texture2DArray.ArraySize = TextureDesc.ArraySize;
				DSVDesc.Texture2DArray.MipSlice = 0;
			}
		}
		else
		{
			if (bIsMultisampled)
			{
				DSVDesc.ViewDimension = D3D11_DSV_DIMENSION_TEXTURE2DMS;
				// Nothing to set
			}
			else
			{
				DSVDesc.ViewDimension = D3D11_DSV_DIMENSION_TEXTURE2D;
				DSVDesc.Texture2D.MipSlice = 0;
			}
		}

		for (uint32 AccessType = 0; AccessType < FExclusiveDepthStencil::MaxIndex; ++AccessType)
		{
			// Create a read-only access views for the texture.
			// Read-only DSVs are not supported in Feature Level 10 so 
			// a dummy DSV is created in order reduce logic complexity at a higher-level.
			DSVDesc.Flags = (AccessType & FExclusiveDepthStencil::DepthRead_StencilWrite) ? D3D11_DSV_READ_ONLY_DEPTH : 0;
			if (UE::DXGIUtilities::HasStencilBits(DSVDesc.Format))
			{
				DSVDesc.Flags |= (AccessType & FExclusiveDepthStencil::DepthWrite_StencilRead) ? D3D11_DSV_READ_ONLY_STENCIL : 0;
			}
			VERIFYD3D11RESULT_EX(Direct3DDevice->CreateDepthStencilView(TextureResource,&DSVDesc,DepthStencilViews[AccessType].GetInitReference()), Direct3DDevice);
		}
	}

	// Create a shader resource view for the texture.
	if (bCreateShaderResource)
	{
		{
			D3D11_SHADER_RESOURCE_VIEW_DESC SRVDesc;
			FMemory::Memzero(SRVDesc);

			SRVDesc.Format = PlatformShaderResourceFormat;

			if (bCubeTexture && bTextureArray)
			{
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURECUBEARRAY;
				SRVDesc.TextureCubeArray.MostDetailedMip = 0;
				SRVDesc.TextureCubeArray.MipLevels = TextureDesc.MipLevels;
				SRVDesc.TextureCubeArray.First2DArrayFace = 0;
				SRVDesc.TextureCubeArray.NumCubes = TextureDesc.ArraySize / 6;
			}
			else if (bCubeTexture)
			{
				SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURECUBE;
				SRVDesc.TextureCube.MostDetailedMip = 0;
				SRVDesc.TextureCube.MipLevels = TextureDesc.MipLevels;
			}
			else if (bTextureArray)
			{
				if (bIsMultisampled)
				{
					SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DMSARRAY;
					SRVDesc.Texture2DMSArray.FirstArraySlice = 0;
					SRVDesc.Texture2DMSArray.ArraySize = TextureDesc.ArraySize;
				}
				else
				{
					SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DARRAY;
					SRVDesc.Texture2DArray.MostDetailedMip = 0;
					SRVDesc.Texture2DArray.MipLevels = TextureDesc.MipLevels;
					SRVDesc.Texture2DArray.FirstArraySlice = 0;
					SRVDesc.Texture2DArray.ArraySize = TextureDesc.ArraySize;
				}
			}
			else 
			{
				if (bIsMultisampled)
				{
					SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2DMS;
					// Nothing to set
				}
				else
				{
					SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2D;
					SRVDesc.Texture2D.MostDetailedMip = 0;
					SRVDesc.Texture2D.MipLevels = TextureDesc.MipLevels;
				}
			}
			VERIFYD3D11RESULT_EX(Direct3DDevice->CreateShaderResourceView(TextureResource,&SRVDesc,ShaderResourceView.GetInitReference()), Direct3DDevice);
		}

		check(IsValidRef(ShaderResourceView));
	}

	const ETextureDimension Dimension = bTextureArray
		? (bCubeTexture ? ETextureDimension::TextureCubeArray : ETextureDimension::Texture2DArray)
		: (bCubeTexture ? ETextureDimension::TextureCube      : ETextureDimension::Texture2D     );

	const uint32 RHIArraySize = bCubeTexture ? TextureDesc.ArraySize / 6 : TextureDesc.ArraySize;

	const FRHITextureCreateDesc RHITextureDesc =
		FRHITextureCreateDesc::Create(TEXT("FD3D11DynamicRHI::CreateTextureFromResource"), Dimension)
		.SetExtent(TextureDesc.Width, TextureDesc.Height)
		.SetFormat((EPixelFormat)Format)
		.SetClearValue(ClearValueBinding)
		.SetArraySize(RHIArraySize)
		.SetFlags(TexCreateFlags)
		.SetNumMips(TextureDesc.MipLevels)
		.SetNumSamples(TextureDesc.SampleDesc.Count)
		.DetermineInititialState();

	FD3D11Texture* Texture2D = new FD3D11Texture(
		RHITextureDesc,
		TextureResource,
		ShaderResourceView,
		TextureDesc.ArraySize,
		bCreatedRTVPerSlice,
		RenderTargetViews,
		DepthStencilViews
	);

	return Texture2D;
}

FTextureRHIRef FD3D11DynamicRHI::RHICreateTexture2DFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* TextureResource)
{
	return CreateTextureFromResource(false, false, Format, TexCreateFlags, ClearValueBinding, TextureResource);
}

FTextureRHIRef FD3D11DynamicRHI::RHICreateTexture2DArrayFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* TextureResource)
{
	return CreateTextureFromResource(true, false, Format, TexCreateFlags, ClearValueBinding, TextureResource);
}

FTextureRHIRef FD3D11DynamicRHI::RHICreateTextureCubeFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* TextureResource)
{
	return CreateTextureFromResource(false, true, Format, TexCreateFlags, ClearValueBinding, TextureResource);
}

FD3D11Texture::FD3D11Texture(const FRHITextureCreateDesc& InDesc)
	: FRHITexture(InDesc)
	, bCreatedRTVsPerSlice(false)
	, bAlias(false)
{
}

void FD3D11Texture::FinalizeCreation(
	ID3D11Resource* InResource,
	ID3D11ShaderResourceView* InShaderResourceView,
	int32 InRTVArraySize,
	bool bInCreatedRTVsPerSlice,
	TConstArrayView<TRefCountPtr<ID3D11RenderTargetView>> InRenderTargetViews,
	TConstArrayView<TRefCountPtr<ID3D11DepthStencilView>> InDepthStencilViews)
{
	Resource = InResource;
	ShaderResourceView = InShaderResourceView;
	RenderTargetViews = InRenderTargetViews;
	RTVArraySize = InRTVArraySize;
	bCreatedRTVsPerSlice = bInCreatedRTVsPerSlice;

	// Set the DSVs for all the access type combinations
	if (InDepthStencilViews.Num())
	{
		check(InDepthStencilViews.Num() == FExclusiveDepthStencil::MaxIndex);
		for (uint32 Index = 0; Index < FExclusiveDepthStencil::MaxIndex; Index++)
		{
			DepthStencilViews[Index] = InDepthStencilViews[Index];
		}
	}

	UpdateD3D11TextureStats(*this, true);
}

FD3D11Texture::FD3D11Texture(FD3D11Texture const& Other, const FString& Name, EAliasResourceParam)
	: FRHITexture(FRHITextureCreateDesc(Other.GetDesc(), ERHIAccess::SRVMask, *Name))
	, bAlias(true)
{
	AliasResource(Other);
}

FD3D11Texture::~FD3D11Texture()
{
	if (!bAlias)
	{
		UpdateD3D11TextureStats(*this, false);
	}
}

void FD3D11Texture::AliasResource(FD3D11Texture const& Other)
{
	check(bAlias);
	IHVResourceHandle    = Other.IHVResourceHandle;
	Resource             = Other.Resource;
	ShaderResourceView   = Other.ShaderResourceView;
	RenderTargetViews    = Other.RenderTargetViews;
	bCreatedRTVsPerSlice = Other.bCreatedRTVsPerSlice;
	RTVArraySize         = Other.RTVArraySize;

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(DepthStencilViews); ++Index)
	{
		DepthStencilViews[Index] = Other.DepthStencilViews[Index];
	}
}

void FD3D11DynamicRHI::RHIAliasTextureResources(FTextureRHIRef& DstTextureRHI, FTextureRHIRef& SrcTextureRHI)
{
	FD3D11Texture* DstTexture = ResourceCast(DstTextureRHI);
	FD3D11Texture* SrcTexture = ResourceCast(SrcTextureRHI);
	check(DstTexture && SrcTexture);

	DstTexture->AliasResource(*SrcTexture);
}

FTextureRHIRef FD3D11DynamicRHI::RHICreateAliasedTexture(FTextureRHIRef& SrcTextureRHI)
{
	FD3D11Texture* SrcTexture = ResourceCast(SrcTextureRHI);
	check(SrcTexture);
	const FString Name = SrcTextureRHI->GetName().ToString() + TEXT("Alias");

	return new FD3D11Texture(*SrcTexture, *Name, FD3D11Texture::CreateAlias);
}

void FD3D11DynamicRHI::RHICopyTexture(FRHITexture* SourceTextureRHI, FRHITexture* DestTextureRHI, const FRHICopyTextureInfo& CopyInfo)
{
	FRHICommandList_RecursiveHazardous RHICmdList(this);	

	FD3D11Texture* SourceTexture = ResourceCast(SourceTextureRHI);
	FD3D11Texture* DestTexture = ResourceCast(DestTextureRHI);

	check(SourceTexture && DestTexture);

	const FRHITextureDesc& SourceDesc = SourceTextureRHI->GetDesc();
	const FRHITextureDesc& DestDesc = DestTextureRHI->GetDesc();

	const uint16 SourceArraySize = SourceDesc.ArraySize * (SourceDesc.IsTextureCube() ? 6 : 1);
	const uint16 DestArraySize   = DestDesc.ArraySize   * (DestDesc.IsTextureCube()   ? 6 : 1);

	const bool bAllPixels =
		SourceDesc.GetSize() == DestDesc.GetSize() && (CopyInfo.Size == FIntVector::ZeroValue || CopyInfo.Size == SourceDesc.GetSize());

	const bool bAllSubresources =
		SourceDesc.NumMips == DestDesc.NumMips && SourceDesc.NumMips == CopyInfo.NumMips &&
		SourceArraySize == DestArraySize && SourceArraySize == CopyInfo.NumSlices;

	if (!bAllPixels || !bAllSubresources)
	{
		const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[SourceTextureRHI->GetFormat()];

		const FIntVector SourceSize = SourceDesc.GetSize();
		const FIntVector CopySize = CopyInfo.Size == FIntVector::ZeroValue ? SourceSize >> CopyInfo.SourceMipIndex : CopyInfo.Size;

		for (uint32 SliceIndex = 0; SliceIndex < CopyInfo.NumSlices; ++SliceIndex)
		{
			uint32 SourceSliceIndex = CopyInfo.SourceSliceIndex + SliceIndex;
			uint32 DestSliceIndex   = CopyInfo.DestSliceIndex   + SliceIndex;

			for (uint32 MipIndex = 0; MipIndex < CopyInfo.NumMips; ++MipIndex)
			{
				uint32 SourceMipIndex = CopyInfo.SourceMipIndex + MipIndex;
				uint32 DestMipIndex   = CopyInfo.DestMipIndex   + MipIndex;

				const uint32 SourceSubresource = D3D11CalcSubresource(SourceMipIndex, SourceSliceIndex, SourceTextureRHI->GetNumMips());
				const uint32 DestSubresource = D3D11CalcSubresource(DestMipIndex, DestSliceIndex, DestTextureRHI->GetNumMips());

				D3D11_BOX SrcBox;
				SrcBox.left   = CopyInfo.SourcePosition.X >> MipIndex;
				SrcBox.top    = CopyInfo.SourcePosition.Y >> MipIndex;
				SrcBox.front  = CopyInfo.SourcePosition.Z >> MipIndex;
				SrcBox.right  = AlignArbitrary<uint32>(FMath::Max<uint32>((CopyInfo.SourcePosition.X + CopySize.X) >> MipIndex, 1), PixelFormatInfo.BlockSizeX);
				SrcBox.bottom = AlignArbitrary<uint32>(FMath::Max<uint32>((CopyInfo.SourcePosition.Y + CopySize.Y) >> MipIndex, 1), PixelFormatInfo.BlockSizeY);
				SrcBox.back   = AlignArbitrary<uint32>(FMath::Max<uint32>((CopyInfo.SourcePosition.Z + CopySize.Z) >> MipIndex, 1), PixelFormatInfo.BlockSizeZ);

				const uint32 DestX = CopyInfo.DestPosition.X >> MipIndex;
				const uint32 DestY = CopyInfo.DestPosition.Y >> MipIndex;
				const uint32 DestZ = CopyInfo.DestPosition.Z >> MipIndex;

				Direct3DDeviceIMContext->CopySubresourceRegion(DestTexture->GetResource(), DestSubresource, DestX, DestY, DestZ, SourceTexture->GetResource(), SourceSubresource, &SrcBox);
			}
		}
	}
	else
	{
		// Make sure the params are all by default when using this case
		ensure(CopyInfo.SourceSliceIndex == 0 && CopyInfo.DestSliceIndex == 0 && CopyInfo.SourcePosition == FIntVector::ZeroValue && CopyInfo.DestPosition == FIntVector::ZeroValue && CopyInfo.SourceMipIndex == 0 && CopyInfo.DestMipIndex == 0);
		Direct3DDeviceIMContext->CopyResource(DestTexture->GetResource(), SourceTexture->GetResource());
	}
}

void FD3D11DynamicRHI::RHICopyBufferRegion(FRHIBuffer* DstBuffer, uint64 DstOffset, FRHIBuffer* SrcBuffer, uint64 SrcOffset, uint64 NumBytes)
{
	if (!DstBuffer || !SrcBuffer || DstBuffer == SrcBuffer || !NumBytes)
	{
		return;
	}

	FD3D11Buffer* DstBufferD3D11 = ResourceCast(DstBuffer);
	FD3D11Buffer* SrcBufferD3D11 = ResourceCast(SrcBuffer);

	check(DstBufferD3D11 && SrcBufferD3D11);
	check(DstOffset + NumBytes <= DstBuffer->GetSize() && SrcOffset + NumBytes <= SrcBuffer->GetSize());

	D3D11_BOX SrcBox;
	SrcBox.left = SrcOffset;
	SrcBox.right = SrcOffset + NumBytes;
	SrcBox.top = 0;
	SrcBox.bottom = 1;
	SrcBox.front = 0;
	SrcBox.back = 1;

	ID3D11Resource* DstResource = DstBufferD3D11->Resource.GetReference();
	ID3D11Resource* SrcResource = SrcBufferD3D11->Resource.GetReference();
	Direct3DDeviceIMContext->CopySubresourceRegion(DstResource, 0, DstOffset, 0, 0, SrcResource, 0, &SrcBox);
}
