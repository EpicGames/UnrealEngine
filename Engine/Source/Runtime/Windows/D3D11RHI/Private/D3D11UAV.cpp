// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D11RHIPrivate.h"
#include "ClearReplacementShaders.h"

// -----------------------------------------------------------------------------------------------------
//
//                                       FD3D11UnorderedAccessView                                      
//
// -----------------------------------------------------------------------------------------------------

FD3D11UnorderedAccessView::FD3D11UnorderedAccessView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
	: FRHIUnorderedAccessView(Resource, ViewDesc)
{
	RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		UpdateView();
	});
}

void FD3D11UnorderedAccessView::UpdateView()
{
	ID3D11Resource* D3DResource = nullptr;
	D3D11_UNORDERED_ACCESS_VIEW_DESC UAVDesc = {};

	if (IsBuffer())
	{
		FD3D11Buffer* Buffer = FD3D11DynamicRHI::ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.UAV.GetViewInfo(Buffer);

		if (!Info.bNullView)
		{
			D3DResource = Buffer->Resource;

			UAVDesc.Format = UE::DXGIUtilities::FindUnorderedAccessFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat));
			UAVDesc.ViewDimension = D3D11_UAV_DIMENSION_BUFFER;

			UAVDesc.Buffer.FirstElement = Info.OffsetInBytes / Info.StrideInBytes;
			UAVDesc.Buffer.NumElements = Info.NumElements;

			UAVDesc.Buffer.Flags |= Info.bAppendBuffer ? D3D11_BUFFER_UAV_FLAG_APPEND : 0;
			UAVDesc.Buffer.Flags |= Info.bAtomicCounter ? D3D11_BUFFER_UAV_FLAG_COUNTER : 0;

			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Typed:
			case FRHIViewDesc::EBufferType::Structured:
				break;

			case FRHIViewDesc::EBufferType::Raw:
				UAVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
				UAVDesc.Buffer.Flags |= D3D11_BUFFER_UAV_FLAG_RAW;
				break;

			default:
				checkNoEntry(); // unsupported
				break;
			}
		}
	}
	else
	{
		FD3D11Texture* Texture = FD3D11DynamicRHI::ResourceCast(GetTexture());
		D3DResource = Texture->GetResource();

		FRHITextureDesc const& TextureDesc = Texture->GetDesc();
		auto const Info = ViewDesc.Texture.UAV.GetViewInfo(Texture);

		UAVDesc.Format = UE::DXGIUtilities::FindUnorderedAccessFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat));

		switch (Info.Dimension)
		{
		case FRHIViewDesc::EDimension::Texture2D:
			UAVDesc.ViewDimension = D3D11_UAV_DIMENSION_TEXTURE2D;
			UAVDesc.Texture2D.MipSlice = Info.MipLevel;
			ensureAlwaysMsgf(Info.ArrayRange.First == 0, TEXT("Trying to create an UAV beyond the first slice. This is not supported on d3d11, and binding a single slice 2D array as an HLSL RWTexture2D is not supported either"));
			break;

		case FRHIViewDesc::EDimension::TextureCube:
		case FRHIViewDesc::EDimension::TextureCubeArray:
			UAVDesc.ViewDimension = D3D11_UAV_DIMENSION_TEXTURE2DARRAY;
			UAVDesc.Texture2DArray.FirstArraySlice = Info.ArrayRange.First * 6;
			UAVDesc.Texture2DArray.ArraySize       = Info.ArrayRange.Num * 6;
			UAVDesc.Texture2DArray.MipSlice        = Info.MipLevel;
			break;

		case FRHIViewDesc::EDimension::Texture2DArray:
			UAVDesc.ViewDimension = D3D11_UAV_DIMENSION_TEXTURE2DARRAY;
			UAVDesc.Texture2DArray.FirstArraySlice = Info.ArrayRange.First;
			UAVDesc.Texture2DArray.ArraySize       = Info.ArrayRange.Num;
			UAVDesc.Texture2DArray.MipSlice        = Info.MipLevel;
			break;

		case FRHIViewDesc::EDimension::Texture3D:
			UAVDesc.ViewDimension = D3D11_UAV_DIMENSION_TEXTURE3D;
			UAVDesc.Texture3D.FirstWSlice = 0;
			UAVDesc.Texture3D.WSize       = FMath::Max(TextureDesc.Depth >> Info.MipLevel, 1);
			UAVDesc.Texture3D.MipSlice    = Info.MipLevel;
			break;

		default:
			checkNoEntry();
			break;
		}
	}

	View = nullptr;
	if (D3DResource)
	{
		ID3D11Device* Device = FD3D11DynamicRHI::Get().GetDevice();
		VERIFYD3D11RESULT_EX(Device->CreateUnorderedAccessView(D3DResource, &UAVDesc, View.GetInitReference()), Device);
	}
}

FD3D11ViewableResource* FD3D11UnorderedAccessView::GetBaseResource() const
{
	return IsBuffer()
		? static_cast<FD3D11ViewableResource*>(FD3D11DynamicRHI::ResourceCast(GetBuffer()))
		: static_cast<FD3D11ViewableResource*>(FD3D11DynamicRHI::ResourceCast(GetTexture()));
}

// -----------------------------------------------------------------------------------------------------
//
//                                            RHI Functions                                             
//
// -----------------------------------------------------------------------------------------------------

FUnorderedAccessViewRHIRef FD3D11DynamicRHI::RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	return new FD3D11UnorderedAccessView(RHICmdList, Resource, ViewDesc);
}

void FD3D11DynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	SetD3D11ResourceName(ResourceCast(UnorderedAccessViewRHI), Name);
#endif
}

// -----------------------------------------------------------------------------------------------------
//
//                                         UAV Clear Functions                                          
//
// -----------------------------------------------------------------------------------------------------

void FD3D11DynamicRHI::ClearUAV(TRHICommandList_RecursiveHazardous<FD3D11DynamicRHI>& RHICmdList, FD3D11UnorderedAccessView* UnorderedAccessView, const void* ClearValues, bool bFloat)
{
	D3D11_UNORDERED_ACCESS_VIEW_DESC UAVDesc;
	UnorderedAccessView->View->GetDesc(&UAVDesc);

	// Only structured buffers can have an unknown format
	check(UAVDesc.ViewDimension == D3D11_UAV_DIMENSION_BUFFER || UAVDesc.Format != DXGI_FORMAT_UNKNOWN);

	EClearReplacementValueType ValueType = bFloat ? EClearReplacementValueType::Float : EClearReplacementValueType::Uint32;
	switch (UAVDesc.Format)
	{
	case DXGI_FORMAT_R32G32B32A32_SINT:
	case DXGI_FORMAT_R32G32B32_SINT:
	case DXGI_FORMAT_R16G16B16A16_SINT:
	case DXGI_FORMAT_R32G32_SINT:
	case DXGI_FORMAT_R8G8B8A8_SINT:
	case DXGI_FORMAT_R16G16_SINT:
	case DXGI_FORMAT_R32_SINT:
	case DXGI_FORMAT_R8G8_SINT:
	case DXGI_FORMAT_R16_SINT:
	case DXGI_FORMAT_R8_SINT:
		ValueType = EClearReplacementValueType::Int32;
		break;

	case DXGI_FORMAT_R32G32B32A32_UINT:
	case DXGI_FORMAT_R32G32B32_UINT:
	case DXGI_FORMAT_R16G16B16A16_UINT:
	case DXGI_FORMAT_R32G32_UINT:
	case DXGI_FORMAT_D32_FLOAT_S8X24_UINT:
	case DXGI_FORMAT_X32_TYPELESS_G8X24_UINT:
	case DXGI_FORMAT_R10G10B10A2_UINT:
	case DXGI_FORMAT_R8G8B8A8_UINT:
	case DXGI_FORMAT_R16G16_UINT:
	case DXGI_FORMAT_R32_UINT:
	case DXGI_FORMAT_D24_UNORM_S8_UINT:
	case DXGI_FORMAT_X24_TYPELESS_G8_UINT:
	case DXGI_FORMAT_R8G8_UINT:
	case DXGI_FORMAT_R16_UINT:
	case DXGI_FORMAT_R8_UINT:
		ValueType = EClearReplacementValueType::Uint32;
		break;
	}

	ensureMsgf((UAVDesc.Format == DXGI_FORMAT_UNKNOWN) || (bFloat == (ValueType == EClearReplacementValueType::Float)), TEXT("Attempt to clear a UAV using the wrong RHIClearUAV function. Float vs Integer mismatch."));

	if (UAVDesc.ViewDimension == D3D11_UAV_DIMENSION_BUFFER)
	{
		const bool bByteAddressBuffer = (UAVDesc.Buffer.Flags & D3D11_BUFFER_UAV_FLAG_RAW) != 0;

		if (UAVDesc.Format == DXGI_FORMAT_UNKNOWN || bByteAddressBuffer)
		{
			// Structured buffer. Use the clear function on the immediate context, since we can't use a general purpose shader for these.
			RHICmdList.RunOnContext([UnorderedAccessView, ClearValues](auto& Context)
			{
				Context.Direct3DDeviceIMContext->ClearUnorderedAccessViewUint(UnorderedAccessView->View, *reinterpret_cast<const UINT(*)[4]>(ClearValues));
			});
		}
		else
		{
			if (UAVDesc.Buffer.NumElements < 65536 * 64)
			{
				ClearUAVShader_T<EClearReplacementResourceType::Buffer, 4, false>(RHICmdList, UnorderedAccessView, UAVDesc.Buffer.NumElements, 1, 1, ClearValues, ValueType);
			}
			else
			{
				ClearUAVShader_T<EClearReplacementResourceType::LargeBuffer, 4, false>(RHICmdList, UnorderedAccessView, UAVDesc.Buffer.NumElements, 1, 1, ClearValues, ValueType);
			}
			
		}
	}
	else
	{
		if (UAVDesc.ViewDimension == D3D11_UAV_DIMENSION_TEXTURE2D)
		{
			FD3D11Texture* Texture = ResourceCast(UnorderedAccessView->GetTexture());
			FIntVector Size = Texture->GetSizeXYZ();

			uint32 Width  = Size.X >> UAVDesc.Texture2D.MipSlice;
			uint32 Height = Size.Y >> UAVDesc.Texture2D.MipSlice;
			ClearUAVShader_T<EClearReplacementResourceType::Texture2D, 4, false>(RHICmdList, UnorderedAccessView, Width, Height, 1, ClearValues, ValueType);
		}
		else if (UAVDesc.ViewDimension == D3D11_UAV_DIMENSION_TEXTURE2DARRAY)
		{
			FD3D11Texture* Texture = ResourceCast(UnorderedAccessView->GetTexture());
			FIntVector Size = Texture->GetSizeXYZ();

			uint32 Width = Size.X >> UAVDesc.Texture2DArray.MipSlice;
			uint32 Height = Size.Y >> UAVDesc.Texture2DArray.MipSlice;
			ClearUAVShader_T<EClearReplacementResourceType::Texture2DArray, 4, false>(RHICmdList, UnorderedAccessView, Width, Height, UAVDesc.Texture2DArray.ArraySize, ClearValues, ValueType);
		}
		else if (UAVDesc.ViewDimension == D3D11_UAV_DIMENSION_TEXTURE3D)
		{
			FD3D11Texture* Texture = ResourceCast(UnorderedAccessView->GetTexture());
			FIntVector Size = Texture->GetSizeXYZ();

			// @todo - is WSize / mip index handling here correct?
			uint32 Width = Size.X >> UAVDesc.Texture2DArray.MipSlice;
			uint32 Height = Size.Y >> UAVDesc.Texture2DArray.MipSlice;
			ClearUAVShader_T<EClearReplacementResourceType::Texture3D, 4, false>(RHICmdList, UnorderedAccessView, Width, Height, UAVDesc.Texture3D.WSize, ClearValues, ValueType);
		}
		else
		{
			ensure(0);
		}
	}
}

void FD3D11DynamicRHI::RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values)
{
	TRHICommandList_RecursiveHazardous<FD3D11DynamicRHI> RHICmdList(this);
	ClearUAV(RHICmdList, ResourceCast(UnorderedAccessViewRHI), &Values, true);
}

void FD3D11DynamicRHI::RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values)
{
	TRHICommandList_RecursiveHazardous<FD3D11DynamicRHI> RHICmdList(this);
	ClearUAV(RHICmdList, ResourceCast(UnorderedAccessViewRHI), &Values, false);
}
