// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11UniformBuffer.cpp: D3D uniform buffer RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "UniformBuffer.h"
#include "ShaderParameterStruct.h"

/** Describes a uniform buffer in the free pool. */
struct FPooledUniformBuffer
{
	TRefCountPtr<ID3D11Buffer> Buffer;
	uint32 CreatedSize;
	uint32 FrameFreed;
};

/** 
 * Number of size buckets to use for the uniform buffer free pool. 
 * This needs to be enough to cover the valid uniform buffer size range combined with the heuristic used to map sizes to buckets.
 */
const int32 NumPoolBuckets = 17;

/** 
 * Number of frames that a uniform buffer will not be re-used for, after being freed. 
 * This is done as a workaround for what appears to be an AMD driver bug with 11.10 drivers and a 6970 HD,
 * Where reusing a constant buffer with D3D11_MAP_WRITE_DISCARD still in use by the GPU will result in incorrect contents randomly.
 */
const int32 NumSafeFrames = 3;

/** Returns the size in bytes of the bucket that the given size fits into. */
uint32 GetPoolBucketSize(uint32 NumBytes)
{
	return FMath::RoundUpToPowerOfTwo(NumBytes);
}

/** Returns the index of the bucket that the given size fits into. */
uint32 GetPoolBucketIndex(uint32 NumBytes)
{
	return FMath::CeilLogTwo(NumBytes);
}

/** Pool of free uniform buffers, indexed by bucket for constant size search time. */
TArray<FPooledUniformBuffer> UniformBufferPool[NumPoolBuckets];

/** Uniform buffers that have been freed more recently than NumSafeFrames ago. */
TArray<FPooledUniformBuffer> SafeUniformBufferPools[NumSafeFrames][NumPoolBuckets];

static int32 GD3DUniformPoolRecycleDepth = 30;
static FAutoConsoleVariableRef CVarD3DUniformPoolRecycleDepth(
	TEXT("r.d3d.uniformbufferrecycledepth"),
	GD3DUniformPoolRecycleDepth,
	TEXT("Number of frames before recycling freed uniform buffers .\n"),	
	ECVF_Default
);

/** Does per-frame global updating for the uniform buffer pool. */
void UniformBufferBeginFrame()
{
	int32 NumCleaned = 0;

	SCOPE_CYCLE_COUNTER(STAT_D3D11CleanUniformBufferTime);

	// Clean a limited number of old entries to reduce hitching when leaving a large level
	for (int32 BucketIndex = 0; BucketIndex < NumPoolBuckets; BucketIndex++)
	{
		for (int32 EntryIndex = UniformBufferPool[BucketIndex].Num() - 1; EntryIndex >= 0 && NumCleaned < 10; EntryIndex--)
		{
			FPooledUniformBuffer& PoolEntry = UniformBufferPool[BucketIndex][EntryIndex];

			check(IsValidRef(PoolEntry.Buffer));

			// Clean entries that are unlikely to be reused
			if (GFrameNumberRenderThread - PoolEntry.FrameFreed > (uint32)GD3DUniformPoolRecycleDepth)
			{
				DEC_DWORD_STAT(STAT_D3D11NumFreeUniformBuffers);
				DEC_MEMORY_STAT_BY(STAT_D3D11FreeUniformBufferMemory, PoolEntry.CreatedSize);
				NumCleaned++;
				D3D11BufferStats::UpdateUniformBufferStats(PoolEntry.Buffer, PoolEntry.CreatedSize, false);
				PoolEntry.Buffer.SafeRelease();
				UniformBufferPool[BucketIndex].RemoveAtSwap(EntryIndex);
			}
		}
	}

	// Index of the bucket that is now old enough to be reused
	const int32 SafeFrameIndex = GFrameNumberRenderThread % NumSafeFrames;

	// Merge the bucket into the free pool array
	for (int32 BucketIndex = 0; BucketIndex < NumPoolBuckets; BucketIndex++)
	{
		int32 LastNum = UniformBufferPool[BucketIndex].Num();
		UniformBufferPool[BucketIndex].Append(SafeUniformBufferPools[SafeFrameIndex][BucketIndex]);
#if DO_CHECK
		while (LastNum < UniformBufferPool[BucketIndex].Num())
		{
			check(IsValidRef(UniformBufferPool[BucketIndex][LastNum].Buffer));
			LastNum++;
		}
#endif
		SafeUniformBufferPools[SafeFrameIndex][BucketIndex].Reset();
	}
}

static bool IsPoolingEnabled()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.UniformBufferPooling"));
	int32 CVarValue = CVar->GetValueOnAnyThread();
	return CVarValue != 0;
};

static TRefCountPtr<ID3D11Buffer> CreateAndUpdatePooledUniformBuffer(
	FD3D11Device* Device,
	FD3D11DeviceContext* Context,
	const void* Contents,
	uint32 NumBytes)
{
	TRefCountPtr<ID3D11Buffer> UniformBufferResource;

	// Find the appropriate bucket based on size
	const uint32 BucketIndex = GetPoolBucketIndex(NumBytes);
	TArray<FPooledUniformBuffer>& PoolBucket = UniformBufferPool[BucketIndex];

	if (PoolBucket.Num() > 0)
	{
		// Reuse the last entry in this size bucket
		FPooledUniformBuffer FreeBufferEntry = PoolBucket.Pop();
		check(IsValidRef(FreeBufferEntry.Buffer));
		checkf(FreeBufferEntry.CreatedSize >= NumBytes, TEXT("%u %u %u %u"), NumBytes, BucketIndex, FreeBufferEntry.CreatedSize, GetPoolBucketSize(NumBytes));
		DEC_DWORD_STAT(STAT_D3D11NumFreeUniformBuffers);
		DEC_MEMORY_STAT_BY(STAT_D3D11FreeUniformBufferMemory, FreeBufferEntry.CreatedSize);
		UniformBufferResource = FreeBufferEntry.Buffer;
	}

	if (!IsValidRef(UniformBufferResource))
	{
		D3D11_BUFFER_DESC Desc;
		// Allocate based on the bucket size, since this uniform buffer will be reused later
		Desc.ByteWidth = GetPoolBucketSize(NumBytes);
		// Use D3D11_USAGE_DYNAMIC, which allows multiple CPU writes for pool reuses
		// This is method of updating is vastly superior to creating a new constant buffer each time with D3D11_USAGE_IMMUTABLE,
		// Since that inserts the data into the command buffer which causes GPU flushes
		Desc.Usage = D3D11_USAGE_DYNAMIC;
		Desc.BindFlags = D3D11_BIND_CONSTANT_BUFFER;
		Desc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE;
		Desc.MiscFlags = 0;
		Desc.StructureByteStride = 0;

		VERIFYD3D11RESULT_EX(Device->CreateBuffer(&Desc, NULL, UniformBufferResource.GetInitReference()), Device);

		D3D11BufferStats::UpdateUniformBufferStats(UniformBufferResource, Desc.ByteWidth, true);
	}

	check(IsValidRef(UniformBufferResource));

	if (Contents)
	{
		D3D11_MAPPED_SUBRESOURCE MappedSubresource;
		// Discard previous results since we always do a full update
		VERIFYD3D11RESULT_EX(Context->Map(UniformBufferResource, 0, D3D11_MAP_WRITE_DISCARD, 0, &MappedSubresource), Device);
		check(MappedSubresource.RowPitch >= NumBytes);
		FMemory::Memcpy(MappedSubresource.pData, Contents, NumBytes);
		Context->Unmap(UniformBufferResource, 0);
	}

	return UniformBufferResource;
}

FUniformBufferRHIRef FD3D11DynamicRHI::RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation)
{
	if (Contents && Validation == EUniformBufferValidation::ValidateResources)
	{
		ValidateShaderParameterResourcesRHI(Contents, *Layout);
	}

	FD3D11UniformBuffer* NewUniformBuffer = nullptr;
	const uint32 NumBytes = Layout->ConstantBufferSize;
	if (NumBytes > 0)
	{
		// Constant buffers must also be 16-byte aligned.
		check(Align(NumBytes,16) == NumBytes);
		check(Align(Contents,16) == Contents);
		check(NumBytes <= D3D11_REQ_CONSTANT_BUFFER_ELEMENT_COUNT * 16);
		check(NumBytes < (1 << NumPoolBuckets));

		SCOPE_CYCLE_COUNTER(STAT_D3D11UpdateUniformBufferTime);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (IsPoolingEnabled() && (IsInActualRenderingThread() || IsInRHIThread()) && !UE::Tasks::Private::IsThreadRetractingTask())
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			const bool bAllocatedFromPool = true;

			if (ShouldNotEnqueueRHICommand())
			{
				TRefCountPtr<ID3D11Buffer> UniformBufferResource = CreateAndUpdatePooledUniformBuffer(
					Direct3DDevice.GetReference(),
					Direct3DDeviceIMContext.GetReference(),
					Contents,
					NumBytes);

				NewUniformBuffer = new FD3D11UniformBuffer(this, Layout, UniformBufferResource, FRingAllocation(), bAllocatedFromPool);
			}
			else
			{
				NewUniformBuffer = new FD3D11UniformBuffer(this, Layout, nullptr, FRingAllocation(), bAllocatedFromPool);

				void* CPUContent = nullptr;
				
				if (Contents)
				{
					CPUContent = FMemory::Malloc(NumBytes);
					FMemory::Memcpy(CPUContent, Contents, NumBytes);
				}

				RunOnRHIThread(
					[NewUniformBuffer, CPUContent, NumBytes]()
				{
					NewUniformBuffer->Resource = CreateAndUpdatePooledUniformBuffer(
						D3D11RHI_DEVICE,
						D3D11RHI_IMMEDIATE_CONTEXT,
						CPUContent,
						NumBytes);
					FMemory::Free(CPUContent);
				});
			}
		}
		else
		{
			// No pooling
			D3D11_BUFFER_DESC Desc;
			Desc.ByteWidth = NumBytes;
			Desc.Usage = D3D11_USAGE_DYNAMIC;
			Desc.BindFlags = D3D11_BIND_CONSTANT_BUFFER;
			Desc.CPUAccessFlags = D3D11_CPU_ACCESS_WRITE;
			Desc.MiscFlags = 0;
			Desc.StructureByteStride = 0;

			D3D11_SUBRESOURCE_DATA ImmutableData;
			ImmutableData.pSysMem = Contents;
			ImmutableData.SysMemPitch = ImmutableData.SysMemSlicePitch = 0;

			TRefCountPtr<ID3D11Buffer> UniformBufferResource;
			VERIFYD3D11RESULT_EX(Direct3DDevice->CreateBuffer(&Desc, Contents ? &ImmutableData : nullptr,UniformBufferResource.GetInitReference()), Direct3DDevice);

			const bool bAllocatedFromPool = false;
			NewUniformBuffer = new FD3D11UniformBuffer(this, Layout, UniformBufferResource, FRingAllocation(), bAllocatedFromPool);

			INC_DWORD_STAT(STAT_D3D11NumImmutableUniformBuffers);
		}
	}
	else
	{
		// This uniform buffer contains no constants, only a resource table.
		const bool bAllocatedFromPool = false;
		NewUniformBuffer = new FD3D11UniformBuffer(this, Layout, nullptr, FRingAllocation(), bAllocatedFromPool);
	}

	if (Layout->Resources.Num())
	{
		const int32 ResourceCount = Layout->Resources.Num();
		NewUniformBuffer->GetResourceTable().Empty(ResourceCount);
		NewUniformBuffer->GetResourceTable().AddZeroed(ResourceCount);

		if (Contents)
		{
			for (int32 Index = 0; Index < ResourceCount; ++Index)
			{
				const auto ResourceParameter = Layout->Resources[Index];
				NewUniformBuffer->GetResourceTable()[Index] = GetShaderParameterResourceRHI(Contents, ResourceParameter.MemberOffset, ResourceParameter.MemberType);
			}
		}
	}

	return NewUniformBuffer;
}

void UpdateUniformBufferContents(FD3D11Device* Direct3DDevice, FD3D11DeviceContext* Context, FD3D11UniformBuffer* UniformBuffer, const void* Contents, uint32 ConstantBufferSize)
{
	// Update the contents of the uniform buffer.
	if (ConstantBufferSize > 0)
	{
		// Constant buffers must also be 16-byte aligned.
		check(Align(Contents, 16) == Contents);

		D3D11_MAPPED_SUBRESOURCE MappedSubresource;
		// Discard previous results since we always do a full update
		VERIFYD3D11RESULT_EX(Context->Map(UniformBuffer->Resource.GetReference(), 0, D3D11_MAP_WRITE_DISCARD, 0, &MappedSubresource), Direct3DDevice);
		check(MappedSubresource.RowPitch >= ConstantBufferSize);
		FMemory::Memcpy(MappedSubresource.pData, Contents, ConstantBufferSize);
		Context->Unmap(UniformBuffer->Resource.GetReference(), 0);
	}
}

void FD3D11DynamicRHI::RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents)
{
	check(UniformBufferRHI);

	FD3D11UniformBuffer* UniformBuffer = ResourceCast(UniformBufferRHI);
	const FRHIUniformBufferLayout& Layout = UniformBufferRHI->GetLayout();
	ValidateShaderParameterResourcesRHI(Contents, Layout);

	const uint32 ConstantBufferSize = Layout.ConstantBufferSize;
	const int32 NumResources = Layout.Resources.Num();

	check(UniformBuffer->GetResourceTable().Num() == NumResources);

	if (RHICmdList.Bypass())
	{
		UpdateUniformBufferContents(Direct3DDevice, Direct3DDeviceIMContext, UniformBuffer, Contents, ConstantBufferSize);

		for (int32 Index = 0; Index < NumResources; ++Index)
		{
			const auto Parameter = Layout.Resources[Index];
			UniformBuffer->GetResourceTable()[Index] = GetShaderParameterResourceRHI(Contents, Parameter.MemberOffset, Parameter.MemberType);
		}
	}
	else
	{
		FRHIResource** CmdListResources = nullptr;
		void* CmdListConstantBufferData = nullptr;

		if (NumResources > 0)
		{
			CmdListResources = (FRHIResource**)RHICmdList.Alloc(sizeof(FRHIResource*) * NumResources, alignof(FRHIResource*));

			for (int32 Index = 0; Index < NumResources; ++Index)
			{
				const auto Parameter = Layout.Resources[Index];
				CmdListResources[Index] = GetShaderParameterResourceRHI(Contents, Parameter.MemberOffset, Parameter.MemberType);
			}
		}

		if (ConstantBufferSize > 0)
		{
			CmdListConstantBufferData = (void*)RHICmdList.Alloc(ConstantBufferSize, 16);
			FMemory::Memcpy(CmdListConstantBufferData, Contents, ConstantBufferSize);
		}

		RHICmdList.EnqueueLambda([Direct3DDeviceIMContext = Direct3DDeviceIMContext.GetReference(),
			Direct3DDevice = Direct3DDevice.GetReference(),
			UniformBuffer,
			CmdListResources,
			NumResources,
			CmdListConstantBufferData,
			ConstantBufferSize](FRHICommandListBase&)
		{
			UpdateUniformBufferContents(Direct3DDevice, Direct3DDeviceIMContext, UniformBuffer, CmdListConstantBufferData, ConstantBufferSize);

			// Update resource table.
			for (int32 ResourceIndex = 0; ResourceIndex < NumResources; ++ResourceIndex)
			{
				UniformBuffer->GetResourceTable()[ResourceIndex] = CmdListResources[ResourceIndex];
			}
		});
		RHICmdList.RHIThreadFence(true);
	}
}

FD3D11UniformBuffer::~FD3D11UniformBuffer()
{
	// Do not return the allocation to the pool if it is in the dynamic constant buffer!
	if (bAllocatedFromPool && !RingAllocation.IsValid() && Resource != nullptr)
	{
		check(IsInRHIThread() || IsInRenderingThread());
		D3D11_BUFFER_DESC Desc;
		Resource->GetDesc(&Desc);

		// Return this uniform buffer to the free pool
		if (Desc.CPUAccessFlags == D3D11_CPU_ACCESS_WRITE && Desc.Usage == D3D11_USAGE_DYNAMIC)
		{
			check(IsValidRef(Resource));
			FPooledUniformBuffer NewEntry;
			NewEntry.Buffer = Resource;
			NewEntry.FrameFreed = GFrameNumberRenderThread;
			NewEntry.CreatedSize = Desc.ByteWidth;

			// Add to this frame's array of free uniform buffers
			const int32 SafeFrameIndex = (GFrameNumberRenderThread - 1) % NumSafeFrames;
			const uint32 BucketIndex = GetPoolBucketIndex(Desc.ByteWidth);
			int32 LastNum = SafeUniformBufferPools[SafeFrameIndex][BucketIndex].Num();
			check(Desc.ByteWidth <= GetPoolBucketSize(Desc.ByteWidth));
			SafeUniformBufferPools[SafeFrameIndex][BucketIndex].Add(NewEntry);
			INC_DWORD_STAT(STAT_D3D11NumFreeUniformBuffers);
			INC_MEMORY_STAT_BY(STAT_D3D11FreeUniformBufferMemory, Desc.ByteWidth);

			FPlatformMisc::MemoryBarrier(); // check for unwanted concurrency
			check(SafeUniformBufferPools[SafeFrameIndex][BucketIndex].Num() == LastNum + 1);
		}
	}
}


void FD3D11DynamicRHI::ReleasePooledUniformBuffers()
{
	// Free D3D resources in the pool
	// Don't bother updating pool stats, this is only done on exit
	for (int32 BucketIndex = 0; BucketIndex < NumPoolBuckets; BucketIndex++)
	{
		UniformBufferPool[BucketIndex].Empty();
	}
	
	for (int32 SafeFrameIndex = 0; SafeFrameIndex < NumSafeFrames; SafeFrameIndex++)
	{
		for (int32 BucketIndex = 0; BucketIndex < NumPoolBuckets; BucketIndex++)
		{
			SafeUniformBufferPools[SafeFrameIndex][BucketIndex].Empty();
		}
	}
}