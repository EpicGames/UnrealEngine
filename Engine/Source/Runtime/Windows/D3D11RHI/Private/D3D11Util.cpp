// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Util.h: D3D RHI utility implementation.
=============================================================================*/

#include "D3D11Util.h"
#include "D3D11RHIPrivate.h"
#include "EngineModule.h"
#include "Engine/GameEngine.h"
#include "RendererInterface.h"
#include "ProfilingDebugging/ScopedDebugInfo.h"
#include "HAL/ExceptionHandling.h"

#define D3DERR(x) case x: ErrorCodeText = TEXT(#x); break;
#define LOCTEXT_NAMESPACE "Developer.MessageLog"

void SetD3D11ObjectName(ID3D11DeviceChild* Object, FAnsiStringView Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	if (Object)
	{
		Object->SetPrivateData(WKPDID_D3DDebugObjectName, Name.Len() + 1, GetData(Name));
	}
#endif
}

void SetD3D11ObjectName(ID3D11DeviceChild* Object, FStringView Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	if (Object != nullptr)
	{
		FAnsiString AnsiString(Name);
		SetD3D11ObjectName(Object, AnsiString);
	}
#endif
}

void SetD3D11ResourceName(FD3D11Texture* Texture, FStringView Name)
{
	SetD3D11ObjectName(Texture->GetResource(), Name);
}

void SetD3D11ResourceName(FD3D11Buffer* Buffer, FStringView Name)
{
	SetD3D11ObjectName(Buffer->GetResource(), Name);
}

void SetD3D11ResourceName(FD3D11UnorderedAccessView* UAV, FStringView Name)
{
	SetD3D11ObjectName(UAV->View, Name);
}

static FString GetD3D11DeviceHungErrorString(HRESULT ErrorCode)
{
	FString ErrorCodeText;

	switch(ErrorCode)
	{
		D3DERR(DXGI_ERROR_DEVICE_HUNG)
		D3DERR(DXGI_ERROR_DEVICE_REMOVED)
		D3DERR(DXGI_ERROR_DEVICE_RESET)
		D3DERR(DXGI_ERROR_DRIVER_INTERNAL_ERROR)
		D3DERR(DXGI_ERROR_INVALID_CALL)
		default: ErrorCodeText = FString::Printf(TEXT("%08X"),(int32)ErrorCode);
	}

	return ErrorCodeText;
}

FString GetD3D11ErrorString(HRESULT ErrorCode, ID3D11Device* Device)
{
	FString ErrorCodeText;

	switch(ErrorCode)
	{
		D3DERR(S_OK);
		D3DERR(D3D11_ERROR_FILE_NOT_FOUND)
		D3DERR(D3D11_ERROR_TOO_MANY_UNIQUE_STATE_OBJECTS)
		D3DERR(E_FAIL)
		D3DERR(E_INVALIDARG)
		D3DERR(E_OUTOFMEMORY)
		D3DERR(DXGI_ERROR_INVALID_CALL)
		D3DERR(DXGI_ERROR_WAS_STILL_DRAWING)
		D3DERR(E_NOINTERFACE)
		D3DERR(DXGI_ERROR_DEVICE_REMOVED)
		D3DERR(DXGI_ERROR_NOT_CURRENTLY_AVAILABLE)
		default: ErrorCodeText = FString::Printf(TEXT("%08X"),(int32)ErrorCode);
	}

	if(ErrorCode == DXGI_ERROR_DEVICE_REMOVED && Device)
	{
		HRESULT hResDeviceRemoved = Device->GetDeviceRemovedReason();
		ErrorCodeText += FString(" ") + GetD3D11DeviceHungErrorString(hResDeviceRemoved);
	}

	return ErrorCodeText;
}

#undef D3DERR

static FString GetD3D11TextureFlagString(uint32 TextureFlags)
{
	FString TextureFormatText;

	if (TextureFlags & D3D11_BIND_RENDER_TARGET)
	{
		TextureFormatText += TEXT("D3D11_BIND_RENDER_TARGET ");
	}

	if (TextureFlags & D3D11_BIND_DEPTH_STENCIL)
	{
		TextureFormatText += TEXT("D3D11_BIND_DEPTH_STENCIL ");
	}

	if (TextureFlags & D3D11_BIND_SHADER_RESOURCE)
	{
		TextureFormatText += TEXT("D3D11_BIND_SHADER_RESOURCE ");
	}

	if (TextureFlags & D3D11_BIND_UNORDERED_ACCESS)
	{
		TextureFormatText += TEXT("D3D11_BIND_UNORDERED_ACCESS ");
	}

	return TextureFormatText;
}

extern CORE_API bool GIsGPUCrashed;
static void TerminateOnDeviceRemoved(HRESULT D3DResult, ID3D11Device* Direct3DDevice)
{
	if (D3DResult == DXGI_ERROR_DEVICE_REMOVED)
	{
		FGenericCrashContext::SetEngineData(TEXT("GPUCrash.D3DResult"), FString::FromInt(D3DResult));
		if (Direct3DDevice)
		{
			const HRESULT hRes = Direct3DDevice->GetDeviceRemovedReason();
			if (hRes != S_OK)
			{
				FGenericCrashContext::SetEngineData(TEXT("GPUCrash.D3DDeviceRemovedReason"), FString::FromInt(hRes));
			}
		}

	#if NV_AFTERMATH
		TArray<UE::RHICore::Nvidia::Aftermath::FCrashResult> AftermathResults;
		UE::RHICore::Nvidia::Aftermath::OnGPUCrash(AftermathResults);
	#endif

		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		if (GameEngine)
		{
			GameEngine->OnGPUCrash();
		}

		// Report the GPU crash which will raise the exception
		ReportGPUCrash(TEXT("GPU Crash dump Triggered"), nullptr);

		GIsGPUCrashed = true;
		if (Direct3DDevice)
		{
			HRESULT hRes = Direct3DDevice->GetDeviceRemovedReason();

			const TCHAR* Reason = TEXT("?");
			switch (hRes)
			{
			case DXGI_ERROR_DEVICE_HUNG:			Reason = TEXT("HUNG"); break;
			case DXGI_ERROR_DEVICE_REMOVED:			Reason = TEXT("REMOVED"); break;
			case DXGI_ERROR_DEVICE_RESET:			Reason = TEXT("RESET"); break;
			case DXGI_ERROR_DRIVER_INTERNAL_ERROR:	Reason = TEXT("INTERNAL_ERROR"); break;
			case DXGI_ERROR_INVALID_CALL:			Reason = TEXT("INVALID_CALL"); break;
			case S_OK:								Reason = TEXT("S_OK"); break;
			}

			// We currently don't support removed devices because FTexture2DResource can't recreate its RHI resources from scratch.
			// We would also need to recreate the viewport swap chains from scratch.			
			UE_LOGF(LogD3D11RHI, Fatal, "Unreal Engine is exiting due to D3D device being lost. (Error: 0x%X - '%ls')", hRes, Reason);
		}
		else
		{
			UE_LOGF(LogD3D11RHI, Fatal, "Unreal Engine is exiting due to D3D device being lost. D3D device was not available to determine DXGI cause.");
		}

		// Workaround for the fact that in non-monolithic builds the exe gets into a weird state and exception handling fails. 
		// @todo investigate why non-monolithic builds fail to capture the exception when graphics driver crashes.
#if !IS_MONOLITHIC
		FPlatformMisc::RequestExit(true, TEXT("TerminateOnDeviceRemoved")); //-V779
#endif
	}
}

void GetAndLogMemoryStats(const FD3D11Adapter& InAdapter, uint64& OutVRAMBudgetBytes, uint64& OutVRAMUsageBytes)
{
	FRHIMemoryStats MemoryStats;
	if (SUCCEEDED(UE::DXGIUtilities::GetD3DMemoryStats(InAdapter.DXGIAdapter, MemoryStats)))
	{
		UE_LOGF(LogD3D11RHI, Error, "\tLocal Budget:\t%7.2f MB", MemoryStats.BudgetLocal / (1024.0f * 1024));
		UE_LOGF(LogD3D11RHI, Error, "\tLocal Used:\t%7.2f MB", MemoryStats.UsedLocal / (1024.0f * 1024));
		UE_LOGF(LogD3D11RHI, Error, "\tSystem Budget:\t%7.2f MB", MemoryStats.BudgetSystem / (1024.0f * 1024));
		UE_LOGF(LogD3D11RHI, Error, "\tSystem Used:\t%7.2f MB", MemoryStats.UsedSystem / (1024.0f * 1024));
		OutVRAMBudgetBytes = MemoryStats.BudgetLocal;
		OutVRAMUsageBytes = MemoryStats.UsedLocal;
	}
}

static void TerminateOnOutOfMemory(HRESULT D3DResult, bool bCreatingTextures)
{
	if (D3DResult == E_OUTOFMEMORY)
	{
		uint64 VRAMBudgetBytes = 0, VRAMUsageBytes = 0;
		GetAndLogMemoryStats(GD3D11RHI->GetAdapter(), VRAMBudgetBytes, VRAMUsageBytes);
		FPlatformMemory::DumpStats(*GLog);
		FPlatformMemory::bIsOOM = true;

		FCoreDelegates::GetGPUOutOfMemoryDelegate().Broadcast(VRAMBudgetBytes, VRAMUsageBytes);

		if (bCreatingTextures)
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *LOCTEXT("OutOfVideoMemoryTextures", "Out of video memory trying to allocate a texture! Make sure your video card has the minimum required memory, try lowering the resolution and/or closing other applications that are running. Exiting...").ToString(), TEXT("Error"));
		}
		else
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *NSLOCTEXT("D3D11RHI", "OutOfMemory", "Out of video memory trying to allocate a rendering resource. Make sure your video card has the minimum required memory, try lowering the resolution and/or closing other applications that are running. Exiting...").ToString(), TEXT("Error"));
		}
#if STATS
		GetRendererModule().DebugLogOnCrash();
#endif
		static IConsoleVariable* GPUCrashOOM = IConsoleManager::Get().FindConsoleVariable(TEXT("r.GPUCrashOnOutOfMemory"));
		if (GPUCrashOOM && GPUCrashOOM->GetInt())
		{
			UE_LOGF(LogD3D11RHI, Fatal, "Out of video memory trying to allocate a rendering resource");
		}
		else
		{
			FPlatformMisc::RequestExit(true, TEXT("TerminateOnOutOfMemory"));
		}
	}
}

void VerifyD3D11ResultNoExit(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device)
{
	check(FAILED(D3DResult));

	const FString& ErrorString = GetD3D11ErrorString(D3DResult, Device);

	UE_LOGF(LogD3D11RHI, Error, "%ls failed with error %ls\n at %ls:%u\n Error Code List: https://docs.microsoft.com/en-us/windows/desktop/direct3ddxgi/dxgi-error", ANSI_TO_TCHAR(Code), *ErrorString, ANSI_TO_TCHAR(Filename), Line);
}

void VerifyD3D11Result(HRESULT D3DResult,const ANSICHAR* Code,const ANSICHAR* Filename,uint32 Line, ID3D11Device* Device)
{
	check(FAILED(D3DResult));

	const FString& ErrorString = GetD3D11ErrorString(D3DResult, Device);

	UE_LOGF(LogD3D11RHI, Error, "%ls failed with error %ls\n at %ls:%u", ANSI_TO_TCHAR(Code), *ErrorString, ANSI_TO_TCHAR(Filename), Line);

	TerminateOnDeviceRemoved(D3DResult, Device);
	TerminateOnOutOfMemory(D3DResult, false);

	UE_LOGF(LogD3D11RHI, Fatal,"%ls failed with error %ls\n at %ls:%u",ANSI_TO_TCHAR(Code), *ErrorString, ANSI_TO_TCHAR(Filename), Line);
}

void VerifyD3D11ShaderResult(FRHIShader* Shader, HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device)
{
	check(FAILED(D3DResult));

	const FString& ErrorString = GetD3D11ErrorString(D3DResult, Device);

	UE_LOGF(LogD3D11RHI, Error, "%ls failed trying to create shader '%ls' with error %ls\n at %ls:%u", ANSI_TO_TCHAR(Code), Shader->GetShaderName(), *ErrorString, ANSI_TO_TCHAR(Filename), Line);
	TerminateOnDeviceRemoved(D3DResult, Device);
	TerminateOnOutOfMemory(D3DResult, false);

	UE_LOGF(LogD3D11RHI, Fatal, "%ls failed trying to create shader '%ls' with error %ls\n at %ls:%u", ANSI_TO_TCHAR(Code), Shader->GetShaderName(), *ErrorString, ANSI_TO_TCHAR(Filename), Line);
}

void VerifyD3D11CreateTextureResult(HRESULT D3DResult, int32 UEFormat,const ANSICHAR* Code,const ANSICHAR* Filename,uint32 Line,uint32 SizeX,uint32 SizeY,uint32 SizeZ,uint8 D3DFormat,uint32 NumMips,uint32 Flags,
	D3D11_USAGE Usage, uint32 CPUAccessFlags, uint32 MiscFlags, uint32 SampleCount, uint32 SampleQuality,
	const void* SubResPtr, uint32 SubResPitch, uint32 SubResSlicePitch, ID3D11Device* Device, const TCHAR* DebugName)
{
	check(FAILED(D3DResult));

	const FString ErrorString = GetD3D11ErrorString(D3DResult, 0);
	const TCHAR* D3DFormatString = UE::DXGIUtilities::GetFormatString((DXGI_FORMAT)D3DFormat);

	FString DebugInfoString;

	if (FScopedDebugInfo* DebugInfo = FScopedDebugInfo::GetDebugInfoStack())
	{
		DebugInfoString = DebugInfo->GetFunctionName();
	}

	UE_LOGF(LogD3D11RHI, Error,
		"%ls failed with error %ls\n at %ls:%u\n Size=%ix%ix%i PF=%d D3DFormat=%ls(0x%08X), NumMips=%i, Flags=%ls, Usage:0x%x, CPUFlags:0x%x, MiscFlags:0x%x, SampleCount:0x%x, SampleQuality:0x%x, SubresPtr:0x%p, SubresPitch:%i, SubresSlicePitch:%i, Name:'%ls', DebugInfo: %ls",
		ANSI_TO_TCHAR(Code),
		*ErrorString,
		ANSI_TO_TCHAR(Filename),
		Line,
		SizeX,
		SizeY,
		SizeZ,
		UEFormat,
		D3DFormatString,
		D3DFormat,
		NumMips,
		*GetD3D11TextureFlagString(Flags),
		Usage,
		CPUAccessFlags,
		MiscFlags,
		SampleCount,
		SampleQuality,
		SubResPtr,
		SubResPitch,
		SubResSlicePitch,
		DebugName ? DebugName : TEXT(""),
		*DebugInfoString);

	TerminateOnDeviceRemoved(D3DResult, Device);
	TerminateOnOutOfMemory(D3DResult, true);

	UE_LOGF(LogD3D11RHI, Fatal,
		"%ls failed with error %ls\n at %ls:%u\n Size=%ix%ix%i PF=%d Format=%ls(0x%08X), NumMips=%i, Flags=%ls, Usage:0x%x, CPUFlags:0x%x, MiscFlags:0x%x, SampleCount:0x%x, SampleQuality:0x%x, SubresPtr:0x%p, SubresPitch:%i, SubresSlicePitch:%i",
		ANSI_TO_TCHAR(Code),
		*ErrorString,
		ANSI_TO_TCHAR(Filename),
		Line,
		SizeX,
		SizeY,
		SizeZ,
		UEFormat,
		D3DFormatString,
		D3DFormat,
		NumMips,
		*GetD3D11TextureFlagString(Flags),
		Usage,
		CPUAccessFlags,
		MiscFlags,
		SampleCount,
		SampleQuality,
		SubResPtr,
		SubResPitch,
		SubResSlicePitch);
}

void VerifyD3D11ResizeViewportResult(
	HRESULT D3DResult,
	const ANSICHAR* Code,
	const ANSICHAR* Filename,
	uint32 Line,
	const FD3D11ResizeViewportState& OldState,
	const FD3D11ResizeViewportState& NewState,
	ID3D11Device* Device)
{
	check(FAILED(D3DResult));

	const FString ErrorString = GetD3D11ErrorString(D3DResult, 0);
	const TCHAR* OldStateFormat = UE::DXGIUtilities::GetFormatString(OldState.Format);
	const TCHAR* NewStateFormat = UE::DXGIUtilities::GetFormatString(NewState.Format);

	UE_LOGF(LogD3D11RHI, Error,
		"%ls failed with error %ls\n at %ls:%u\n (Size=%ix%i Fullscreen=%d Format=%ls(0x%08X)) -> (Size=%ix%i Fullscreen=%d Format=%ls(0x%08X))",
		ANSI_TO_TCHAR(Code),
		*ErrorString,
		ANSI_TO_TCHAR(Filename),
		Line,
		OldState.SizeX,
		OldState.SizeY,
		OldState.bIsFullscreen ? 1 : 0,
		OldStateFormat,
		OldState.Format,
		NewState.SizeX,
		NewState.SizeY,
		NewState.bIsFullscreen ? 1 : 0,
		NewStateFormat,
		NewState.Format);

	TerminateOnDeviceRemoved(D3DResult, Device);
	TerminateOnOutOfMemory(D3DResult, true);

	UE_LOGF(LogD3D11RHI, Fatal,
		"%ls failed with error %ls\n at %ls:%u\n (Size=%ix%i Fullscreen=%d Format=%ls(0x%08X)) -> (Size=%ix%i Fullscreen=%d Format=%ls(0x%08X))",
		ANSI_TO_TCHAR(Code),
		*ErrorString,
		ANSI_TO_TCHAR(Filename),
		Line,
		OldState.SizeX,
		OldState.SizeY,
		OldState.bIsFullscreen ? 1 : 0,
		OldStateFormat,
		OldState.Format,
		NewState.SizeX,
		NewState.SizeY,
		NewState.bIsFullscreen ? 1 : 0,
		NewStateFormat,
		NewState.Format);
}

void VerifyD3D11CreateViewResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device, const FString& ResourceName, const D3D11_UNORDERED_ACCESS_VIEW_DESC& Desc)
{
	check(FAILED(D3DResult));

	D3D11_FEATURE_DATA_FORMAT_SUPPORT FormatSupport{};
	FormatSupport.InFormat = Desc.Format;
	Device->CheckFeatureSupport(D3D11_FEATURE_FORMAT_SUPPORT, &FormatSupport, sizeof(FormatSupport));

	D3D11_FEATURE_DATA_FORMAT_SUPPORT2 FormatSupport2{};
	FormatSupport2.InFormat = Desc.Format;
	Device->CheckFeatureSupport(D3D11_FEATURE_FORMAT_SUPPORT2, &FormatSupport2, sizeof(FormatSupport2));

	const TCHAR* ViewFormat = UE::DXGIUtilities::GetFormatString(Desc.Format);

	const FString& ErrorString = GetD3D11ErrorString(D3DResult, Device);

	UE_LOGF(LogD3D11RHI, Error, "%ls failed with error %ls (Name='%ls', Format='%ls' (0x%08X), FormatSupport=0x%08X, FormatSupport2=0x%08X)\n at %ls:%u",
		ANSI_TO_TCHAR(Code), *ErrorString,
		*ResourceName, ViewFormat, Desc.Format, FormatSupport.OutFormatSupport, FormatSupport2.OutFormatSupport2,
		ANSI_TO_TCHAR(Filename), Line);

	TerminateOnDeviceRemoved(D3DResult, Device);
	TerminateOnOutOfMemory(D3DResult, false);

	UE_LOGF(LogD3D11RHI, Fatal, "%ls failed with error %ls (Name='%ls', Format='%ls' (0x%08X), FormatSupport=0x%08X, FormatSupport2=0x%08X)\n at %ls:%u",
		ANSI_TO_TCHAR(Code), *ErrorString,
		*ResourceName, ViewFormat, Desc.Format, FormatSupport.OutFormatSupport, FormatSupport2.OutFormatSupport2,
		ANSI_TO_TCHAR(Filename), Line);
}

void VerifyD3D11CreateViewResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device, FRHITexture* Texture, const D3D11_UNORDERED_ACCESS_VIEW_DESC& Desc)
{
	const FString TextureName = Texture ? Texture->GetName().ToString() : FString("<Unknown>");
	VerifyD3D11CreateViewResult(D3DResult, Code, Filename, Line, Device, TextureName, Desc);
}

void VerifyD3D11CreateViewResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device, FRHIBuffer* Buffer, const D3D11_UNORDERED_ACCESS_VIEW_DESC& Desc)
{
	FString BufferName = FString("<Unknown>");
#if ENABLE_RHI_VALIDATION
	if (Buffer)
	{
		BufferName = Buffer->GetDebugName();
	}
#endif

	VerifyD3D11CreateViewResult(D3DResult, Code, Filename, Line, Device, BufferName, Desc);
}

void VerifyComRefCount(IUnknown* Object,int32 ExpectedRefs,const TCHAR* Code,const TCHAR* Filename,int32 Line)
{
	int32 NumRefs;

	if (Object)
	{
		Object->AddRef();
		NumRefs = Object->Release();

		checkSlow(NumRefs == ExpectedRefs);

		if (NumRefs != ExpectedRefs)
		{
			UE_LOGF(
				LogD3D11RHI,
				Error,
				"%ls:(%d): %ls has %d refs, expected %d",
				Filename,
				Line,
				Code,
				NumRefs,
				ExpectedRefs
				);
		}
	}
}

FD3D11BoundRenderTargets::FD3D11BoundRenderTargets(ID3D11DeviceContext* InDeviceContext)
{
	FMemory::Memzero(RenderTargetViews,sizeof(RenderTargetViews));
	DepthStencilView = NULL;
	InDeviceContext->OMGetRenderTargets(
		MaxSimultaneousRenderTargets,
		&RenderTargetViews[0],
		&DepthStencilView
		);

	// Find the last non-null rendertarget to determine the max 
	// We traverse the array backwards, since they can be sparse
	for (NumActiveTargets = MaxSimultaneousRenderTargets; NumActiveTargets > 0; --NumActiveTargets)
	{
		if (RenderTargetViews[NumActiveTargets-1] != NULL)
		{
			break;
		}
	}
}

FD3D11BoundRenderTargets::~FD3D11BoundRenderTargets()
{
	// OMGetRenderTargets calls AddRef on each RTV/DSV it returns. We need
	// to make a corresponding call to Release.
	for (int32 TargetIndex = 0; TargetIndex < NumActiveTargets; ++TargetIndex)
	{
		if (RenderTargetViews[TargetIndex] != nullptr)
		{
			RenderTargetViews[TargetIndex]->Release();
		}
	}
	if (DepthStencilView)
	{
		DepthStencilView->Release();
	}
}


//
// Stat declarations.
//


DEFINE_STAT(STAT_D3D11PresentTime);
DEFINE_STAT(STAT_D3D11CustomPresentTime);
DEFINE_STAT(STAT_D3D11TexturesAllocated);
DEFINE_STAT(STAT_D3D11TexturesReleased);
DEFINE_STAT(STAT_D3D11ClearShaderResourceTime);
DEFINE_STAT(STAT_D3D11CreateTextureTime);
DEFINE_STAT(STAT_D3D11LockTextureTime);
DEFINE_STAT(STAT_D3D11UnlockTextureTime);
DEFINE_STAT(STAT_D3D11CopyTextureTime);
DEFINE_STAT(STAT_D3D11NewBoundShaderStateTime);
DEFINE_STAT(STAT_D3D11CreateBoundShaderStateTime);
DEFINE_STAT(STAT_D3D11CleanUniformBufferTime);
DEFINE_STAT(STAT_D3D11UpdateUniformBufferTime);
DEFINE_STAT(STAT_D3D11TexturePoolMemory);
DEFINE_STAT(STAT_D3D11FreeUniformBufferMemory);
DEFINE_STAT(STAT_D3D11NumFreeUniformBuffers);
DEFINE_STAT(STAT_D3D11NumImmutableUniformBuffers);
DEFINE_STAT(STAT_D3D11NumBoundShaderState);
DEFINE_STAT(STAT_D3D11RenderTargetCommits);
DEFINE_STAT(STAT_D3D11RenderTargetCommitsUAV);

#undef LOCTEXT_NAMESPACE
