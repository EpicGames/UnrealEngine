// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11VertexDeclaration.cpp: D3D vertex declaration RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"

/**
 * Key used to look up vertex declarations in the cache.
 */
struct FD3D11VertexDeclarationKey
{
	/** Vertex elements in the declaration. */
	FD3D11VertexElements VertexElements;
	/** Hash of the vertex elements. */
	uint32 Hash;

	uint16 StreamStrides[MaxVertexElementCount];

	/** Initialization constructor. */
	explicit FD3D11VertexDeclarationKey(const FVertexDeclarationElementList& InElements)
	{
		uint16 UsedStreamsMask = 0;
		FMemory::Memzero(StreamStrides);

		for(int32 ElementIndex = 0;ElementIndex < InElements.Num();ElementIndex++)
		{
			const FVertexElement& Element = InElements[ElementIndex];
			D3D11_INPUT_ELEMENT_DESC D3DElement = {0};
			D3DElement.InputSlot = Element.StreamIndex;
			D3DElement.AlignedByteOffset = Element.Offset;
			switch (Element.Type)
			{
			case VET_Float1:		D3DElement.Format = DXGI_FORMAT_R32_FLOAT; break;
			case VET_Float2:		D3DElement.Format = DXGI_FORMAT_R32G32_FLOAT; break;
			case VET_Float3:		D3DElement.Format = DXGI_FORMAT_R32G32B32_FLOAT; break;
			case VET_Float4:		D3DElement.Format = DXGI_FORMAT_R32G32B32A32_FLOAT; break;
			case VET_PackedNormal:	D3DElement.Format = DXGI_FORMAT_R8G8B8A8_SNORM; break; //TODO: uint32 doesn't work because D3D11 squishes it to 0 in the IA-VS conversion
			case VET_UByte4:		D3DElement.Format = DXGI_FORMAT_R8G8B8A8_UINT; break; //TODO: SINT, blendindices
			case VET_UByte4N:		D3DElement.Format = DXGI_FORMAT_R8G8B8A8_UNORM; break;
			case VET_Color:			D3DElement.Format = DXGI_FORMAT_B8G8R8A8_UNORM; break;
			case VET_Short2:		D3DElement.Format = DXGI_FORMAT_R16G16_SINT; break;
			case VET_Short4:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_SINT; break;
			case VET_Short2N:		D3DElement.Format = DXGI_FORMAT_R16G16_SNORM; break;
			case VET_Half2:			D3DElement.Format = DXGI_FORMAT_R16G16_FLOAT; break;
			case VET_Half4:			D3DElement.Format = DXGI_FORMAT_R16G16B16A16_FLOAT; break;
			case VET_Short4N:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_SNORM; break;
			case VET_UShort2:		D3DElement.Format = DXGI_FORMAT_R16G16_UINT; break;
			case VET_UShort4:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_UINT; break;
			case VET_UShort2N:		D3DElement.Format = DXGI_FORMAT_R16G16_UNORM; break;
			case VET_UShort4N:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_UNORM; break;
			case VET_URGB10A2N:		D3DElement.Format = DXGI_FORMAT_R10G10B10A2_UNORM; break;
			case VET_UInt:			D3DElement.Format = DXGI_FORMAT_R32_UINT; break;
			default: UE_LOGF(LogD3D11RHI, Fatal, "Unknown RHI vertex element type %u", (uint8)InElements[ElementIndex].Type);
			};
			D3DElement.SemanticName = "ATTRIBUTE";
			D3DElement.SemanticIndex = Element.AttributeIndex;
			D3DElement.InputSlotClass = Element.bUseInstanceIndex ? D3D11_INPUT_PER_INSTANCE_DATA : D3D11_INPUT_PER_VERTEX_DATA;

			// This is a divisor to apply to the instance index used to read from this stream.
			D3DElement.InstanceDataStepRate = Element.bUseInstanceIndex ? 1 : 0;

			if ((UsedStreamsMask & 1 << Element.StreamIndex) != 0)
			{
				ensure(StreamStrides[Element.StreamIndex] == Element.Stride);
			}
			else
			{
				UsedStreamsMask = UsedStreamsMask | (1 << Element.StreamIndex);
				StreamStrides[Element.StreamIndex] = Element.Stride;
			}

			VertexElements.Add(D3DElement);
		}

		// Sort by stream then offset.
		struct FCompareDesc
		{
			FORCEINLINE bool operator()( const D3D11_INPUT_ELEMENT_DESC& A, const D3D11_INPUT_ELEMENT_DESC &B ) const
			{
				return ((int32)A.AlignedByteOffset + A.InputSlot * MAX_uint16) < ((int32)B.AlignedByteOffset + B.InputSlot * MAX_uint16);
			}
		};
		Algo::Sort(VertexElements, FCompareDesc());

		// Hash once.
		Hash = FCrc::MemCrc_DEPRECATED(VertexElements.GetData(),VertexElements.Num()*sizeof(D3D11_INPUT_ELEMENT_DESC));
		Hash = FCrc::MemCrc_DEPRECATED(StreamStrides, sizeof(StreamStrides), Hash);
	}
};

/** Hashes the array of D3D11 vertex element descriptions. */
uint32 GetTypeHash(const FD3D11VertexDeclarationKey& Key)
{
	return Key.Hash;
}

/** Compare two vertex declaration keys. */
bool operator==(const FD3D11VertexDeclarationKey& A, const FD3D11VertexDeclarationKey& B)
{
	return A.VertexElements == B.VertexElements
		&& !FMemory::Memcmp(A.StreamStrides, B.StreamStrides, sizeof(A.StreamStrides));
}

/** Global cache of vertex declarations. */
TMap<FD3D11VertexDeclarationKey,FVertexDeclarationRHIRef> GVertexDeclarationCache;

FVertexDeclarationRHIRef FD3D11DynamicRHI::RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements)
{
	// Construct a key from the elements.
	FD3D11VertexDeclarationKey Key(Elements);

	// Check for a cached vertex declaration.
	FVertexDeclarationRHIRef* VertexDeclarationRefPtr = GVertexDeclarationCache.Find(Key);
	if (VertexDeclarationRefPtr == NULL)
	{
		// Create and add to the cache if it doesn't exist.
		VertexDeclarationRefPtr = &GVertexDeclarationCache.Add(Key,new FD3D11VertexDeclaration(Key.VertexElements, Key.StreamStrides));
	}

	// The cached declaration must match the input declaration!
	check(VertexDeclarationRefPtr);
	check(IsValidRef(*VertexDeclarationRefPtr));
	FD3D11VertexDeclaration* D3D11VertexDeclaration = (FD3D11VertexDeclaration*)VertexDeclarationRefPtr->GetReference();
	checkSlow(D3D11VertexDeclaration->VertexElements == Key.VertexElements);

	return *VertexDeclarationRefPtr;
}
