// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Viewport.cpp: D3D viewport RHI implementation.
=============================================================================*/

#include "D3D11Viewport.h"
#include "D3D11RHIPrivate.h"
#include "RenderCore.h"
#include "HDRHelper.h"
#include "Engine/RendererSettings.h"
#include "HAL/ThreadHeartBeat.h"
#include "RHIUtilities.h"

#ifndef D3D11_WITH_DWMAPI
#define D3D11_WITH_DWMAPI	1
#endif

#if D3D11_WITH_DWMAPI
	#include "Windows/AllowWindowsPlatformTypes.h"
		#include <dwmapi.h>
#endif	//D3D11_WITH_DWMAPI

#ifndef DXGI_PRESENT_ALLOW_TEARING
#define DXGI_PRESENT_ALLOW_TEARING          0x00000200UL
#endif

#ifndef DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING
#define DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING  2048
#endif


/**
 * RHI console variables used by viewports.
 */
namespace RHIConsoleVariables
{
	int32 bSyncWithDWM = 0;
	static FAutoConsoleVariableRef CVarSyncWithDWM(
		TEXT("RHI.SyncWithDWM"),
		bSyncWithDWM,
		TEXT("If true, synchronize with the desktop window manager for vblank."),
		ECVF_RenderThreadSafe
		);

	float RefreshPercentageBeforePresent = 1.0f;
	static FAutoConsoleVariableRef CVarRefreshPercentageBeforePresent(
		TEXT("RHI.RefreshPercentageBeforePresent"),
		RefreshPercentageBeforePresent,
		TEXT("The percentage of the refresh period to wait before presenting."),
		ECVF_RenderThreadSafe
		);

	int32 TargetRefreshRate = 0;
	static FAutoConsoleVariableRef CVarTargetRefreshRate(
		TEXT("RHI.TargetRefreshRate"),
		TargetRefreshRate,
		TEXT("If non-zero, the display will never update more often than the target refresh rate (in Hz)."),
		ECVF_RenderThreadSafe
		);

	float SyncRefreshThreshold = 1.05f;
	static FAutoConsoleVariableRef CVarSyncRefreshThreshold(
		TEXT("RHI.SyncRefreshThreshold"),
		SyncRefreshThreshold,
		TEXT("Threshold for time above which vsync will be disabled as a percentage of the refresh rate."),
		ECVF_RenderThreadSafe
		);

	int32 MaxSyncCounter = 8;
	static FAutoConsoleVariableRef CVarMaxSyncCounter(
		TEXT("RHI.MaxSyncCounter"),
		MaxSyncCounter,
		TEXT("Maximum sync counter to smooth out vsync transitions."),
		ECVF_RenderThreadSafe
		);

	int32 SyncThreshold = 7;
	static FAutoConsoleVariableRef CVarSyncThreshold(
		TEXT("RHI.SyncThreshold"),
		SyncThreshold,
		TEXT("Number of consecutive 'fast' frames before vsync is enabled."),
		ECVF_RenderThreadSafe
		);

	int32 MaximumFrameLatency = 3;
	static FAutoConsoleVariableRef CVarMaximumFrameLatency(
		TEXT("RHI.MaximumFrameLatency"),
		MaximumFrameLatency,
		TEXT("Number of frames that can be queued for render."),
		ECVF_RenderThreadSafe
		);
};

/**
 * Returns the current swap chain flags but with the same tearing policy used during construction.
 */
uint32 FD3D11Viewport::GetSwapChainFlags()
{
	uint32 SwapChainFlags = GSwapChainFlags;

	// Ensure AllowTearing consistency or ResizeBuffers will fail with E_INVALIDARG
	if (bAllowTearing != !!(SwapChainFlags & DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING))
	{
		SwapChainFlags ^= DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING;
	}

	return SwapChainFlags;
}


/**
 * Creates a FD3D11Surface to represent a swap chain's back buffer.
 */
FD3D11Texture* FD3D11Viewport::GetSwapChainSurface(FD3D11DynamicRHI* D3DRHI, EPixelFormat PixelFormat, uint32 SizeX, uint32 SizeY, IDXGISwapChain* SwapChain)
{
	// Grab the back buffer
	TRefCountPtr<ID3D11Texture2D> BackBufferResource;
	if (SwapChain)
	{
		VERIFYD3D11RESULT_EX(SwapChain->GetBuffer(0, IID_ID3D11Texture2D, (void**)BackBufferResource.GetInitReference()), D3DRHI->GetDevice());
	}
	else
	{
		// Create custom back buffer texture as no swap chain is created in pixel streaming windowless mode
		DXGI_FORMAT TextureFormat = GetRenderTargetFormat(PixelFormat);
		D3D11_TEXTURE2D_DESC TextureDesc;
		FMemory::Memzero(TextureDesc);
		TextureDesc.Width = SizeX;
		TextureDesc.Height = SizeY;
		TextureDesc.MipLevels = 1;
		TextureDesc.ArraySize = 1;
		TextureDesc.Format = TextureFormat;
		TextureDesc.SampleDesc.Count = 1;
		TextureDesc.SampleDesc.Quality = 0;
		TextureDesc.Usage = D3D11_USAGE_DEFAULT;
		TextureDesc.BindFlags = D3D11_BIND_RENDER_TARGET | D3D11_BIND_SHADER_RESOURCE;
		VERIFYD3D11RESULT_EX(D3DRHI->GetDevice()->CreateTexture2D(&TextureDesc, NULL, BackBufferResource.GetInitReference()), D3DRHI->GetDevice());
	}

	// create the render target view
	TRefCountPtr<ID3D11RenderTargetView> BackBufferRenderTargetView;
	TRefCountPtr<ID3D11RenderTargetView> BackBufferRenderTargetViewRight; // right eye RTV
	
	// dx11.1 active stereoscopy initialization
	bool const bQuadBufferStereo = D3DRHI->IsQuadBufferStereoEnabled();
	if (bQuadBufferStereo)
	{
		// left
		CD3D11_RENDER_TARGET_VIEW_DESC RTVDescCD3D11_left(D3D11_RTV_DIMENSION_TEXTURE2DARRAY, DXGI_FORMAT_R10G10B10A2_UNORM, 0, 0, 1);
		VERIFYD3D11RESULT_EX(D3DRHI->GetDevice()->CreateRenderTargetView(BackBufferResource, &RTVDescCD3D11_left, BackBufferRenderTargetView.GetInitReference()), D3DRHI->GetDevice());
		
		// right
		CD3D11_RENDER_TARGET_VIEW_DESC renderTargetViewRightDesc_right(D3D11_RTV_DIMENSION_TEXTURE2DARRAY, DXGI_FORMAT_R10G10B10A2_UNORM, 0, 1, 1);
		VERIFYD3D11RESULT_EX(D3DRHI->GetDevice()->CreateRenderTargetView(BackBufferResource, &renderTargetViewRightDesc_right, BackBufferRenderTargetViewRight.GetInitReference()), D3DRHI->GetDevice());
	}
	else
	{
		D3D11_RENDER_TARGET_VIEW_DESC RTVDesc;
		RTVDesc.Format = DXGI_FORMAT_UNKNOWN;
		RTVDesc.ViewDimension = D3D11_RTV_DIMENSION_TEXTURE2D;
		RTVDesc.Texture2D.MipSlice = 0;
		VERIFYD3D11RESULT_EX(D3DRHI->GetDevice()->CreateRenderTargetView(BackBufferResource, &RTVDesc, BackBufferRenderTargetView.GetInitReference()), D3DRHI->GetDevice());
	}

	D3D11_TEXTURE2D_DESC TextureDesc;
	BackBufferResource->GetDesc(&TextureDesc);

	TArray<TRefCountPtr<ID3D11RenderTargetView> > RenderTargetViews;
	RenderTargetViews.Add(BackBufferRenderTargetView);

	// add right eye render target view
	if (bQuadBufferStereo)
	{
		RenderTargetViews.Add(BackBufferRenderTargetViewRight);
	}

	// create a shader resource view to allow using the backbuffer as a texture
	TRefCountPtr<ID3D11ShaderResourceView> BackBufferShaderResourceView;
	D3D11_SHADER_RESOURCE_VIEW_DESC SRVDesc;
	SRVDesc.Format = DXGI_FORMAT_UNKNOWN;
	SRVDesc.ViewDimension = D3D11_SRV_DIMENSION_TEXTURE2D;
	SRVDesc.Texture2D.MostDetailedMip = 0;
	SRVDesc.Texture2D.MipLevels = 1;
	VERIFYD3D11RESULT_EX(D3DRHI->GetDevice()->CreateShaderResourceView(BackBufferResource,&SRVDesc,BackBufferShaderResourceView.GetInitReference()), D3DRHI->GetDevice());

	const FRHITextureCreateDesc CreateDesc =
		FRHITextureCreateDesc::Create(TEXT("FD3D11Viewport::GetSwapChainSurface"),
			bQuadBufferStereo ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D)
		.SetArraySize(bQuadBufferStereo ? 2 : 1)
		.SetExtent(FIntPoint(TextureDesc.Width, TextureDesc.Height))
		.SetFormat(PixelFormat)
		.SetFlags(ETextureCreateFlags::RenderTargetable)
		.DetermineInititialState();

	FD3D11Texture* NewTexture = new FD3D11Texture(
		CreateDesc,
		BackBufferResource,
		BackBufferShaderResourceView,
		bQuadBufferStereo ? 2 : 1,
		false,
		RenderTargetViews,
		{}
	);

	return NewTexture;
}

FD3D11Viewport::~FD3D11Viewport()
{
	check(IsInRHIThread() || IsInRenderingThread());

#if UE_INPUT_LATENCY_TRACKING
	if (SwapChain.IsValid())
	{
		FRHIFrameFlipStatTrackingRunnable::Release();
	}
#endif

	// If the swap chain was in fullscreen mode, switch back to windowed before releasing the swap chain.
	// DXGI throws an error otherwise.
	if (SwapChain)
	{
		VERIFYD3D11RESULT_EX(SwapChain->SetFullscreenState(false, NULL), D3DRHI->GetDevice());
	}

	D3DRHI->Viewports.Remove(this);
}

DXGI_MODE_DESC FD3D11Viewport::SetupDXGI_MODE_DESC() const
{
	DXGI_MODE_DESC Ret;

	Ret.Width = SizeX;
	Ret.Height = SizeY;
	Ret.RefreshRate.Numerator = 0;	// illamas: use 0 to avoid a potential mismatch with hw
	Ret.RefreshRate.Denominator = 0;	// illamas: ditto
	Ret.Format = GetRenderTargetFormat(PixelFormat);
	Ret.ScanlineOrdering = DXGI_MODE_SCANLINE_ORDER_UNSPECIFIED;
	Ret.Scaling = DXGI_MODE_SCALING_UNSPECIFIED;

	return Ret;
}

void FD3D11Viewport::Resize(uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat PreferredPixelFormat)
{
	if (bResizeInProgress)
	{
		return;
	}

	bResizeInProgress = true;
	ON_SCOPE_EXIT
	{
		bResizeInProgress = false;
	};

	// Unbind any dangling references to resources
	D3DRHI->SetRenderTargets(0, nullptr, nullptr);
	D3DRHI->ClearState();
	D3DRHI->GetDeviceContext()->Flush(); // Potential perf hit

	if (IsValidRef(CustomPresent))
	{
		CustomPresent->OnBackBufferResize();
	}

	// Release our backbuffer reference, as required by DXGI before calling ResizeBuffers.
	if (IsValidRef(BackBuffer))
	{
		check(BackBuffer->GetRefCount() == 1);

		checkComRefCount(BackBuffer->GetResource(),1);
		checkComRefCount(BackBuffer->GetRenderTargetView(0, -1),1);
		checkComRefCount(BackBuffer->GetShaderResourceView(),1);
	}
	BackBuffer.SafeRelease();

	// Flush the outstanding GPU work and wait for it to complete.
	FlushRenderingCommands();

	// Make sure we use a format the current device supports.
	PreferredPixelFormat = D3DRHI->GetDisplayFormat(PreferredPixelFormat);

	const FD3D11ResizeViewportState OldState{ SizeX, SizeY, GetRenderTargetFormat(PixelFormat), bIsFullscreen };
	const FD3D11ResizeViewportState NewState{ InSizeX, InSizeY, GetRenderTargetFormat(PreferredPixelFormat), bInIsFullscreen };

	if(SizeX != InSizeX || SizeY != InSizeY || PixelFormat != PreferredPixelFormat)
	{
		SizeX = InSizeX;
		SizeY = InSizeY;
		PixelFormat = PreferredPixelFormat;

		check(SizeX > 0);
		check(SizeY > 0);

		if (bNeedSwapChain)
		{
			// Resize the swap chain.

			const UINT SwapChainFlags = GetSwapChainFlags();
			const DXGI_FORMAT RenderTargetFormat = GetRenderTargetFormat(PixelFormat);

			// Resize all existing buffers, don't change count
			VERIFYD3D11RESIZEVIEWPORTRESULT(SwapChain->ResizeBuffers(0, SizeX, SizeY, RenderTargetFormat, SwapChainFlags), OldState, NewState, D3DRHI->GetDevice());

			if (bInIsFullscreen)
			{
				DXGI_MODE_DESC BufferDesc = SetupDXGI_MODE_DESC();

				if (FAILED(SwapChain->ResizeTarget(&BufferDesc)))
				{
					ResetSwapChainInternal(true);
					VERIFYD3D11RESIZEVIEWPORTRESULT(SwapChain->ResizeBuffers(0, SizeX, SizeY, RenderTargetFormat, SwapChainFlags), OldState, NewState, D3DRHI->GetDevice());
				}
			}
		}
	}

	if(bIsFullscreen != bInIsFullscreen)
	{
		bIsFullscreen = bInIsFullscreen;
		ValidState = VIEWPORT_INVALID;

		if (bNeedSwapChain)
		{
			// Use ConditionalResetSwapChain to call SetFullscreenState, to handle the failure case.
			// Ignore the viewport's focus state; since Resize is called as the result of a user action we assume authority without waiting for Focus.
			ResetSwapChainInternal(true);

			if (!bIsFullscreen)
			{
				// When exiting fullscreen, make sure that the window has the correct size. This is necessary in the following scenario:
				//	* we enter exclusive fullscreen with a resolution lower than the monitor's native resolution, or from windowed with a window size smaller than the screen
				//	* the application loses focus, so Slate asks us to switch to Windowed Fullscreen (see FSlateRenderer::IsViewportFullscreen)
				//	* InSizeX and InSizeY are given to us as the monitor resolution, so we resize the buffers to the correct resolution below
				//	* however, the target still has the smaller size, because Slate doesn't know it has to resize the window too (as far as it's concerned, it's already the right size)
				//	* therefore, we need to call ResizeTarget, which in windowed mode behaves like SetWindowPos.
				const DXGI_MODE_DESC BufferDesc = SetupDXGI_MODE_DESC();
				SwapChain->ResizeTarget(&BufferDesc);
			}

			DXGI_FORMAT RenderTargetFormat = GetRenderTargetFormat(PixelFormat);
			VERIFYD3D11RESIZEVIEWPORTRESULT(SwapChain->ResizeBuffers(0, SizeX, SizeY, RenderTargetFormat, GetSwapChainFlags()), OldState, NewState, D3DRHI->GetDevice());
		}
	}

	RECT WindowRect = {};
	GetWindowRect(WindowHandle, &WindowRect);

	FVector2D WindowTopLeft((float)WindowRect.left, (float)WindowRect.top);
	FVector2D WindowBottomRight((float)WindowRect.right, (float)WindowRect.bottom);
	bool bHDREnabled;
	HDRGetMetaData(DisplayOutputFormat, DisplayColorGamut, bHDREnabled, WindowTopLeft, WindowBottomRight, (void*)WindowHandle);

	// Float RGBA backbuffers are requested whenever HDR mode is desired
	if (bHDREnabled)
	{
		EnableHDR();
	}
	else
	{
		ShutdownHDR();
	}

	// Create a RHI surface to represent the viewport's back buffer.
	BackBuffer = GetSwapChainSurface(D3DRHI, PixelFormat, SizeX, SizeY, SwapChain);
}

/** Returns true if desktop composition is enabled. */
static bool IsCompositionEnabled()
{
	BOOL bDwmEnabled = false;
#if D3D11_WITH_DWMAPI
	DwmIsCompositionEnabled(&bDwmEnabled);
#endif	//D3D11_WITH_DWMAPI
	return !!bDwmEnabled;
}

/** Presents the swap chain checking the return result. */
bool FD3D11Viewport::PresentChecked(IRHICommandContext& RHICmdContext, int32 SyncInterval)
{
	HRESULT Result = S_OK;
	bool bNeedNativePresent = true;

	if (IsValidRef(CustomPresent))
	{
		SCOPE_CYCLE_COUNTER(STAT_D3D11CustomPresentTime);
		bNeedNativePresent = CustomPresent->Present(this, RHICmdContext, SyncInterval);
	}

	if (bNeedNativePresent)
	{
		if (SwapChain.IsValid())
		{
			if (PresentBeginLatencyCallback)
			{
				PresentBeginLatencyCallback();
			}

			// Check if the viewport's swap chain has been invalidated by DXGI.
			BOOL bSwapChainFullscreenState;
			TRefCountPtr<IDXGIOutput> SwapChainOutput;
			SwapChain->GetFullscreenState(&bSwapChainFullscreenState, SwapChainOutput.GetInitReference());
			// Can't compare BOOL with bool...
			if ((!!bSwapChainFullscreenState) != bIsFullscreen)
			{
				ValidState = (VIEWPORT_INVALID | VIEWPORT_FULLSCREEN_LOST);
			}
			else
			{
			// Present the back buffer to the viewport window.
			uint32 Flags = 0;
			if( (GetSwapChainFlags() & DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING) != 0 && !SyncInterval && !bIsFullscreen )
			{
				Flags |= DXGI_PRESENT_ALLOW_TEARING;
			}

			{
				FRenderThreadIdleScope IdleScope(ERenderThreadIdleTypes::WaitingForGPUPresent);
				TRACE_CPUPROFILER_EVENT_SCOPE(D3D11_Present);
				Result = SwapChain->Present(SyncInterval, Flags);
			}

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
			extern int32 GLogDX11RTRebinds;
			extern FThreadSafeCounter GDX11RTRebind;
			extern FThreadSafeCounter GDX11CommitGraphicsResourceTables;
			if (GLogDX11RTRebinds)
			{
				static int Counter = 0;
				Counter++;
				if (Counter == 60)
				{
					Counter = 0;
					int32 RTRebinds = GDX11RTRebind.Set(0);
					int32 CommitGraphicsResourceTables = GDX11CommitGraphicsResourceTables.Set(0);
					FGenericPlatformMisc::LowLevelOutputDebugStringf(TEXT("RT Rebind %6.2f Commit Graphics Resource Tables %6.2f\n"), RTRebinds / 60.f, CommitGraphicsResourceTables / 60.f);
				}
			}
#endif
		}
		}

		if (IsValidRef(CustomPresent))
		{
			CustomPresent->PostPresent();
		}
	}

	if (PresentEndLatencyCallback)
	{
		PresentEndLatencyCallback();
	}

	FThreadHeartBeat::Get().PresentFrame();
	if (FAILED(Result))
	{
		PresentFailCount++;
		UE_LOGF(LogRHI, Error, "Present Fail Count %i", PresentFailCount);
		DXGI_SWAP_CHAIN_DESC Desc;
		UE_LOGF(LogRHI, Error, "SyncInterval %i", SyncInterval);
		if (SwapChain.IsValid())
		{
			if (!FAILED(SwapChain->GetDesc(&Desc)))
			{
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferDesc.Width %i", Desc.BufferDesc.Width);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferDesc.Height %i", Desc.BufferDesc.Height);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferDesc.RefreshRate.Numerator %i", Desc.BufferDesc.RefreshRate.Numerator);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferDesc.RefreshRate.Denominator %i", Desc.BufferDesc.RefreshRate.Denominator);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferDesc.Format %i", Desc.BufferDesc.Format);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferDesc.ScanlineOrdering %i", Desc.BufferDesc.ScanlineOrdering);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferDesc.Scaling %i", Desc.BufferDesc.Scaling);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.SampleDesc.Count %i", Desc.SampleDesc.Count);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.SampleDesc.Quality %i", Desc.SampleDesc.Quality);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferUsage %i", Desc.BufferUsage);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.BufferCount %i", Desc.BufferCount);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.OutputWindow %p", Desc.OutputWindow);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.Windowed %ls", Desc.Windowed ? TEXT("true") : TEXT("false"));
				UE_LOGF(LogRHI, Error, "SwapChainDesc.SwapEffect %u", Desc.SwapEffect);
				UE_LOGF(LogRHI, Error, "SwapChainDesc.Flags %u", Desc.Flags);
			}
		}
		if (PresentFailCount > 10)
		{
			VERIFYD3D11RESULT_EX(Result, D3DRHI->GetDevice());
		}
		else
		{
			ValidState = (VIEWPORT_INVALID | VIEWPORT_FULLSCREEN_LOST);
		}
	}
	else
	{
		PresentFailCount = 0;
	}

	FThreadHeartBeat::Get().PresentFrame();

	D3DRHI->GetDeviceContext()->OMSetRenderTargets(0,0,0);

	UINT PresentID;
	if ( (SwapChain.IsValid()) && (SUCCEEDED(SwapChain->GetLastPresentCount(&PresentID))) )
	{
		GRHIPresentCounter = PresentID + 1;
	}
	else
	{
		GRHIPresentCounter++;
	}

	return bNeedNativePresent;
}

/** Blocks the CPU to synchronize with vblank by communicating with DWM. */
void FD3D11Viewport::PresentWithVsyncDWM(IRHICommandContext& RHICmdContext)
{
#if D3D11_WITH_DWMAPI
	LARGE_INTEGER Cycles;
	DWM_TIMING_INFO TimingInfo;

	// Find out how long since we last flipped and query DWM for timing information.
	QueryPerformanceCounter(&Cycles);
	FMemory::Memzero(TimingInfo);
	TimingInfo.cbSize = sizeof(DWM_TIMING_INFO);
	// Starting at windows 8.1 null must be passed into this method for it to work.  null also works on previous versions
	DwmGetCompositionTimingInfo(nullptr, &TimingInfo);

	uint64 QpcAtFlip = Cycles.QuadPart;
	uint64 CyclesSinceLastFlip = Cycles.QuadPart - LastFlipTime;
	float CPUTime = FPlatformTime::ToMilliseconds(CyclesSinceLastFlip);
	float GPUTime = FPlatformTime::ToMilliseconds(TimingInfo.qpcFrameComplete - LastCompleteTime);
	float DisplayRefreshPeriod = FPlatformTime::ToMilliseconds(TimingInfo.qpcRefreshPeriod);

	// Find the smallest multiple of the refresh rate that is >= 33ms, our target frame rate.
	float RefreshPeriod = DisplayRefreshPeriod;
	if(RHIConsoleVariables::TargetRefreshRate > 0 && RefreshPeriod > 1.0f)
	{
		while(RefreshPeriod - (1000.0f / RHIConsoleVariables::TargetRefreshRate) < -1.0f)
		{
			RefreshPeriod *= 2.0f;
		}
	}


	// If the last frame hasn't completed yet, we don't know how long the GPU took.
	bool bValidGPUTime = (TimingInfo.cFrameComplete > LastFrameComplete);
	if (bValidGPUTime)
	{
		GPUTime /= (float)(TimingInfo.cFrameComplete - LastFrameComplete);
	}

	// Update the sync counter depending on how much time it took to complete the previous frame.
	float FrameTime = FMath::Max<float>(CPUTime, GPUTime);
	if (FrameTime >= RHIConsoleVariables::SyncRefreshThreshold * RefreshPeriod)
	{
		SyncCounter--;
	}
	else if (bValidGPUTime)
	{
		SyncCounter++;
	}
	SyncCounter = FMath::Clamp<int32>(SyncCounter, 0, RHIConsoleVariables::MaxSyncCounter);

	// If frames are being completed quickly enough, block for vsync.
	bool bSync = (SyncCounter >= RHIConsoleVariables::SyncThreshold);
	if (bSync)
	{
		// This flushes the previous present call and blocks until it is made available to DWM.
		D3DRHI->GetDeviceContext()->Flush();
		DwmFlush();

		// We sleep a percentage of the remaining time. The trick is to get the
		// present call in after the vblank we just synced for but with time to
		// spare for the next vblank.
		float MinFrameTime = RefreshPeriod * RHIConsoleVariables::RefreshPercentageBeforePresent;
		float TimeToSleep;
		do 
		{
			QueryPerformanceCounter(&Cycles);
			float TimeSinceFlip = FPlatformTime::ToMilliseconds(Cycles.QuadPart - LastFlipTime);
			TimeToSleep = (MinFrameTime - TimeSinceFlip);
			if (TimeToSleep > 0.0f)
			{
				FPlatformProcess::Sleep(TimeToSleep * 0.001f);
			}
		}
		while (TimeToSleep > 0.0f);
	}

	// Present.
	PresentChecked(RHICmdContext, /*SyncInterval=*/ 0);

	// If we are forcing <= 30Hz, block the CPU an additional amount of time if needed.
	// This second block is only needed when RefreshPercentageBeforePresent < 1.0.
	if (bSync)
	{
		LARGE_INTEGER LocalCycles;
		float TimeToSleep;
		bool bSaveCycles = false;
		do 
		{
			QueryPerformanceCounter(&LocalCycles);
			float TimeSinceFlip = FPlatformTime::ToMilliseconds(LocalCycles.QuadPart - LastFlipTime);
			TimeToSleep = (RefreshPeriod - TimeSinceFlip);
			if (TimeToSleep > 0.0f)
			{
				bSaveCycles = true;
				FPlatformProcess::Sleep(TimeToSleep * 0.001f);
			}
		}
		while (TimeToSleep > 0.0f);

		if (bSaveCycles)
		{
			Cycles = LocalCycles;
		}
	}

	// If we are dropping vsync reset the counter. This provides a debounce time
	// before which we try to vsync again.
	if (!bSync && bSyncedLastFrame)
	{
		SyncCounter = 0;
	}

	if (bSync != bSyncedLastFrame || UE_LOG_ACTIVE(LogRHI,VeryVerbose))
	{
		UE_LOGF(LogRHI,Verbose,"BlockForVsync[%d]: CPUTime:%.2fms GPUTime[%d]:%.2fms Blocked:%.2fms Pending/Complete:%lld/%lld",
			bSync,
			CPUTime,
			bValidGPUTime,
			GPUTime,
			FPlatformTime::ToMilliseconds(Cycles.QuadPart - QpcAtFlip),
			TimingInfo.cFramePending,
			TimingInfo.cFrameComplete);
	}

	// Remember if we synced, when the frame completed, etc.
	bSyncedLastFrame = bSync;
	LastFlipTime = Cycles.QuadPart;
	LastFrameComplete = TimingInfo.cFrameComplete;
	LastCompleteTime = TimingInfo.qpcFrameComplete;
#endif	//D3D11_WITH_DWMAPI
}

bool FD3D11Viewport::Present(IRHICommandContext& RHICmdContext, bool bLockToVsync)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D11Viewport::Present);

	bool bNativelyPresented = true;
#if	D3D11_WITH_DWMAPI
	// We can't call Present if !bIsValid, as it waits a window message to be processed, but the main thread may not be pumping the message handler.
	if(ValidState != 0 && SwapChain.IsValid())
	{
		// Check if the viewport's swap chain has been invalidated by DXGI.
		BOOL bSwapChainFullscreenState;
		TRefCountPtr<IDXGIOutput> SwapChainOutput;
		VERIFYD3D11RESULT_EX(SwapChain->GetFullscreenState(&bSwapChainFullscreenState,SwapChainOutput.GetInitReference()), D3DRHI->GetDevice());
		// Can't compare BOOL with bool...
		if ( (!!bSwapChainFullscreenState)  != bIsFullscreen )
		{
			ValidState = VIEWPORT_INVALID;
		}
	}
	if (MaximumFrameLatency != RHIConsoleVariables::MaximumFrameLatency)
	{
		MaximumFrameLatency = RHIConsoleVariables::MaximumFrameLatency;	
		TRefCountPtr<IDXGIDevice1> DXGIDevice;
		VERIFYD3D11RESULT_EX(D3DRHI->GetDevice()->QueryInterface(IID_IDXGIDevice, (void**)DXGIDevice.GetInitReference()), D3DRHI->GetDevice());
		DXGIDevice->SetMaximumFrameLatency(MaximumFrameLatency);
	}

	if(0 != (ValidState & VIEWPORT_INVALID))
	{
		return false;
	}
	// When desktop composition is enabled, locking to vsync via the Present
	// call is unreliable. Instead, communicate with the desktop window manager
	// directly to enable vsync.
	const bool bSyncWithDWM = bLockToVsync && !bIsFullscreen && RHIConsoleVariables::bSyncWithDWM && IsCompositionEnabled();
	if (bSyncWithDWM)
	{
		PresentWithVsyncDWM(RHICmdContext);
	}
	else
#endif	//D3D11_WITH_DWMAPI
	{
		// Present the back buffer to the viewport window.
		bNativelyPresented = PresentChecked(RHICmdContext, bLockToVsync ? RHIGetSyncInterval() : 0);
	}
	return bNativelyPresented;
}

void* FD3D11Viewport::GetNativeSwapChain() const
{
	return GetSwapChain();
}

void* FD3D11Viewport::GetNativeBackBufferTexture() const
{
	return GetBackBuffer()->GetD3D11Texture2D();
}

void* FD3D11Viewport::GetNativeBackBufferRT() const
{
	return GetBackBuffer()->GetRenderTargetView(0, 0);
}

void FD3D11Viewport::WaitForFrameEventCompletion()
{
	if (FrameSyncEvent.IsValid() && !FrameSyncEvent->IsComplete())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitForFrameEventCompletion);
		FD3D11Sync::PollSyncsUntil(FrameSyncEvent);
	}
}

void FD3D11Viewport::IssueFrameEvent()
{
	FrameSyncEvent = FGraphEvent::CreateGraphEvent();
	FD3D11Sync::EnqueueSync(FrameSyncEvent);
}

/*=============================================================================
 *	The following RHI functions must be called from the main thread.
 *=============================================================================*/
FViewportRHIRef FD3D11DynamicRHI::RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	check(IsInGameThread());
	return new FD3D11Viewport(this, (HWND)WindowHandle, SizeX, SizeY, bIsFullscreen, PixelFormat);
}

void FD3D11DynamicRHI::RHIResizeViewport(FRHIViewport* ViewportRHI, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	check(IsInGameThread());
	ResourceCast(ViewportRHI)->Resize(SizeX, SizeY, bIsFullscreen, PixelFormat);
}

void FD3D11DynamicRHI::RHITick(float DeltaTime)
{
	check(IsInGameThread());
	
	// Check if any swap chains have been invalidated.
	for(int32 ViewportIndex = 0; ViewportIndex < Viewports.Num(); ViewportIndex++)
	{
		Viewports[ViewportIndex]->ConditionalResetSwapChain(false);
	}
}

/*=============================================================================
 *	Viewport functions.
 *=============================================================================*/

void FD3D11DynamicRHI::RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* ViewportRHI, const FRHIPresentArgs& PresentArgs)
{
	//
	// Since D3D11 RHI does not implement parallel translation, we can enqueue the present from within an RHI command lambda.
	// This will run after all prior work that has already been recorded by the renderer, on the RHI thread.
	//
	RHICmdList.EnqueueLambda(TEXT("RHIEndDrawingViewport"), [this, ViewportRHI, PresentArgs](FRHICommandListImmediate&)
	{
		FD3D11Viewport* Viewport = ResourceCast(ViewportRHI);

		SCOPE_CYCLE_COUNTER(STAT_D3D11PresentTime);
		
		// Not all platforms have control on the Present Counter, correlate the information FrameCounter -> PresentCounter before Presenting the frame
		RHIFrameInfoSetPresentCounter(PresentArgs.FrameCounter, GRHIPresentCounter);

		// Clear references the device might have to resources.
		CurrentDepthTexture = nullptr;
		CurrentDepthStencilTarget = nullptr;
		CurrentDSVAccessType = FExclusiveDepthStencil::DepthWrite_StencilWrite;
		CurrentRenderTargets[0] = nullptr;

		for (uint32 RenderTargetIndex = 1; RenderTargetIndex < D3D11_SIMULTANEOUS_RENDER_TARGET_COUNT; ++RenderTargetIndex)
		{
			CurrentRenderTargets[RenderTargetIndex] = nullptr;
		}

		ClearAllShaderResources();

		CommitRenderTargetsAndUAVs();

		StateCache.SetVertexShader(nullptr);

		uint16 NullStreamStrides[MaxVertexElementCount] = {0};
		StateCache.SetStreamStrides(NullStreamStrides);
		for (uint32 StreamIndex = 0; StreamIndex < MaxVertexElementCount; ++StreamIndex)
		{
			StateCache.SetStreamSource(nullptr, StreamIndex, 0, 0);
		}

		StateCache.SetIndexBuffer(nullptr, DXGI_FORMAT_R16_UINT, 0);
	
		CurrentResourceBoundAsIB = nullptr;
		FMemory::Memzero(CurrentResourcesBoundAsVBs, sizeof(CurrentResourcesBoundAsVBs));
		MaxBoundVertexBufferIndex = INDEX_NONE;
	
		StateCache.SetPixelShader(nullptr);
		StateCache.SetGeometryShader(nullptr);
		// Compute Shader is set to nullptr after each Dispatch call, so no need to clear it here

		bool bNativelyPresented = true;
		if (PresentArgs.bPresent)
		{
			bNativelyPresented = Viewport->Present(*this, PresentArgs.bLockToVsync);
		}

		if (bNativelyPresented)
		{ 
			static const auto CFinishFrameVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FinishCurrentFrame"));
			int32 FinishCurrentFrame = CFinishFrameVar->GetValueOnRenderThread();

			if (FinishCurrentFrame == 0)
			{
				// Wait for the GPU to finish rendering the previous frame before finishing this frame.
				Viewport->WaitForFrameEventCompletion();
				Viewport->IssueFrameEvent();
			}
			else if (FinishCurrentFrame > 0)
			{
				// Finish current frame immediately to reduce latency
				Viewport->IssueFrameEvent();
				Viewport->WaitForFrameEventCompletion();
			}
			else if (FinishCurrentFrame < 0)
			{
				//No frame wait
			}
		}
	});
}

FTextureRHIRef FD3D11DynamicRHI::RHIGetViewportBackBuffer(FRHIViewport* ViewportRHI)
{
	FD3D11Viewport* Viewport = ResourceCast(ViewportRHI);

	return Viewport->GetBackBuffer();
}

#if D3D11_WITH_DWMAPI
	#include "Windows/HideWindowsPlatformTypes.h"
#endif	//D3D11_WITH_DWMAPI
