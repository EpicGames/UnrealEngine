// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11RHIPrivateUtil.h: Private D3D RHI Utility definitions for Windows.
=============================================================================*/

#pragma once

#include "WindowsD3D11ConstantBuffer.h"
