// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WindowsD3D11ConstantBuffer.cpp: D3D Constant Buffer functions
=============================================================================*/

#include "WindowsD3D11ConstantBuffer.h"
#include "D3D11RHIPrivate.h"

void FWinD3D11ConstantBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRefCountPtr<ID3D11Buffer> CBuffer;

	D3D11_BUFFER_DESC BufferDesc{};

	// Verify constant buffer ByteWidth requirements
	constexpr uint32 MaxSize = GetMaxSize();
	check(MaxSize <= (D3D11_REQ_CONSTANT_BUFFER_ELEMENT_COUNT * 16) && (MaxSize % 16) == 0);
	BufferDesc.ByteWidth = MaxSize;

	BufferDesc.Usage = D3D11_USAGE_DEFAULT;
	BufferDesc.CPUAccessFlags = 0;
	BufferDesc.BindFlags = D3D11_BIND_CONSTANT_BUFFER;
	BufferDesc.MiscFlags = 0;

	CurrentSubBuffer = 0;

	while (BufferDesc.ByteWidth >= MIN_GLOBAL_CONSTANT_BUFFER_BYTE_SIZE)
	{
		FSubBuffer& SubBuffer = SubBuffers.Emplace_GetRef(BufferDesc.ByteWidth);

		CA_SUPPRESS(6385);	// Doesn't like COM
		VERIFYD3D11RESULT_EX(D3DRHI->GetDevice()->CreateBuffer(&BufferDesc, nullptr, SubBuffer.Buffer.GetInitReference()), D3DRHI->GetDevice());
		D3D11BufferStats::UpdateUniformBufferStats(SubBuffer.Buffer, SubBuffer.Size, true);

		BufferDesc.ByteWidth = Align(BufferDesc.ByteWidth / 2, 16);
	}

	FD3D11ConstantBuffer::InitRHI(RHICmdList);
}

void FWinD3D11ConstantBuffer::ReleaseRHI()
{
	FD3D11ConstantBuffer::ReleaseRHI();

	for (const FSubBuffer& SubBuffer : SubBuffers)
	{
		D3D11BufferStats::UpdateUniformBufferStats(SubBuffer.Buffer, SubBuffer.Size, false);
	}

	SubBuffers.Empty();
}

uint32 FWinD3D11ConstantBuffer::FindSubBufferForAllocationSize(uint64 InSize) const
{
	for (uint32 Index = SubBuffers.Num() - 1; Index > 0; Index--)
	{
		if (InSize <= SubBuffers[Index].Size)
		{
			return Index;
		}
	}
	return 0;
}

bool FWinD3D11ConstantBuffer::CommitConstantsToDevice( bool bDiscardSharedConstants )
{
	// New circular buffer system for faster constant uploads.  Avoids CopyResource and speeds things up considerably
	if (CurrentUpdateSize == 0)
	{
		return false;
	}

	SCOPE_CYCLE_COUNTER(STAT_D3D11GlobalConstantBufferUpdateTime);

	uint32	TotalUpdateSize = 0;

	if ( bDiscardSharedConstants )
	{
		// If we're discarding shared constants, just use constants that have been updated since the last Commit.
		TotalUpdateSize = CurrentUpdateSize;
	}
	else
	{
		// If we're re-using shared constants, use all constants.
		TotalUpdateSize = FMath::Max( CurrentUpdateSize, TotalUpdateSize );
	}

	// This basically keeps track dynamically how much data has been updated every frame
	// and then divides up a "max" constant buffer size by halves down until it finds a large sections that more tightly covers
	// the amount updated, assuming that all data in a constant buffer is updated each draw call and contiguous.
	CurrentSubBuffer = FindSubBufferForAllocationSize(TotalUpdateSize);

	check(IsAligned(ShadowData, 16));

	const uint32 BufferSize = SubBuffers[CurrentSubBuffer].Size;
	ID3D11Buffer* Buffer = SubBuffers[CurrentSubBuffer].Buffer.GetReference();

	D3DRHI->GetDeviceContext()->UpdateSubresource(Buffer, 0, nullptr, ShadowData, BufferSize, BufferSize);

	CurrentUpdateSize = 0;

	return true;
}

void FD3D11DynamicRHI::InitConstantBuffers()
{
	VSConstantBuffer = new FWinD3D11ConstantBuffer(this);
	PSConstantBuffer = new FWinD3D11ConstantBuffer(this);
	GSConstantBuffer = new FWinD3D11ConstantBuffer(this);
	CSConstantBuffer = new FWinD3D11ConstantBuffer(this);
}

DEFINE_STAT(STAT_D3D11GlobalConstantBufferUpdateTime);
