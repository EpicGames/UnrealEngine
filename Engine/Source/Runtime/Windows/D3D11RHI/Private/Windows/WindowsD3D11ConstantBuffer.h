// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WindowsD3D11ConstantBuffer.h: D3D Constant Buffer functions
=============================================================================*/

#pragma once

#include "D3D11ConstantBuffer.h"
#include "Windows/D3D11ThirdParty.h"

class FWinD3D11ConstantBuffer : public FD3D11ConstantBuffer
{
public:
	FWinD3D11ConstantBuffer(FD3D11DynamicRHI* InD3DRHI) :
		FD3D11ConstantBuffer(InD3DRHI)
	{
	}

	// FRenderResource interface.
	virtual void	InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void	ReleaseRHI() override;

	/**
	* Get the current pool buffer
	*/
	ID3D11Buffer* GetConstantBuffer() const
	{
		return SubBuffers[CurrentSubBuffer].Buffer.GetReference();
	}

	uint32 FindSubBufferForAllocationSize(uint64 InSize) const;

	/**
	* Unlocks the constant buffer so the data can be transmitted to the device
	*/
	bool CommitConstantsToDevice(bool bDiscardSharedConstants);

private:
	struct FSubBuffer
	{
		FSubBuffer(uint64 InSize) : Size(InSize) { }
		TRefCountPtr<ID3D11Buffer> Buffer;
		uint64 Size;
	};
	TArray<FSubBuffer> SubBuffers;
	uint32	CurrentSubBuffer = 0;
};
