// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WindowsD3D11Device.cpp: Windows D3D device RHI implementation.
=============================================================================*/
#include "D3D11RHIPrivate.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/EngineVersion.h"
#include "Windows/WindowsPlatformCrashContext.h"
#include "HardwareInfo.h"
#include "IHeadMountedDisplayModule.h"
#include "GenericPlatform/GenericPlatformDriver.h"			// FGPUDriverInfo
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "RenderingThread.h"
#include "RHIValidation.h"
#include "RHIUtilities.h"
#include "ShaderDiagnostics.h"
#include "HDRHelper.h"
#include "GlobalShader.h"
#include "Windows/WindowsD3D.h"

#include "Windows/AllowWindowsPlatformTypes.h"
	#if WITH_NVAPI
		#include "nvapi.h"
		#include "nvShaderExtnEnums.h"
	#endif
	#if WITH_AMD_AGS
		#include "amd_ags.h"
	#endif
#include "Windows/HideWindowsPlatformTypes.h"

FD3D11DynamicRHI*	GD3D11RHI = nullptr;

bool D3D11RHI_ShouldAllowAsyncResourceCreation();

#ifdef AMD_AGS_API
// Filled in during InitializeDeviceAgnosticVendorAPI if IsRHIDeviceAMD
struct AmdAgsInfo
{
	AGSContext* AmdAgsContext;
	AGSGPUInfo AmdGpuInfo;
};
static AmdAgsInfo AmdInfo;
#endif

TAutoConsoleVariable<int32> GD3D11DebugCvar(
	TEXT("r.D3D11.EnableD3DDebug"),
	0,
	TEXT("0 to disable d3ddebug layer (default)\n")
	TEXT("1 to enable error logging (-d3ddebug) \n")
	TEXT("2 to enable error & warning logging (-d3dlogwarnings)\n")
	TEXT("3 to enable breaking on errors & warnings (-d3dbreakonwarning)\n")
	TEXT("4 to enable CONTINUING on errors (-d3dcontinueonerrors)\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

bool D3D11_ShouldLogD3DDebugWarnings()
{
	return GD3D11DebugCvar.GetValueOnAnyThread() > 1;
}

bool D3D11_ShouldBreakOnD3DDebugErrors()
{
	return GD3D11DebugCvar.GetValueOnAnyThread() > 0 && GD3D11DebugCvar.GetValueOnAnyThread() != 4;
}

bool D3D11_ShouldBreakOnD3DDebugWarnings()
{
	return GD3D11DebugCvar.GetValueOnAnyThread() > 3;
}

static TAutoConsoleVariable<int32> CVarAMDUseMultiThreadedDevice(
	TEXT("r.AMDD3D11MultiThreadedDevice"),
	0,
	TEXT("If true, creates a multithreaded D3D11 device on AMD hardware (workaround for driver bug)\n")
	TEXT("Changes will only take effect in new game/editor instances - can't be changed at runtime.\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarAMDDisableAsyncTextureCreation(
	TEXT("r.AMDDisableAsyncTextureCreation"),
	0,
	TEXT("If true, uses synchronous texture creation on AMD hardware (workaround for driver bug)\n")
	TEXT("Changes will only take effect in new game/editor instances - can't be changed at runtime.\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarNVidiaTimestampWorkaround(
	TEXT("r.NVIDIATimestampWorkaround"),
	1,
	TEXT("If true we disable timestamps on pre-maxwell hardware (workaround for driver bug)\n"),
	ECVF_Default);

/**
 * Console variables used by the D3D11 RHI device.
 */
namespace RHIConsoleVariables
{
	int32 MaxFeatureSetLimit = -1;
	static FAutoConsoleVariableRef CVarMaxFeatureSetLimit(
		TEXT("RHI.FeatureSetLimit"),
		MaxFeatureSetLimit,
		TEXT("If set to 10, limit D3D RHI to D3D10 feature level. Otherwise, it will use default. Changing this at run-time has no effect. (default is -1)")
		);
};

static void FD3D11DumpLiveObjects()
{
	if (GRHIGlobals.IsDebugLayerEnabled)
	{
		TRefCountPtr<ID3D11Debug> DebugDevice = nullptr;
		VERIFYD3D11RESULT_EX(GD3D11RHI->GetDevice()->QueryInterface(__uuidof(ID3D11Debug), (void**)DebugDevice.GetInitReference()), GD3D11RHI->GetDevice());
		if (DebugDevice)
		{
			HRESULT HR = DebugDevice->ReportLiveDeviceObjects(D3D11_RLDO_SUMMARY|D3D11_RLDO_DETAIL);
			if (HR != S_OK)
			{
				UE_LOGF(LogD3D11RHI, Warning, "ReportLiveDeviceObjects failed with 0x%x", HR);
			}
		}		
	}
	else
	{
		UE_LOGF(LogD3D11RHI, Warning, "Must run with -d3ddebug to report live objects");
	}
}

FAutoConsoleCommand FD3DDumpLiveObjectsCommand
(
	TEXT("r.d3d11.dumpliveobjects"),
	TEXT("When using -d3ddebug will dump a list of live d3d objects.  Mostly for finding leaks."),
	FConsoleCommandDelegate::CreateStatic(&FD3D11DumpLiveObjects)
);

static bool CheckD3D11StoredMessages()
{
	bool bResult = false;

	TRefCountPtr<ID3D11Debug> DebugDevice = nullptr;
	VERIFYD3D11RESULT_EX(GD3D11RHI->GetDevice()->QueryInterface(__uuidof(ID3D11Debug), (void**)DebugDevice.GetInitReference()), GD3D11RHI->GetDevice());
	if (DebugDevice)
	{
		TRefCountPtr<ID3D11InfoQueue> d3dInfoQueue;
		if (SUCCEEDED(GD3D11RHI->GetDevice()->QueryInterface(__uuidof(ID3D11InfoQueue), (void**)d3dInfoQueue.GetInitReference())))
		{
			D3D11_MESSAGE* d3dMessage = nullptr;
			SIZE_T AllocateSize = 0;

			static bool bBreakOnWarning = FParse::Param(FCommandLine::Get(), TEXT("d3dbreakonwarning"));

			int StoredMessageCount = d3dInfoQueue->GetNumStoredMessagesAllowedByRetrievalFilter();
			for (int MessageIndex = 0; MessageIndex < StoredMessageCount; MessageIndex++)
			{
				SIZE_T MessageLength = 0;
				HRESULT hr = d3dInfoQueue->GetMessage(MessageIndex, nullptr, &MessageLength);

				// realloc the message
				if (MessageLength > AllocateSize)
				{
					if (d3dMessage)
					{
						FMemory::Free(d3dMessage);
						d3dMessage = nullptr;
						AllocateSize = 0;
					}

					d3dMessage = (D3D11_MESSAGE*)FMemory::Malloc(MessageLength);
					AllocateSize = MessageLength;
				}

				if (d3dMessage)
				{
					// get the actual message data from the queue
					hr = d3dInfoQueue->GetMessage(MessageIndex, d3dMessage, &MessageLength);

					switch (d3dMessage->Severity)
					{
					case D3D11_MESSAGE_SEVERITY_CORRUPTION:
					case D3D11_MESSAGE_SEVERITY_ERROR:
						{
							UE_LOGF(LogD3D11RHI, Error, "[D3DDebug] %ls", ANSI_TO_TCHAR(d3dMessage->pDescription));
							bResult = true;
							break;
						}
					case D3D11_MESSAGE_SEVERITY_WARNING:
						{
							UE_LOGF(LogD3D11RHI, Warning, "[D3DDebug] %ls", ANSI_TO_TCHAR(d3dMessage->pDescription));
							if (bBreakOnWarning)
							{
								bResult = true;
							}
							break;
						}
					default:
						{
							UE_LOGF(LogD3D11RHI, Log, "[D3DDebug] %ls", ANSI_TO_TCHAR(d3dMessage->pDescription));
						}
					}
				}
			}
			d3dInfoQueue->ClearStoredMessages();
			if (AllocateSize > 0)
			{
				FMemory::Free(d3dMessage);
			}
		}
	}

	return bResult;
}

static LONG __stdcall D3D11VectoredExceptionHandler(EXCEPTION_POINTERS* InInfo)
{
	// Only handle D3D error codes here
	if (InInfo->ExceptionRecord->ExceptionCode == _FACDXGI)
	{
		if (CheckD3D11StoredMessages())
		{
			if (FPlatformMisc::IsDebuggerPresent())
			{
				// when we get here, then it means that BreakOnSeverity was set for this error message, so request the debug break here as well
				// when the debugger is attached
				UE_DEBUG_BREAK();
			}
		}

		// Handles the exception
		return EXCEPTION_CONTINUE_EXECUTION;
	}

	// continue searching
	return EXCEPTION_CONTINUE_SEARCH;
}

static bool bIsQuadBufferStereoEnabled = false;

static void SafeCreateDXGIFactory(IDXGIFactory1** DXGIFactory1, bool bWithDebug)
{
	bIsQuadBufferStereoEnabled = FParse::Param(FCommandLine::Get(), TEXT("quad_buffer_stereo"));

	const uint32 Flags = bWithDebug ? DXGI_CREATE_FACTORY_DEBUG : 0;
	verify(SUCCEEDED(CreateDXGIFactory2(Flags, IID_PPV_ARGS(DXGIFactory1))));
}

/**
 * Returns the lowest D3D feature level we are allowed to created based on
 * command line parameters.
 */
static D3D_FEATURE_LEVEL GetMinAllowedD3DFeatureLevel()
{
	return D3D_FEATURE_LEVEL_11_0;
}

/**
 * Returns the highest D3D feature level we are allowed to created based on
 * command line parameters.
 */
static D3D_FEATURE_LEVEL GetMaxAllowedD3DFeatureLevel()
{
	return D3D_FEATURE_LEVEL_11_1;
}

inline bool IsDeviceUMA(ID3D11Device* device)
{
	ID3D11Device3* pDevice3 = nullptr;
	if (SUCCEEDED(device->QueryInterface(__uuidof(ID3D11Device3), (void**)&pDevice3)))
	{
		D3D11_FEATURE_DATA_D3D11_OPTIONS2 FeatureData = {};
		if (SUCCEEDED(pDevice3->CheckFeatureSupport(D3D11_FEATURE_D3D11_OPTIONS2, &FeatureData, sizeof(FeatureData))))
		{
			pDevice3->Release();
			return (bool)(FeatureData.UnifiedMemoryArchitecture);
		}
		pDevice3->Release();
	}
	return false;
}

/**
 * Attempts to create a D3D11 device for the adapter using at most MaxFeatureLevel or find an existing one.
 * If creation is successful, true is returned.
 */
static bool SafeTestD3D11CreateDevice(IDXGIAdapter* Adapter, const DXGI_ADAPTER_DESC& AdapterDesc, D3D_FEATURE_LEVEL MinFeatureLevel, D3D_FEATURE_LEVEL MaxFeatureLevel, FD3D11DeviceBasicInfo& OutInfo)
{
	const uint64 Luid = BitCast<uint64>(AdapterDesc.AdapterLuid);
	int32 CachedFeatureLevel;
	ID3D11Device* D3DDevice = FWindowsD3D::FindCachedD3D11AdapterDevice(Luid, CachedFeatureLevel);
	if (D3DDevice)
	{
		OutInfo.MaxFeatureLevel = static_cast<D3D_FEATURE_LEVEL>(CachedFeatureLevel);
	}
	else
	{
		uint32 DeviceFlags = D3D11_CREATE_DEVICE_SINGLETHREADED;
		// Use a debug device if specified on the command line.
		if(GRHIGlobals.IsDebugLayerEnabled)
		{
			DeviceFlags |= D3D11_CREATE_DEVICE_DEBUG;
		}

		// @MIXEDREALITY_CHANGE : BEGIN - Add BGRA flag for Windows Mixed Reality HMD's
		DeviceFlags |= D3D11_CREATE_DEVICE_BGRA_SUPPORT;
		// @MIXEDREALITY_CHANGE : END

		D3D_FEATURE_LEVEL RequestedFeatureLevels[] =
		{
			D3D_FEATURE_LEVEL_11_1,
			D3D_FEATURE_LEVEL_11_0,
		};
	
		// Trim to allowed feature levels
		int32 FirstAllowedFeatureLevel = 0;
		int32 NumAllowedFeatureLevels = UE_ARRAY_COUNT(RequestedFeatureLevels);
		int32 LastAllowedFeatureLevel = NumAllowedFeatureLevels - 1;
	
		while (FirstAllowedFeatureLevel < NumAllowedFeatureLevels)
		{
			if (RequestedFeatureLevels[FirstAllowedFeatureLevel] == MaxFeatureLevel)
			{
				break;
			}
			FirstAllowedFeatureLevel++;
		}

		while (LastAllowedFeatureLevel > 0)
		{
			if (RequestedFeatureLevels[LastAllowedFeatureLevel] >= MinFeatureLevel)
			{
				break;
			}
			LastAllowedFeatureLevel--;
		}
	
		NumAllowedFeatureLevels = LastAllowedFeatureLevel - FirstAllowedFeatureLevel + 1;
		if (MaxFeatureLevel < MinFeatureLevel || NumAllowedFeatureLevels <= 0)
		{
			return false;
		}

		// We don't want software renderer. Ideally we specify D3D_DRIVER_TYPE_HARDWARE on creation but
		// when we specify an adapter we need to specify D3D_DRIVER_TYPE_UNKNOWN (otherwise the call fails).
		// We cannot check the device type later (seems this is missing functionality in D3D).
		D3D_FEATURE_LEVEL DeviceFeatureLevel = (D3D_FEATURE_LEVEL)0;
		TRefCountPtr<ID3D11Device> DevicePtr;
		HRESULT Result = D3D11CreateDevice(
			Adapter,
			D3D_DRIVER_TYPE_UNKNOWN,
			nullptr,
			DeviceFlags,
			&RequestedFeatureLevels[FirstAllowedFeatureLevel],
			NumAllowedFeatureLevels,
			D3D11_SDK_VERSION,
			DevicePtr.GetInitReference(),
			&DeviceFeatureLevel,
			nullptr);

		if (FAILED(Result))
		{
			// Log any reason for failure to create test device. Extra debug help.
			VERIFYD3D11RESULT_NOEXIT(Result);

			// Fatal error on 0x887A002D
			if (DXGI_ERROR_SDK_COMPONENT_MISSING == Result)
			{
				UE_LOGF(LogD3D11RHI, Fatal, "-d3ddebug was used but optional Graphics Tools were not found. Install them through the Manage Optional Features in windows. See: https://docs.microsoft.com/en-us/windows/uwp/gaming/use-the-directx-runtime-and-visual-studio-graphics-diagnostic-features");
			}

			return false;
		}

		D3DDevice = DevicePtr.GetReference();
		FWindowsD3D::CacheD3D11AdapterDevice(Luid, MoveTemp(DevicePtr), DeviceFeatureLevel);
		OutInfo.MaxFeatureLevel = DeviceFeatureLevel;
	}

	OutInfo.bUMA = IsDeviceUMA(D3DDevice);
	return true;
}

struct DisplayChromacities
{
	float RedX, RedY;
	float GreenX, GreenY;
	float BlueX, BlueY;
	float WpX, WpY;
};

const DisplayChromacities DisplayChromacityList[] =
{
	{ 0.64000f, 0.33000f, 0.30000f, 0.60000f, 0.15000f, 0.06000f, 0.31270f, 0.32900f }, // EDisplayColorGamut::sRGB_D65
	{ 0.68000f, 0.32000f, 0.26500f, 0.69000f, 0.15000f, 0.06000f, 0.31270f, 0.32900f }, // EDisplayColorGamut::DCIP3_D65
	{ 0.70800f, 0.29200f, 0.17000f, 0.79700f, 0.13100f, 0.04600f, 0.31270f, 0.32900f }, // EDisplayColorGamut::Rec2020_D65
	{ 0.73470f, 0.26530f, 0.00000f, 1.00000f, 0.00010f,-0.07700f, 0.32168f, 0.33767f }, // EDisplayColorGamut::ACES_D60
	{ 0.71300f, 0.29300f, 0.16500f, 0.83000f, 0.12800f, 0.04400f, 0.32168f, 0.33767f }, // EDisplayColorGamut::ACEScg_D60
};

static UE::Color::FColorSpace MakeColorSpaceFromDXGI(const DXGI_OUTPUT_DESC1& OutputDesc)
{
	return UE::Color::FColorSpace(
		FVector2d(OutputDesc.RedPrimary[0], OutputDesc.RedPrimary[1]),
		FVector2d(OutputDesc.GreenPrimary[0], OutputDesc.GreenPrimary[1]),
		FVector2d(OutputDesc.BluePrimary[0], OutputDesc.BluePrimary[1]),
		FVector2d(OutputDesc.WhitePoint[0], OutputDesc.WhitePoint[1]));
}

bool FD3D11DynamicRHI::SetupDisplayHDRMetaData()
{
	check(GetDevice());

	TRefCountPtr<IDXGIAdapter> DXGIAdapter = Adapter.DXGIAdapter;

	if (DXGIFactory1.IsValid() && !DXGIFactory1->IsCurrent())
	{
		if (!DXGIFactoryForDisplayList.IsValid() || !DXGIFactoryForDisplayList->IsCurrent())
		{
			SafeCreateDXGIFactory(DXGIFactoryForDisplayList.GetInitReference(), GRHIGlobals.IsDebugLayerEnabled);
		}

		if (DXGIFactoryForDisplayList.IsValid() && DXGIFactoryForDisplayList->IsCurrent())
		{
			TRefCountPtr<IDXGIAdapter> TempAdapter;
			bool bMatchingAdapterFound = false;
			for (uint32 AdapterIndex = 0; DXGIFactoryForDisplayList->EnumAdapters(AdapterIndex, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
			{
				DXGI_ADAPTER_DESC AdapterDesc;
				if (!TempAdapter.IsValid() || FAILED(TempAdapter->GetDesc(&AdapterDesc)))
				{
					continue;
				}

				if (AdapterDesc.AdapterLuid.LowPart != Adapter.DXGIAdapterDesc.AdapterLuid.LowPart)
				{
					continue;
				}

				if (AdapterDesc.AdapterLuid.HighPart != Adapter.DXGIAdapterDesc.AdapterLuid.HighPart)
				{
					continue;
				}

				bMatchingAdapterFound = true;
				break;
			}

			if (bMatchingAdapterFound && ensure(TempAdapter.IsValid()))
			{
				DXGIAdapter = TempAdapter;
			}
		}
	}

	DisplayList.Empty();

	// Determines if any displays support HDR
	bool bSupportsHDROutput = false;
	{
		if (!DXGIAdapter)
		{
			return false;
		}

		for (uint32 DisplayIndex = 0; true; ++DisplayIndex)
		{
			TRefCountPtr<IDXGIOutput> DXGIOutput;
			if (S_OK != DXGIAdapter->EnumOutputs(DisplayIndex, DXGIOutput.GetInitReference()))
			{
				break;
			}

			TRefCountPtr<IDXGIOutput6> Output6;
			if (SUCCEEDED(DXGIOutput->QueryInterface(IID_PPV_ARGS(Output6.GetInitReference()))))
			{
				DXGI_OUTPUT_DESC1 OutputDesc;
				VERIFYD3D11RESULT(Output6->GetDesc1(&OutputDesc));

				// Check for HDR support on the display.
				const bool bDisplaySupportsHDROutput = (OutputDesc.ColorSpace == DXGI_COLOR_SPACE_RGB_FULL_G2084_NONE_P2020);
				if (bDisplaySupportsHDROutput)
				{
					UE_LOGF(LogD3D11RHI, Log, "HDR output is supported on adapter %i, display %u:", 0, DisplayIndex);
					UE_LOGF(LogD3D11RHI, Log, "\t\tMinLuminance = %f", OutputDesc.MinLuminance);
					UE_LOGF(LogD3D11RHI, Log, "\t\tMaxLuminance = %f", OutputDesc.MaxLuminance);
					UE_LOGF(LogD3D11RHI, Log, "\t\tMaxFullFrameLuminance = %f", OutputDesc.MaxFullFrameLuminance);

					bSupportsHDROutput = true;
				}

				FDisplayInformation& DisplayInformation = DisplayList.Emplace_GetRef();
				DisplayInformation.bHDRSupported = bDisplaySupportsHDROutput;
				const RECT& DisplayCoords = OutputDesc.DesktopCoordinates;
				DisplayInformation.DesktopCoordinates = FIntRect(DisplayCoords.left, DisplayCoords.top, DisplayCoords.right, DisplayCoords.bottom);
				DisplayInformation.MinimumLuminanceInNits = OutputDesc.MinLuminance;
				DisplayInformation.MaximumLuminanceInNits = OutputDesc.MaxLuminance ?
					OutputDesc.MaxLuminance :
					(bDisplaySupportsHDROutput ? 1000.f : 100.f);
				DisplayInformation.MaximumFullFrameLuminanceInNits = OutputDesc.MaxFullFrameLuminance ?
					OutputDesc.MaxFullFrameLuminance :
					(bDisplaySupportsHDROutput ? 600.f : 100.f);
				const bool bIsGamutValid = OutputDesc.RedPrimary[0] && OutputDesc.RedPrimary[1] &&
					OutputDesc.GreenPrimary[0] && OutputDesc.GreenPrimary[1] &&
					OutputDesc.BluePrimary[0] && OutputDesc.BluePrimary[1] &&
					OutputDesc.WhitePoint[0] && OutputDesc.WhitePoint[1];
				DisplayInformation.LimitingColorSpace = bIsGamutValid ?
					MakeColorSpaceFromDXGI(OutputDesc) :
					(bDisplaySupportsHDROutput ?
						UE::Color::FColorSpace::GetP3D65() :
						UE::Color::FColorSpace::GetSRGB());
			}
		}
	}

	return bSupportsHDROutput;
}

extern void HDRSettingChangedSinkCallback();
void FD3D11DynamicRHI::RHIHandleDisplayChange()
{
	RHIBlockUntilGPUIdle();
	GRHISupportsHDROutput = SetupDisplayHDRMetaData();
	// make sure CVars are being updated properly
	HDRSettingChangedSinkCallback();
}

static bool IsDeviceOverclocked()
{
	if (IsRHIDeviceNVIDIA())
	{
#ifdef NVAPI_INTERFACE
		NvAPI_Status Status;
		NvU32 NumGPUs;
		NvPhysicalGpuHandle GPUHandles[NVAPI_MAX_PHYSICAL_GPUS];

		// Fetch GPU NV handles
		Status = NvAPI_EnumPhysicalGPUs(GPUHandles, &NumGPUs);

		if (Status != NVAPI_OK)
		{
			return false;
		}

		for (NvU32 GPU = 0; GPU < NumGPUs; ++GPU)
		{
			const NvPhysicalGpuHandle& GPUHandle = GPUHandles[GPU];

			NV_GPU_PERF_PSTATES20_INFO PS20Info = {0};
			PS20Info.version = NV_GPU_PERF_PSTATES20_INFO_VER;

			Status = NvAPI_GPU_GetPstates20(GPUHandle, &PS20Info);

			// Some GPUs use an older struct layout
			if (Status == NVAPI_INCOMPATIBLE_STRUCT_VERSION)
			{
				PS20Info.version = NV_GPU_PERF_PSTATES20_INFO_VER1;
				Status = NvAPI_GPU_GetPstates20(GPUHandle, &PS20Info);
			}

			// Assume non-overclocked on failure
			if (Status != NVAPI_OK)
			{
				return false;
			}

			// Where state was marked editable, check for relevant modifications
			if (PS20Info.bIsEditable == 0)
			{
				continue;
			}

			NvU32 PState, Clock, Voltage;
			for (PState = 0; PState < PS20Info.numPstates; PState++)
			{
				if (PS20Info.pstates[PState].bIsEditable != 0)
				{
					// Are clocks user modified?
					for (Clock = 0; Clock < PS20Info.numClocks; Clock++)
					{
						if (PS20Info.pstates[PState].clocks[Clock].bIsEditable != 0 && PS20Info.pstates[PState].clocks[Clock].freqDelta_kHz.value != 0)
						{
							return true;
						}
					}

					// Are voltages user modified?
					for (Voltage = 0; Voltage < PS20Info.numBaseVoltages; Voltage++)
					{
						if (PS20Info.pstates[PState].baseVoltages[Voltage].bIsEditable != 0 && PS20Info.pstates[PState].baseVoltages[Voltage].voltDelta_uV.value != 0)
						{
							return true;
						}
					}
				}
			}
		}
#endif // NVAPI_INTERFACE
	}

	// Assume non-overclocked by default
	return false;
}

bool FD3D11DynamicRHIModule::IsSupported()
{
	// if not computed yet
	if(!ChosenAdapter.IsValid())
	{
		FindAdapter();
	}

	// The hardware must support at least 11_0.
	return ChosenAdapter.IsValid()
		&& ChosenAdapter.MaxSupportedFeatureLevel >= D3D_FEATURE_LEVEL_11_0;
}

bool FD3D11DynamicRHIModule::IsSupported(ERHIFeatureLevel::Type RequestedFeatureLevel)
{
	return RequestedFeatureLevel <= ERHIFeatureLevel::SM5 && IsSupported();
}

const TCHAR* GetFeatureLevelString(D3D_FEATURE_LEVEL FeatureLevel)
{
	switch(FeatureLevel)
	{
		case D3D_FEATURE_LEVEL_9_1:		return TEXT("9_1");
		case D3D_FEATURE_LEVEL_9_2:		return TEXT("9_2");
		case D3D_FEATURE_LEVEL_9_3:		return TEXT("9_3");
		case D3D_FEATURE_LEVEL_10_0:	return TEXT("10_0");
		case D3D_FEATURE_LEVEL_10_1:	return TEXT("10_1");
		case D3D_FEATURE_LEVEL_11_0:	return TEXT("11_0");
		case D3D_FEATURE_LEVEL_11_1:	return TEXT("11_1");
	}
	return TEXT("X_X");
}

static uint32 CountAdapterOutputs(TRefCountPtr<IDXGIAdapter>& Adapter)
{
	uint32 OutputCount = 0;
	for(;;)
	{
		TRefCountPtr<IDXGIOutput> Output;
		HRESULT hr = Adapter->EnumOutputs(OutputCount, Output.GetInitReference());
		if(FAILED(hr))
		{
			break;
		}
		++OutputCount;
	}

	return OutputCount;
}

static void LogDXGIAdapterDesc(const DXGI_ADAPTER_DESC& AdapterDesc)
{
	UE_LOGF(LogD3D11RHI, Log, "    Description : %ls", AdapterDesc.Description);
	UE_LOGF(LogD3D11RHI, Log, "    VendorId    : %04x", AdapterDesc.VendorId);
	UE_LOGF(LogD3D11RHI, Log, "    DeviceId    : %04x", AdapterDesc.DeviceId);
	UE_LOGF(LogD3D11RHI, Log, "    SubSysId    : %04x", AdapterDesc.SubSysId);
	UE_LOGF(LogD3D11RHI, Log, "    Revision    : %04x", AdapterDesc.Revision);
	UE_LOGF(LogD3D11RHI, Log, "    DedicatedVideoMemory : %zu bytes", AdapterDesc.DedicatedVideoMemory);
	UE_LOGF(LogD3D11RHI, Log, "    DedicatedSystemMemory : %zu bytes", AdapterDesc.DedicatedSystemMemory);
	UE_LOGF(LogD3D11RHI, Log, "    SharedSystemMemory : %zu bytes", AdapterDesc.SharedSystemMemory);
	UE_LOGF(LogD3D11RHI, Log, "    AdapterLuid : %lu %lu", AdapterDesc.AdapterLuid.HighPart, AdapterDesc.AdapterLuid.LowPart);
}

void FD3D11DynamicRHIModule::FindAdapter()
{
	// Once we chosen one we don't need to do it again.
	check(!ChosenAdapter.IsValid());

	// Try to create the DXGIFactory1.  This will fail if we're not running Vista SP2 or higher.
	TRefCountPtr<IDXGIFactory1> DXGIFactory1;
	SafeCreateDXGIFactory(DXGIFactory1.GetInitReference(), GRHIGlobals.IsDebugLayerEnabled);
	if(!DXGIFactory1)
	{
		return;
	}

	TRefCountPtr<IDXGIFactory6> DXGIFactory6;
	DXGIFactory1->QueryInterface(IID_PPV_ARGS(DXGIFactory6.GetInitReference()));

	// Allow HMD to override which graphics adapter is chosen, so we pick the adapter where the HMD is connected
	const uint64 HmdGraphicsAdapterLuid = IHeadMountedDisplayModule::IsAvailable() ? IHeadMountedDisplayModule::Get().GetGraphicsAdapterLuid() : 0;
	const uint64 AdapterLuid = FWindowsD3D::ChooseD3D11Adapter(HmdGraphicsAdapterLuid);

	TRefCountPtr<IDXGIAdapter> TempAdapter;
	D3D_FEATURE_LEVEL MinAllowedFeatureLevel = GetMinAllowedD3DFeatureLevel();
	D3D_FEATURE_LEVEL MaxAllowedFeatureLevel = GetMaxAllowedD3DFeatureLevel();

	UE_LOGF(LogD3D11RHI, Log, "D3D11 min allowed feature level: %ls", GetFeatureLevelString(MinAllowedFeatureLevel));
	UE_LOGF(LogD3D11RHI, Log, "D3D11 max allowed feature level: %ls", GetFeatureLevelString(MaxAllowedFeatureLevel));

	UE_LOGF(LogD3D11RHI, Log, "D3D11 adapters:");

	int GpuPreferenceInt = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE;
	FParse::Value(FCommandLine::Get(), TEXT("-gpupreference="), GpuPreferenceInt);
	DXGI_GPU_PREFERENCE GpuPreference;
	switch(GpuPreferenceInt)
	{
	case 1: GpuPreference = DXGI_GPU_PREFERENCE_MINIMUM_POWER; break;
	case 2: GpuPreference = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE; break;
	default: GpuPreference = DXGI_GPU_PREFERENCE_UNSPECIFIED; break;
	}

	auto LocalEnumAdapters = [&DXGIFactory6, &DXGIFactory1, GpuPreference](UINT AdapterIndex, IDXGIAdapter** Adapter) -> HRESULT
	{
		if(!DXGIFactory6 || GpuPreference == DXGI_GPU_PREFERENCE_UNSPECIFIED)
		{
			return DXGIFactory1->EnumAdapters(AdapterIndex, Adapter);
		}
		else
		{
			return DXGIFactory6->EnumAdapterByGpuPreference(AdapterIndex, GpuPreference, __uuidof(IDXGIAdapter), (void**)Adapter);
		}
	};

	// Enumerate the DXGIFactory's adapters.
	for(uint32 AdapterIndex = 0; LocalEnumAdapters(AdapterIndex,TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
	{
		// Check that if adapter supports D3D11.
		if(TempAdapter)
		{
			UE_LOGF(LogD3D11RHI, Log, "Testing D3D11 Adapter %u:", AdapterIndex);
			DXGI_ADAPTER_DESC AdapterDesc;
			if (HRESULT DescResult = TempAdapter->GetDesc(&AdapterDesc); SUCCEEDED(DescResult))
			{
				LogDXGIAdapterDesc(AdapterDesc);

				FD3D11DeviceBasicInfo DeviceInfo;
				if(SafeTestD3D11CreateDevice(TempAdapter, AdapterDesc, MinAllowedFeatureLevel, MaxAllowedFeatureLevel, DeviceInfo))
				{
					// Log some information about the available D3D11 adapters.

					uint32 OutputCount = CountAdapterOutputs(TempAdapter);

					UE_LOGF(LogD3D11RHI, Log,
						"  %2u. '%ls' (Feature Level %ls)",
						AdapterIndex,
						AdapterDesc.Description,
						GetFeatureLevelString(DeviceInfo.MaxFeatureLevel)
						);
					UE_LOGF(LogD3D11RHI, Log,
						"      %u/%u/%u MB DedicatedVideo/DedicatedSystem/SharedSystem, Outputs:%d, VendorId:0x%x UMA:%ls",
						(uint32)(AdapterDesc.DedicatedVideoMemory / (1024*1024)),
						(uint32)(AdapterDesc.DedicatedSystemMemory / (1024*1024)),
						(uint32)(AdapterDesc.SharedSystemMemory / (1024*1024)),
						OutputCount,
						AdapterDesc.VendorId,
						DeviceInfo.bUMA ? TEXT("true") : TEXT("false")
						);

					const bool bIsWARP = RHIConvertToGpuVendorId(AdapterDesc.VendorId) == EGpuVendorId::Microsoft;

					if (!FMemory::Memcmp(&AdapterLuid, &AdapterDesc.AdapterLuid, sizeof(LUID)))
					{
						ChosenAdapter = FD3D11Adapter(TempAdapter, bIsWARP, DeviceInfo);
						// Don't break here so we can log all adapters.
					}
				}
			}
			else
			{
				UE_LOGF(LogD3D11RHI, Warning, "Failed to get description for adapter %u.", AdapterIndex);
			}
		}
	}

	if(ChosenAdapter.IsValid())
	{
		UE_LOGF(LogD3D11RHI, Log, "Chosen D3D11 Adapter:");
		LogDXGIAdapterDesc(ChosenAdapter.DXGIAdapterDesc);
	}
	else
	{
		UE_LOGF(LogD3D11RHI, Error, "Failed to choose a D3D11 Adapter.");
	}

	GRHIAdapterName = ChosenAdapter.DXGIAdapterDesc.Description;
	GRHIVendorId = ChosenAdapter.DXGIAdapterDesc.VendorId;
	GRHIDeviceId = ChosenAdapter.DXGIAdapterDesc.DeviceId;
	GRHIDeviceRevision = ChosenAdapter.DXGIAdapterDesc.Revision;
	GRHIGlobals.GpuInfo.DedicatedVideoMemory = ChosenAdapter.DXGIAdapterDesc.DedicatedVideoMemory;
	GRHIDeviceIsIntegrated = ChosenAdapter.bUMA;
	GRHIDeviceIsUMASeparateVA = ChosenAdapter.bUMA;

	UE_LOGF(LogD3D11RHI, Log, "Integrated GPU (iGPU): %ls", GRHIDeviceIsIntegrated ? TEXT("true") : TEXT("false"));
}

FDynamicRHI* FD3D11DynamicRHIModule::CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel)
{
	IDXGIFactory1* DXGIFactory1;
	VERIFYD3D11RESULT(ChosenAdapter.DXGIAdapter->GetParent(__uuidof(DXGIFactory1), reinterpret_cast<void**>(&DXGIFactory1)));

	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_PCD3D_ES3_1;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_PCD3D_SM5;

	ERHIFeatureLevel::Type PreviewFeatureLevel;
	if (RHIGetPreviewFeatureLevel(PreviewFeatureLevel))
	{
		// ES3.1 feature level emulation in D3D11
		GMaxRHIFeatureLevel = PreviewFeatureLevel;
	}
	else
	{
		GMaxRHIFeatureLevel = ERHIFeatureLevel::SM5;
		if (RequestedFeatureLevel < ERHIFeatureLevel::SM6)
		{
			GMaxRHIFeatureLevel = RequestedFeatureLevel;
		}
	}

	if (!ensure(GMaxRHIFeatureLevel < ERHIFeatureLevel::SM6))
	{
		GMaxRHIFeatureLevel = ERHIFeatureLevel::SM5;
	}
	GMaxRHIShaderPlatform = GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel];
	check(GMaxRHIShaderPlatform != SP_NumPlatforms);

	GD3D11RHI = new FD3D11DynamicRHI(DXGIFactory1, ChosenAdapter.MaxSupportedFeatureLevel,ChosenAdapter);
	FDynamicRHI* FinalRHI = GD3D11RHI;

#if ENABLE_RHI_VALIDATION
	if (FParse::Param(FCommandLine::Get(), TEXT("RHIValidation")))
	{
		FinalRHI = new FValidationRHI(FinalRHI);
	}
#endif

	FGenericCrashContext::SetEngineData(TEXT("RHI.IntegratedGPU"), ChosenAdapter.bUMA ? TEXT("true") : TEXT("false"));

	return FinalRHI;
}

void FD3D11DynamicRHI::Init()
{
	InitD3DDevice();
}

bool FD3D11DynamicRHI::IsQuadBufferStereoEnabled()
{
	return bIsQuadBufferStereoEnabled;
}

void FD3D11DynamicRHI::DisableQuadBufferStereo()
{
	bIsQuadBufferStereoEnabled = false;
}

void FD3D11DynamicRHI::FlushPendingLogs()
{
#if !(UE_BUILD_SHIPPING && WITH_EDITOR)

	if (GRHIGlobals.IsDebugLayerEnabled)
	{
		TRefCountPtr<ID3D11InfoQueue> InfoQueue = nullptr;
		VERIFYD3D11RESULT_EX(Direct3DDevice->QueryInterface(IID_ID3D11InfoQueue, (void**)InfoQueue.GetInitReference()), Direct3DDevice);
		if (InfoQueue)
		{
			FString FullMessage;
			uint64 NumMessages = InfoQueue->GetNumStoredMessagesAllowedByRetrievalFilter();
			for (uint64 Index = 0; Index < NumMessages; ++Index)
			{
				SIZE_T Length = 0;
				if (SUCCEEDED(InfoQueue->GetMessage(Index, nullptr, &Length)))
				{
					TArray<uint8> Bytes;
					Bytes.AddUninitialized((int32)Length);
					D3D11_MESSAGE* Message = (D3D11_MESSAGE*)Bytes.GetData();
					if (SUCCEEDED(InfoQueue->GetMessage(Index, Message, &Length)))
					{
						FullMessage += TEXT("\n\t");
						FullMessage += Message->pDescription;
					}
				}
			}

			if (FullMessage.Len() > 0)
			{
				UE_LOGF(LogD3D11RHI, Warning, "d3debug warnings/errors found:%ls", *FullMessage);
			}
			InfoQueue->ClearStoredMessages();
		}
	}
#endif
}

#if INTEL_EXTENSIONS
void FD3D11DynamicRHI::StartIntelExtensions()
{
	if (!UE::RHICore::AllowVendorDevice())
	{
		return;
	}

	const INTCExtensionVersion AtomicsRequiredVersion = { 3, 4, 1 }; // version 3.4.1
	const INTCExtensionVersion UAVOverlapRequiredVersion = { 1, 1, 0 };

	INTCExtensionVersion* SupportedExtensionsVersions = nullptr;
	uint32_t SupportedExtensionsVersionCount = 0;
	INTCExtensionInfo INTCExtensionInfo = {};

	if (FAILED(INTC_LoadExtensionsLibrary(false)))
	{
		UE_LOGF(LogD3D11RHI, Log, "Failed to load Intel Extensions Library");
	}

	if (SUCCEEDED(INTC_D3D11_GetSupportedVersions(Direct3DDevice, nullptr, &SupportedExtensionsVersionCount)))
	{
		SupportedExtensionsVersions = new INTCExtensionVersion[SupportedExtensionsVersionCount]{};
	}

	// Workaround for C6385, if we pass in SupportedExtensionsVersionCount again, the static analyzer thinks it may be different from the first call
	uint32_t DummyCount = SupportedExtensionsVersionCount;
	if (SUCCEEDED(INTC_D3D11_GetSupportedVersions(Direct3DDevice, SupportedExtensionsVersions, &DummyCount)) && SupportedExtensionsVersions != nullptr)
	{
		check(SupportedExtensionsVersionCount == DummyCount);
		for (uint32_t i = 0; i < SupportedExtensionsVersionCount; i++)
		{
			if ((SupportedExtensionsVersions[i].HWFeatureLevel >= AtomicsRequiredVersion.HWFeatureLevel) &&
				(SupportedExtensionsVersions[i].APIVersion >= AtomicsRequiredVersion.APIVersion) &&
				(SupportedExtensionsVersions[i].Revision >= AtomicsRequiredVersion.Revision) &&
				!GRHISupportsAtomicUInt64)
			{
				UE_LOGF(LogD3D11RHI, Log, "Intel Extensions loaded requested version Atomics Version: %u.%u.%u",
					SupportedExtensionsVersions[i].HWFeatureLevel,
					SupportedExtensionsVersions[i].APIVersion,
					SupportedExtensionsVersions[i].Revision);

				INTCExtensionInfo.RequestedExtensionVersion = SupportedExtensionsVersions[i];
				GRHISupportsAtomicUInt64 = true;
			}

			if ((SupportedExtensionsVersions[i].HWFeatureLevel >= UAVOverlapRequiredVersion.HWFeatureLevel) &&
				(SupportedExtensionsVersions[i].APIVersion >= UAVOverlapRequiredVersion.APIVersion) &&
				(SupportedExtensionsVersions[i].Revision >= UAVOverlapRequiredVersion.Revision) &&
				!bIntelSupportsUAVOverlap)
			{
				UE_LOGF(LogD3D11RHI, Log, "Intel Extensions loaded requested version for UAVOverlap: %u.%u.%u",
					SupportedExtensionsVersions[i].HWFeatureLevel,
					SupportedExtensionsVersions[i].APIVersion,
					SupportedExtensionsVersions[i].Revision);

				if (SupportedExtensionsVersions[i].HWFeatureLevel >= INTCExtensionInfo.RequestedExtensionVersion.HWFeatureLevel &&
					SupportedExtensionsVersions[i].APIVersion >= INTCExtensionInfo.RequestedExtensionVersion.APIVersion &&
					SupportedExtensionsVersions[i].Revision >= INTCExtensionInfo.RequestedExtensionVersion.Revision)
				{
					INTCExtensionInfo.RequestedExtensionVersion = SupportedExtensionsVersions[i];
				}
				bIntelSupportsUAVOverlap = true;
			}

			if (GRHISupportsAtomicUInt64 && bIntelSupportsUAVOverlap)
			{
				break;
			}
		}
	}

	check(IntelExtensionContext == nullptr);
	INTCExtensionAppInfo AppInfo = {};
	AppInfo.pEngineName = TEXT("Unreal Engine");
	AppInfo.EngineVersion = 5;

	HRESULT hr = INTC_D3D11_CreateDeviceExtensionContext(Direct3DDevice, &IntelExtensionContext, &INTCExtensionInfo, &AppInfo);
	bool bEnabled = false;
	if (SUCCEEDED(hr))
	{
		bEnabled = true;
		UE_LOGF(LogD3D11RHI, Log, "Intel Extensions Framework enabled");
	}
	else if (hr == E_OUTOFMEMORY)
	{
		UE_LOGF(LogD3D11RHI, Log, "Intel Extensions Framework not supported by driver");
	}
	else if (hr == E_INVALIDARG)
	{
		UE_LOGF(LogD3D11RHI, Log, "Intel Extensions Framework passed invalid creation arguments");
	}

	if (!bEnabled)
	{
		GRHISupportsAtomicUInt64 = false;
		StopIntelExtensions();
	}

	if (SupportedExtensionsVersions != nullptr)
	{
		delete[] SupportedExtensionsVersions;
	}
}

void FD3D11DynamicRHI::StopIntelExtensions()
{
	if(IntelExtensionContext && UE::RHICore::AllowVendorDevice())
	{
		HRESULT hr = INTC_DestroyDeviceExtensionContext(&IntelExtensionContext);

		if (hr == S_OK)
		{
			UE_LOGF(LogD3D11RHI, Log, "Intel Extensions Framework unloaded");
		}
		else if (hr == E_INVALIDARG)
		{
			UE_LOGF(LogD3D11RHI, Log, "Intel Extensions Framework error when unloading");
		}

		IntelExtensionContext = nullptr;
	}
}
#endif // INTEL_EXTENSIONS

void FD3D11DynamicRHI::InitializeDeviceAgnosticVendorAPI()
{
	// We set up AMD and NVIDIA APIs here since they don't
	// need a device. Intel extensions are enabled after
	// device creation.
	if (!UE::RHICore::AllowVendorDevice())
	{
		return;
	}

#ifdef AMD_AGS_API
	if (IsRHIDeviceAMD())
	{
		check(AmdAgsContext == nullptr);

		// agsInitialize should be called before D3D device creation.
		if (agsInitialize(AGS_MAKE_VERSION(AMD_AGS_VERSION_MAJOR, AMD_AGS_VERSION_MINOR, AMD_AGS_VERSION_PATCH), nullptr, &AmdAgsContext, &AmdInfo.AmdGpuInfo) == AGS_SUCCESS)
		{
			AmdInfo.AmdAgsContext = AmdAgsContext;
			for (int32 DeviceIndex = 0; DeviceIndex < AmdInfo.AmdGpuInfo.numDevices; DeviceIndex++)
			{
				const AGSDeviceInfo& DeviceInfo = AmdInfo.AmdGpuInfo.devices[DeviceIndex];
				if (Adapter.DXGIAdapterDesc.VendorId == DeviceInfo.vendorId && Adapter.DXGIAdapterDesc.DeviceId == DeviceInfo.deviceId)
				{
					const uint32 AsicFamily = DeviceInfo.asicFamily;
					UE_LOGF(LogD3D11RHI, Log, "AMD asicFamily=%u", AsicFamily);
					GRHIDeviceIsAMDPreGCNArchitecture = DeviceInfo.asicFamily == AGSDeviceInfo::AsicFamily_PreGCN;
					GRHIGlobals.GpuInfo.AdapterGeneration = AsicFamily;
				}
			}

			if (GRHIDeviceIsAMDPreGCNArchitecture)
			{
				UE_LOGF(LogD3D11RHI, Log, "AMD Pre GCN architecture detected, some driver workarounds will be in place");
			}
		}
		else
		{
			UE_LOGF(LogD3D11RHI, Warning, "Failed to initialize AMD AGS API");

			FMemory::Memzero(&AmdInfo, sizeof(AmdInfo));
			// If agsInitialize returns anything but AGS_SUCCESS, the context pointer should be
			// guaranteed to be NULL, but we'll set it here explicitly, just to be safe.
			AmdAgsContext = nullptr;
		}
	}
	else
	{
		FMemory::Memzero(&AmdInfo, sizeof(AmdInfo));
	}
#endif // AMD_AGS_API

#ifdef NVAPI_INTERFACE
	if (IsRHIDeviceNVIDIA())
	{
		if (NvAPI_Initialize() == NVAPI_OK)
		{
			bNvAPIInitialized = true;

			NvPhysicalGpuHandle GPUHandles[NVAPI_MAX_PHYSICAL_GPUS];
			NvU32 NumGPUs = 0;
			if (NvAPI_EnumPhysicalGPUs(GPUHandles, &NumGPUs) == NVAPI_OK)
			{
				for (NvU32 GPU = 0; GPU < NumGPUs; ++GPU)
				{
					const NvPhysicalGpuHandle& GPUHandle = GPUHandles[GPU];

					NvU32 Unused = 0, SubsystemId = 0, RevisionId = 0, ExtDeviceId = 0;
					if (NvAPI_GPU_GetPCIIdentifiers(GPUHandle, &Unused, &SubsystemId, &RevisionId, &ExtDeviceId) != NVAPI_OK)
					{
						continue;
					}

					if (ExtDeviceId != Adapter.DXGIAdapterDesc.DeviceId || SubsystemId != Adapter.DXGIAdapterDesc.SubSysId || RevisionId != Adapter.DXGIAdapterDesc.Revision)
					{
						continue;
					}

					NV_GPU_ARCH_INFO ArchInfo;
					ArchInfo.version = NV_GPU_ARCH_INFO_VER_2;
					if (NvAPI_GPU_GetArchInfo(GPUHandle, &ArchInfo) != NVAPI_OK)
					{
						continue;
					}

					const uint32 Architecture = ArchInfo.architecture;
					UE_LOGF(LogD3D11RHI, Log, "NVIDIA Architecture=%u", Architecture);
					GRHIGlobals.GpuInfo.AdapterGeneration = Architecture;
				}
			}
		}
		else
		{
			UE_LOGF(LogD3D11RHI, Warning, "Failed to initialize NVAPI");
		}
	}
#endif // NVAPI_INTERFACE
}

void FD3D11DynamicRHI::InitD3DDevice()
{
	check( IsInGameThread() );

	// UE no longer supports clean-up and recovery on DEVICE_LOST.

	// If we don't have a device yet, either because this is the first viewport, or the old device was removed, create a device.
	if(!Direct3DDevice)
	{
		// Wait for the rendering thread to go idle.
		FlushRenderingCommands();

		UE_LOGF(LogD3D11RHI, Log, "Creating new Direct3DDevice");
		check(!GIsRHIInitialized);

		// Clear shadowed shader resources.
		ClearState();

		// In Direct3D 11, if you are trying to create a hardware or a software device, set pAdapter != NULL which constrains the other inputs to be:
		//		DriverType must be D3D_DRIVER_TYPE_UNKNOWN 
		//		Software must be NULL. 
		D3D_DRIVER_TYPE DriverType = D3D_DRIVER_TYPE_UNKNOWN;	

		uint32 DeviceFlags = D3D11RHI_ShouldAllowAsyncResourceCreation() ? 0 : D3D11_CREATE_DEVICE_SINGLETHREADED;

		// Use a debug device if specified on the command line.
		if (FParse::Param(FCommandLine::Get(), TEXT("d3ddebug")) ||
			FParse::Param(FCommandLine::Get(), TEXT("d3debug")) ||
			FParse::Param(FCommandLine::Get(), TEXT("dxdebug")))
		{
			GD3D11DebugCvar->Set(1, ECVF_SetByCommandline);
		}
		if (FParse::Param(FCommandLine::Get(), TEXT("d3dlogwarnings")))
		{
			GD3D11DebugCvar->Set(2, ECVF_SetByCommandline);
		}
		if (FParse::Param(FCommandLine::Get(), TEXT("d3dbreakonwarning")))
		{
			GD3D11DebugCvar->Set(3, ECVF_SetByCommandline);
		}
		if (FParse::Param(FCommandLine::Get(), TEXT("d3dcontinueonerrors")))
		{
			GD3D11DebugCvar->Set(4, ECVF_SetByCommandline);
		}
		GRHIGlobals.IsDebugLayerEnabled = (GD3D11DebugCvar.GetValueOnAnyThread() > 0);

		const bool bWithD3DDebug = GRHIGlobals.IsDebugLayerEnabled;
		FGenericCrashContext::SetEngineData(TEXT("RHI.D3DDebug"), bWithD3DDebug ? TEXT("true") : TEXT("false"));

		if (bWithD3DDebug)
		{
			DeviceFlags |= D3D11_CREATE_DEVICE_DEBUG;

			UE_LOGF(LogD3D11RHI, Log, "InitD3DDevice: -D3DDebug = %ls", bWithD3DDebug ? TEXT("on") : TEXT("off"));
		}

		// @MIXEDREALITY_CHANGE : BEGIN - Add BGRA flag for Windows Mixed Reality HMD's
		DeviceFlags |= D3D11_CREATE_DEVICE_BGRA_SUPPORT;
		// @MIXEDREALITY_CHANGE : END

		GTexturePoolSize = 0;

		// turn off creation on other threads for NVidia since a driver heuristic will notice that and make the creation synchronous, and that is not desirable given that large number of shaders will still be created on a single thread
		GRHISupportsMultithreadedShaderCreation = !IsRHIDeviceNVIDIA();
		GRequiredRecursiveShaders = ERecursiveShader::Resolve | ERecursiveShader::Clear;

		UE_LOGF(LogD3D11RHI, Log, "    GPU DeviceId: 0x%x (for the marketing name, search the web for \"GPU Device Id\")", Adapter.DXGIAdapterDesc.DeviceId);

		// Issue: 32bit windows doesn't report 64bit value, we take what we get.
		FD3D11GlobalStats::GDedicatedVideoMemory = int64(Adapter.DXGIAdapterDesc.DedicatedVideoMemory);
		FD3D11GlobalStats::GDedicatedSystemMemory = int64(Adapter.DXGIAdapterDesc.DedicatedSystemMemory);
		FD3D11GlobalStats::GSharedSystemMemory = int64(Adapter.DXGIAdapterDesc.SharedSystemMemory);

		// Total amount of system memory, clamped to 8 GB
		int64 TotalPhysicalMemory = FMath::Min(int64(FPlatformMemory::GetConstants().TotalPhysicalGB), 8ll) * (1024ll * 1024ll * 1024ll);

		// Consider 50% of the shared memory but max 25% of total system memory.
		int64 ConsideredSharedSystemMemory = FMath::Min( FD3D11GlobalStats::GSharedSystemMemory / 2ll, TotalPhysicalMemory / 4ll );

		TRefCountPtr<IDXGIAdapter3> DxgiAdapter3;
		DXGI_QUERY_VIDEO_MEMORY_INFO LocalVideoMemoryInfo;
		FD3D11GlobalStats::GTotalGraphicsMemory = 0;
		if (Adapter.bUMA)
		{
			// It's all system memory.
			FD3D11GlobalStats::GTotalGraphicsMemory = FD3D11GlobalStats::GDedicatedVideoMemory;
			FD3D11GlobalStats::GTotalGraphicsMemory += FD3D11GlobalStats::GDedicatedSystemMemory;
			FD3D11GlobalStats::GTotalGraphicsMemory += ConsideredSharedSystemMemory;
		}
		else if (IsRHIDeviceAMD() && SUCCEEDED(Adapter.DXGIAdapter->QueryInterface(_uuidof(IDXGIAdapter3), (void**)DxgiAdapter3.GetInitReference())) &&
			DxgiAdapter3.IsValid() && SUCCEEDED(DxgiAdapter3->QueryVideoMemoryInfo(0, DXGI_MEMORY_SEGMENT_GROUP_LOCAL, &LocalVideoMemoryInfo)))
		{
			// use the entire budget for D3D11, in keeping with setting GTotalGraphicsMemory to all of AdapterDesc.DedicatedVideoMemory
			// in the other method directly below
			FD3D11GlobalStats::GTotalGraphicsMemory = LocalVideoMemoryInfo.Budget;
		}
		else if ( FD3D11GlobalStats::GDedicatedVideoMemory >= 200*1024*1024 )
		{
			// Use dedicated video memory, if it's more than 200 MB
			FD3D11GlobalStats::GTotalGraphicsMemory = FD3D11GlobalStats::GDedicatedVideoMemory;
		}
		else if ( FD3D11GlobalStats::GDedicatedSystemMemory >= 200*1024*1024 )
		{
			// Use dedicated system memory, if it's more than 200 MB
			FD3D11GlobalStats::GTotalGraphicsMemory = FD3D11GlobalStats::GDedicatedSystemMemory;
		}
		else if ( FD3D11GlobalStats::GSharedSystemMemory >= 400*1024*1024 )
		{
			// Use some shared system memory, if it's more than 400 MB
			FD3D11GlobalStats::GTotalGraphicsMemory = ConsideredSharedSystemMemory;
		}
		else
		{
			// Otherwise consider 25% of total system memory for graphics.
			FD3D11GlobalStats::GTotalGraphicsMemory = TotalPhysicalMemory / 4ll;
		}

		if ( sizeof(SIZE_T) < 8 )
		{
			// Clamp to 1 GB if we're less than 64-bit
			FD3D11GlobalStats::GTotalGraphicsMemory = FMath::Min( FD3D11GlobalStats::GTotalGraphicsMemory, 1024ll * 1024ll * 1024ll );
		}

		if ( GPoolSizeVRAMPercentage > 0 )
		{
			float PoolSize = float(GPoolSizeVRAMPercentage) * 0.01f * float(FD3D11GlobalStats::GTotalGraphicsMemory);

			// Truncate GTexturePoolSize to MB (but still counted in bytes)
			GTexturePoolSize = int64(FGenericPlatformMath::TruncToFloat(PoolSize / 1024.0f / 1024.0f)) * 1024 * 1024;

			UE_LOGF(LogRHI,Log,"Texture pool is %llu MB (%d%% of %llu MB)",
				GTexturePoolSize / 1024 / 1024,
				GPoolSizeVRAMPercentage,
				FD3D11GlobalStats::GTotalGraphicsMemory / 1024 / 1024);
		}

		const bool bIsPerfHUD = !FCString::Stricmp(Adapter.DXGIAdapterDesc.Description,TEXT("NVIDIA PerfHUD"));

		if(bIsPerfHUD)
		{
			DriverType =  D3D_DRIVER_TYPE_REFERENCE;
		}

#if NV_AFTERMATH
		UE::RHICore::Nvidia::Aftermath::InitializeBeforeDeviceCreation();
#endif

		D3D_FEATURE_LEVEL ActualFeatureLevel = (D3D_FEATURE_LEVEL)0;

		if (IsRHIDeviceAMD() && CVarAMDUseMultiThreadedDevice.GetValueOnAnyThread())
		{
			DeviceFlags &= ~D3D11_CREATE_DEVICE_SINGLETHREADED;
		}

		bool bDeviceCreated = false;
#ifdef AMD_AGS_API
		if (IsRHIDeviceAMD() && AmdAgsContext && UE::RHICore::AllowVendorDevice())
		{
			uint32 AmdSupportedExtensionFlags = 0;

			UE_LOGF(LogD3D11RHI, Log, "Creating D3DDevice with AMD AGS, using adapter:");
			LogDXGIAdapterDesc(Adapter.DXGIAdapterDesc);

			AGSDX11DeviceCreationParams DeviceCreationParams = 
			{
				Adapter.DXGIAdapter,
				DriverType,
				NULL,
				DeviceFlags,
				&FeatureLevel,
				1,
				D3D11_SDK_VERSION,
				nullptr
			};

			// Engine registration can be disabled via console var. Also disable automatically if ShaderDevelopmentMode is on.
			auto* CVarDisableEngineAndAppRegistration = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DisableEngineAndAppRegistration"));
			const bool bDisableEngineRegistration = IsShaderDevelopmentModeEnabled() || 
				(CVarDisableEngineAndAppRegistration && CVarDisableEngineAndAppRegistration->GetValueOnAnyThread() != 0);
			const bool bDisableAppRegistration = bDisableEngineRegistration || !FApp::HasProjectName();

			AGSDX11ExtensionParams AmdExtensionParams;
			FMemory::Memzero(&AmdExtensionParams, sizeof(AmdExtensionParams));
			// Set the reserved UAV slot - matching the other vendor extensions.
			AmdExtensionParams.uavSlot = 7;
			// Disable old-style, "automatic" alternate-frame rendering (AFR) MGPU driver behavior
			AmdExtensionParams.crossfireMode = AGS_CROSSFIRE_MODE_DISABLE;

			// Register the engine name with the AMD driver, e.g. "UnrealEngine4.19", unless disabled
			// (note: to specify nothing for pEngineName below, you need to pass an empty string, not a null pointer)
			FString EngineName = FApp::GetEpicProductIdentifier() + FEngineVersion::Current().ToString(EVersionComponent::Minor);
			AmdExtensionParams.pEngineName = bDisableEngineRegistration ? TEXT("") : *EngineName;
			AmdExtensionParams.engineVersion = AGS_UNSPECIFIED_VERSION;

			// Register the project name with the AMD driver, unless disabled or no project name
			// (note: to specify nothing for pAppName below, you need to pass an empty string, not a null pointer)
			AmdExtensionParams.pAppName = bDisableAppRegistration ? TEXT("") : FApp::GetProjectName();
			AmdExtensionParams.appVersion = AGS_UNSPECIFIED_VERSION;

			// agsDriverExtensionsDX11_CreateDevice will not check for the DriverStore if there is not already a D3D11 device
			// initialized. In order to use AMF we require libraries to also be loaded from the driver store so we temporarily
			// initialize a device here and destroy it once the agsDriverExtensionsDX11_CreateDevice is run.
			TRefCountPtr<ID3D11Device> D3DDevicePreload;
			TRefCountPtr<ID3D11DeviceContext> D3DDeviceContextPreload;

			int32 NumAllowedFeatureLevels = 1;
			HRESULT Result = D3D11CreateDevice(
				Adapter.DXGIAdapter,
				D3D_DRIVER_TYPE_UNKNOWN,
				nullptr,
				DeviceFlags,
				&FeatureLevel,
				NumAllowedFeatureLevels,
				D3D11_SDK_VERSION,
				D3DDevicePreload.GetInitReference(),
				&ActualFeatureLevel,
				D3DDeviceContextPreload.GetInitReference());
			if (FAILED(Result))
			{
				UE_LOGF(LogD3D11RHI, Error, "Failed to load the AMD DriverStore library (0x%08X)", Result);
			}

			AGSDX11ReturnedParams DeviceCreationReturnedParams;
			FMemory::Memzero(&DeviceCreationReturnedParams, sizeof(DeviceCreationReturnedParams));
			AGSReturnCode DeviceCreation =
				agsDriverExtensionsDX11_CreateDevice(
					AmdAgsContext,
					&DeviceCreationParams,
					&AmdExtensionParams,
					&DeviceCreationReturnedParams);

			// Destroy temporary device and context
			D3DDevicePreload.SafeRelease();
			D3DDeviceContextPreload.SafeRelease();

			if (DeviceCreation == AGS_SUCCESS)
			{
				Direct3DDevice = DeviceCreationReturnedParams.pDevice;
				ActualFeatureLevel = DeviceCreationReturnedParams.featureLevel;
				Direct3DDeviceIMContext = DeviceCreationReturnedParams.pImmediateContext;
				AmdSupportedExtensionFlags = *(uint32*)&DeviceCreationReturnedParams.extensionsSupported;
				bDeviceCreated = true;
				UE_LOGF(LogD3D11RHI, Log, "Created device via AGS, feature level %ls, supported extensions %x.", GetFeatureLevelString(ActualFeatureLevel), AmdSupportedExtensionFlags);

				GRHISupportsAtomicUInt64 = DeviceCreationReturnedParams.extensionsSupported.intrinsics19 != 0;  // "intrinsics19" includes AtomicU64
				GSupportsDepthBoundsTest = DeviceCreationReturnedParams.extensionsSupported.depthBoundsTest != 0;
			}
			else
			{
				agsDeInitialize(AmdAgsContext);
				AmdAgsContext = nullptr;
				AmdSupportedExtensionFlags = 0;
				FMemory::Memzero(&AmdInfo, sizeof(AmdInfo));
				GRHIDeviceIsAMDPreGCNArchitecture = false;
				UE_LOGF(LogD3D11RHI, Warning, "Failed to create device via AGS, code %d.", DeviceCreation);
			}
		}
#endif //AMD_AGS_API

		if (!bDeviceCreated)
		{
			UE_LOGF(LogD3D11RHI, Log, "Creating D3DDevice using adapter:");
			LogDXGIAdapterDesc(Adapter.DXGIAdapterDesc);

			// Creating the Direct3D device.
			VERIFYD3D11RESULT(D3D11CreateDevice(
				Adapter.DXGIAdapter,
				DriverType,
				NULL,
				DeviceFlags,
				&FeatureLevel,
				1,
				D3D11_SDK_VERSION,
				Direct3DDevice.GetInitReference(),
				&ActualFeatureLevel,
				Direct3DDeviceIMContext.GetInitReference()
			));
		}

		// We should get the feature level we asked for as earlier we checked to ensure it is supported.
		check(ActualFeatureLevel == FeatureLevel);

#if WITH_DX_PERF
		Direct3DDeviceIMContext->QueryInterface<ID3DUserDefinedAnnotation>(Direct3DDeviceIMAnnotation.GetInitReference());
#endif

#if NV_AFTERMATH
		AftermathHandle = UE::RHICore::Nvidia::Aftermath::D3D11::InitializeDevice(Direct3DDevice, Direct3DDeviceIMContext);
#endif 

		if (bWithD3DDebug)
		{
			TRefCountPtr<ID3D11InfoQueue> d3dInfoQueue;
			if (SUCCEEDED(GD3D11RHI->GetDevice()->QueryInterface(__uuidof(ID3D11InfoQueue), (void**)d3dInfoQueue.GetInitReference())))
			{
				/* install callback */
				ExceptionHandlerHandle = AddVectoredExceptionHandler(1, D3D11VectoredExceptionHandler);

				/* filter messages */
				const bool bLogWarnings = D3D11_ShouldBreakOnD3DDebugWarnings() || D3D11_ShouldLogD3DDebugWarnings();
				D3D11_INFO_QUEUE_FILTER NewFilter;
				FMemory::Memzero(&NewFilter, sizeof(NewFilter));

				D3D11_MESSAGE_SEVERITY DenySeverity[] = { D3D11_MESSAGE_SEVERITY_INFO, D3D11_MESSAGE_SEVERITY_WARNING };
				NewFilter.DenyList.NumSeverities = 1 + (bLogWarnings ? 0 : 1);
				NewFilter.DenyList.pSeverityList = DenySeverity;


				// Be sure to carefully comment the reason for any additions here!  Someone should be able to look at it later and get an idea of whether it is still necessary.
				D3D11_MESSAGE_ID DenyIds[]  = {
					// OMSETRENDERTARGETS_INVALIDVIEW - d3d will complain if depth and color targets don't have the exact same dimensions, but actually
					//	if the color target is smaller then things are ok.  So turn off this error.  There is a manual check in FD3D11DynamicRHI::SetRenderTarget
					//	that tests for depth smaller than color and MSAA settings to match.
					D3D11_MESSAGE_ID_OMSETRENDERTARGETS_INVALIDVIEW, 

					// QUERY_BEGIN_ABANDONING_PREVIOUS_RESULTS - The RHI exposes the interface to make and issue queries and a separate interface to use that data.
					//		Currently there is a situation where queries are issued and the results may be ignored on purpose.  Filtering out this message so it doesn't
					//		swarm the debug spew and mask other important warnings
					D3D11_MESSAGE_ID_QUERY_BEGIN_ABANDONING_PREVIOUS_RESULTS,
					D3D11_MESSAGE_ID_QUERY_END_ABANDONING_PREVIOUS_RESULTS,

					// D3D11_MESSAGE_ID_CREATEINPUTLAYOUT_EMPTY_LAYOUT - This is a warning that gets triggered if you use a null vertex declaration,
					//       which we want to do when the vertex shader is generating vertices based on ID.
					D3D11_MESSAGE_ID_CREATEINPUTLAYOUT_EMPTY_LAYOUT,

					// D3D11_MESSAGE_ID_DEVICE_DRAW_INDEX_BUFFER_TOO_SMALL - This warning gets triggered by Slate draws which are actually using a valid index range.
					//		The invalid warning seems to only happen when VS 2012 is installed.  Reported to MS.  
					//		There is now an assert in DrawIndexedPrimitive to catch any valid errors reading from the index buffer outside of range.
					D3D11_MESSAGE_ID_DEVICE_DRAW_INDEX_BUFFER_TOO_SMALL,

					// D3D11_MESSAGE_ID_DEVICE_DRAW_RENDERTARGETVIEW_NOT_SET - This warning gets triggered by shadow depth rendering because the shader outputs
					//		a color but we don't bind a color render target. That is safe as writes to unbound render targets are discarded.
					//		Also, batched elements triggers it when rendering outside of scene rendering as it outputs to the GBuffer containing normals which is not bound.
					(D3D11_MESSAGE_ID)3146081, // D3D11_MESSAGE_ID_DEVICE_DRAW_RENDERTARGETVIEW_NOT_SET,

					// Spams constantly as we change the debug name on rendertargets that get reused.
					D3D11_MESSAGE_ID_SETPRIVATEDATA_CHANGINGPARAMS, 
				};

				NewFilter.DenyList.NumIDs = sizeof(DenyIds)/sizeof(D3D11_MESSAGE_ID);
				NewFilter.DenyList.pIDList = (D3D11_MESSAGE_ID*)&DenyIds;

				d3dInfoQueue->PushStorageFilter(&NewFilter);

				/* ensure callback is called */
				d3dInfoQueue->SetBreakOnSeverity(D3D11_MESSAGE_SEVERITY_CORRUPTION, D3D11_ShouldBreakOnD3DDebugErrors());
				d3dInfoQueue->SetBreakOnSeverity(D3D11_MESSAGE_SEVERITY_ERROR, D3D11_ShouldBreakOnD3DDebugErrors());
				if (bLogWarnings)
				{
					d3dInfoQueue->SetBreakOnSeverity(D3D11_MESSAGE_SEVERITY_WARNING, D3D11_ShouldBreakOnD3DDebugWarnings());
				}
			}
		}

		GRHIPersistentThreadGroupCount = 1440; // TODO: Revisit based on vendor/adapter/perf query

		StateCache.Init(Direct3DDeviceIMContext);

		// Check for async texture creation support.
		D3D11_FEATURE_DATA_THREADING ThreadingSupport = {0};
		VERIFYD3D11RESULT_EX(Direct3DDevice->CheckFeatureSupport(D3D11_FEATURE_THREADING, &ThreadingSupport, sizeof(ThreadingSupport)), Direct3DDevice);
		GRHISupportsAsyncTextureCreation = !!ThreadingSupport.DriverConcurrentCreates
			&& (DeviceFlags & D3D11_CREATE_DEVICE_SINGLETHREADED) == 0;

		if (IsRHIDeviceAMD() && CVarAMDDisableAsyncTextureCreation.GetValueOnAnyThread())
		{
			GRHISupportsAsyncTextureCreation = false;
		}

		GRHISupportsMultithreadedResources = GRHISupportsAsyncTextureCreation;

#ifdef NVAPI_INTERFACE
		if (IsRHIDeviceNVIDIA() && UE::RHICore::AllowVendorDevice() && bNvAPIInitialized)
		{
			if (NvAPI_D3D11_IsNvShaderExtnOpCodeSupported(Direct3DDevice, NV_EXTN_OP_UINT64_ATOMIC, &GRHISupportsAtomicUInt64) != NVAPI_OK)
			{
				UE_LOGF(LogD3D11RHI, Warning, "Failed to query support for 64 bit atomics");
			}
		}

		if( IsRHIDeviceNVIDIA() && CVarNVidiaTimestampWorkaround.GetValueOnAnyThread() )
		{
			// Workaround for pre-maxwell TDRs with realtime GPU stats (timestamp queries)
			// Note: Since there is no direct check for Kepler hardware and beyond, check for SHFL instruction
			bool bNVSHFLSupported = false;
			if (NvAPI_D3D11_IsNvShaderExtnOpCodeSupported(Direct3DDevice, NV_EXTN_OP_SHFL, &bNVSHFLSupported) == NVAPI_OK && !bNVSHFLSupported)
			{
				UE_LOGF(LogD3D11RHI, Display, "Timestamp queries are currently disabled on this hardware due to instability. Realtime GPU stats will not be available. You can override this behaviour by setting r.NVIDIATimestampWorkaround to 0");
				GSupportsTimestampRenderQueries = false;
			}
		}
#endif //NVAPI_INTERFACE

		IUnknown* RenderDoc;
		IID RenderDocID;
		if (SUCCEEDED(IIDFromString(L"{A7AA6116-9C8D-4BBA-9083-B4D816B71B78}", &RenderDocID)))
		{
			if (SUCCEEDED(Direct3DDevice->QueryInterface(RenderDocID, (void**)(&RenderDoc))))
			{
				bRenderDoc = true;

				// Running under RenderDoc, so enable capturing mode
				EnableIdealGPUCaptureOptions(true);
			}
		}

		IUnknown* IntelGPA;
		static const IID IntelGPAID = { 0xCCFFEF16, 0x7B69, 0x468F, {0xBC, 0xE3, 0xCD, 0x95, 0x33, 0x69, 0xA3, 0x9A} };

		if (SUCCEEDED(Direct3DDevice->QueryInterface(IntelGPAID, (void**)(&IntelGPA))))
		{
			// Running under Intel GPA, so enable capturing mode
			EnableIdealGPUCaptureOptions(true);
		}

		if (IsRHIDeviceNVIDIA())
		{
			GSupportsDepthBoundsTest = true;
		}

#if INTEL_EXTENSIONS
		if (IsRHIDeviceIntel() && UE::RHICore::AllowVendorDevice())
		{
			StartIntelExtensions();
		}
#endif // INTEL_EXTENSIONS

		if (GRHISupportsAtomicUInt64)
		{
			UE_LOGF(LogD3D11RHI, Log, "RHI has support for 64 bit atomics");
		}
		else
		{
			UE_LOGF(LogD3D11RHI, Log, "RHI does not have support for 64 bit atomics");
		}

		// Disable the RHI thread by default for devices that will likely suffer in performance
		if (FPlatformMisc::NumberOfCores() < 4)
		{
			GRHISupportsRHIThread = false;
		}

		SetupAfterDeviceCreation();
		GRHISupportsHDROutput = SetupDisplayHDRMetaData();

#if WITH_EDITOR
		GRHIHDRDisplayOutputFormat = PF_FloatRGBA;
#else
		GRHIHDRDisplayOutputFormat = PF_A2B10G10R10;
#endif

		// Add device overclock state to crash context
		const bool bIsGPUOverclocked = IsDeviceOverclocked();
		FGenericCrashContext::SetEngineData(TEXT("RHI.IsGPUOverclocked"), bIsGPUOverclocked ? TEXT("true") : TEXT("false"));
		
		FHardwareInfo::RegisterHardwareInfo( NAME_RHI, TEXT( "D3D11" ) );

		GRHISupportsTextureStreaming = true;
		GRHISupportsFirstInstance = true;
		GRHINeedsExtraDeletionLatency = false;
		GRHISupportsEfficientUploadOnResourceCreation = true;

		// Now that the driver extensions have been initialized, turn on UAV overlap for the first time.
		EnableUAVOverlap();

		FRenderResource::InitPreRHIResources();
		GIsRHIInitialized = true;
	}
}

static void GetAvailableResolutionsForOutput(IDXGIOutput* Output, ID3D11Device* Device, int32 CurrentOutput, FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	// TODO: GetDisplayModeList is a terribly SLOW call.  It can take up to a second per invocation.
	//  We might want to work around some DXGI badness here.
	const DXGI_FORMAT DisplayFormats[] = { DXGI_FORMAT_R8G8B8A8_UNORM, DXGI_FORMAT_B8G8R8A8_UNORM };
	DXGI_FORMAT Format = DisplayFormats[0];
	bool bFoundValidResult = false;
	uint32 NumModes = 0;

	for (DXGI_FORMAT CurrentFormat : DisplayFormats)
	{
		const HRESULT HResult = Output->GetDisplayModeList(CurrentFormat, 0, &NumModes, NULL);

		if (FAILED(HResult))
		{
			if (HResult == DXGI_ERROR_NOT_FOUND)
			{
				UE_LOGF(LogD3D11RHI, Warning, "RHIGetAvailableResolutions failed with generic error.");
			}
			else if (HResult == DXGI_ERROR_NOT_CURRENTLY_AVAILABLE)
			{
				UE_LOGF(LogD3D11RHI, Warning, "RHIGetAvailableResolutions does not return results when running under remote desktop.");
				return;
			}
			else
			{
				UE_LOGF(LogD3D11RHI, Warning, "RHIGetAvailableResolutions failed with unknown error (0x%x).", HResult);
				return;
			}
		}
		else
		{
			bFoundValidResult = true;
			if (NumModes)
			{
				Format = CurrentFormat;
				break;
			}
		}
	}

	// If we couldn't iterate any formats for this output, it might be dead in the driver and there's nothing we can do here except move on.
	if (!bFoundValidResult)
	{
		// Grab diagnostic information from the output, if at all possible.
		DXGI_OUTPUT_DESC OutputDesc;
		FMemory::Memzero(OutputDesc);
		HRESULT OutputDescRes = Output->GetDesc(&OutputDesc);

		UE_LOGF(LogD3D11RHI, Warning,
			"RHIGetAvailableResolutions could not get any display modes from output %i (D:%i) (Res:'%ls'(0x%08X)",
			CurrentOutput,
			OutputDesc.AttachedToDesktop,
			*GetD3D11ErrorString(OutputDescRes, Device), OutputDescRes);

		return;
	}

	if (NumModes > 0)
	{
		TArray<DXGI_MODE_DESC> ModeList;
		HRESULT HResult;
		do
		{
			VERIFYD3D11RESULT(Output->GetDisplayModeList(Format, 0, &NumModes, nullptr));
			ModeList.SetNum(NumModes);
			HResult = Output->GetDisplayModeList(Format, 0, &NumModes, ModeList.GetData());
		} while (HResult == DXGI_ERROR_MORE_DATA);
		VERIFYD3D11RESULT(HResult);

		constexpr int32 MinAllowableResolutionX = 0;
		constexpr int32 MinAllowableResolutionY = 0;
		constexpr int32 MaxAllowableResolutionX = 10480;
		constexpr int32 MaxAllowableResolutionY = 10480;
		constexpr int32 MinAllowableRefreshRate = 0;
		constexpr int32 MaxAllowableRefreshRate = 10480;
		for (uint32 m = 0; m < NumModes; m++)
		{
			CA_SUPPRESS(6385);
			if (((int32)ModeList[m].Width >= MinAllowableResolutionX) &&
				((int32)ModeList[m].Width <= MaxAllowableResolutionX) &&
				((int32)ModeList[m].Height >= MinAllowableResolutionY) &&
				((int32)ModeList[m].Height <= MaxAllowableResolutionY)
				)
			{
				bool bAddIt = true;
				if (bIgnoreRefreshRate == false)
				{
					if (((int32)ModeList[m].RefreshRate.Numerator < MinAllowableRefreshRate * ModeList[m].RefreshRate.Denominator) ||
						((int32)ModeList[m].RefreshRate.Numerator > MaxAllowableRefreshRate * ModeList[m].RefreshRate.Denominator)
						)
					{
						continue;
					}
				}
				else
				{
					// See if it is in the list already
					for (int32 CheckIndex = 0; CheckIndex < Resolutions.Num(); CheckIndex++)
					{
						FScreenResolutionRHI& CheckResolution = Resolutions[CheckIndex];
						if ((CheckResolution.Width == ModeList[m].Width) &&
							(CheckResolution.Height == ModeList[m].Height))
						{
							// Already in the list...
							bAddIt = false;
							break;
						}
					}
				}

				if (bAddIt)
				{
					FScreenResolutionRHI& ScreenResolution = Resolutions.Emplace_GetRef();
					ScreenResolution.Width = ModeList[m].Width;
					ScreenResolution.Height = ModeList[m].Height;
					ScreenResolution.RefreshRate = ModeList[m].RefreshRate.Numerator / ModeList[m].RefreshRate.Denominator;
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogD3D11RHI, Warning, "No display modes found for the standard format DXGI_FORMAT_R8G8B8A8_UNORM!");
	}
}

/**
 *	Retrieve available screen resolutions.
 *
 *	@param	Resolutions			TArray<FScreenResolutionRHI> parameter that will be filled in.
 *	@param	bIgnoreRefreshRate	If true, ignore refresh rates.
 *
 *	@return	bool				true if successfully filled the array
 */
bool FD3D11DynamicRHI::RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	// TODO: The resolution information can be stale if the adapter's factory isn't current. Use RHIGetAvailableResolutionsForDisplay instead.
	TRefCountPtr<IDXGIOutput> Output;
	const HRESULT HResult = Adapter.DXGIAdapter->EnumOutputs(0, Output.GetInitReference());
	if (SUCCEEDED(HResult))
	{
		GetAvailableResolutionsForOutput(Output, GetDevice(), 0, Resolutions, bIgnoreRefreshRate);
	}
	return !Resolutions.IsEmpty();
}

bool FD3D11DynamicRHI::RHIGetAvailableResolutionsForDisplay(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate, void* NativeMonitorHandle)
{
	HMONITOR MonitorHandle = static_cast<HMONITOR>(NativeMonitorHandle);
	if (MonitorHandle == NULL)
	{
		return false;
	}

	if (!DXGIFactoryForResolutions.IsValid() || !DXGIFactoryForResolutions->IsCurrent())
	{
		SafeCreateDXGIFactory(DXGIFactoryForResolutions.GetInitReference(), GRHIGlobals.IsDebugLayerEnabled);
	}

	HRESULT HResult;
	TRefCountPtr<IDXGIAdapter> DXGIAdapter;
	for (int32 CurrentAdapter = 0; (HResult = DXGIFactoryForResolutions->EnumAdapters(CurrentAdapter, DXGIAdapter.GetInitReference())) != DXGI_ERROR_NOT_FOUND; ++CurrentAdapter)
	{
		if (FAILED(HResult))
		{
			return false;
		}

		TRefCountPtr<IDXGIOutput> Output;
		for (int32 CurrentOutput = 0; (HResult = DXGIAdapter->EnumOutputs(CurrentOutput, Output.GetInitReference())) != DXGI_ERROR_NOT_FOUND; ++CurrentOutput)
		{
			if (FAILED(HResult))
			{
				return false;
			}

			DXGI_OUTPUT_DESC OutputDesc{};
			HRESULT OutputDescRes = Output->GetDesc(&OutputDesc);
			check(SUCCEEDED(OutputDescRes));
			if (MonitorHandle == OutputDesc.Monitor)
			{
				GetAvailableResolutionsForOutput(Output, GetDevice(), CurrentOutput, Resolutions, bIgnoreRefreshRate);

				// Only allow the desktop resolution if SetFullscreenState is expected to fail to allow borderless to seemlessly take over.
				// Undocumented but assume exclusive only works if the the display is primary, i.e. top-left at (0, 0), or the display is connected to the chosen adapter.
				if (OutputDesc.DesktopCoordinates.left != 0 || OutputDesc.DesktopCoordinates.top != 0)
				{
					const FD3D11Adapter& ChosenAdapter = GetAdapter();
					DXGI_ADAPTER_DESC AdapterDesc;
					if (SUCCEEDED(DXGIAdapter->GetDesc(&AdapterDesc)) &&
						FMemory::Memcmp(&ChosenAdapter.DXGIAdapterDesc.AdapterLuid, &AdapterDesc.AdapterLuid, sizeof(LUID)))
					{
						Resolutions.Reset(1);
						FScreenResolutionRHI& ScreenResolution = Resolutions.Emplace_GetRef();
						ScreenResolution.Width = OutputDesc.DesktopCoordinates.right - OutputDesc.DesktopCoordinates.left;
						ScreenResolution.Height = OutputDesc.DesktopCoordinates.bottom - OutputDesc.DesktopCoordinates.top;
						ScreenResolution.RefreshRate = 0;
					}
				}

				return !Resolutions.IsEmpty();
			}
		}
	}

	return false;
}
