// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Viewport.cpp: D3D viewport RHI implementation.
=============================================================================*/

#include "D3D11RHIPrivate.h"
#include "Misc/CommandLine.h"
#include "Windows/IDXGISwapchainProvider.h"
#include "Features/IModularFeatures.h"
#include "RenderingThread.h"
#include "RHIUtilities.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <dwmapi.h>

#ifndef DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING
#define DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING  2048
#endif

static DXGI_SWAP_EFFECT GSwapEffect = DXGI_SWAP_EFFECT_DISCARD;
static DXGI_SCALING GSwapScaling = DXGI_SCALING_STRETCH;
static uint32 GSwapChainBufferCount = 1;

static int32 GD3D11UseAllowTearing = 1;
static FAutoConsoleVariableRef CVarD3DUseAllowTearing(
	TEXT("r.D3D11.UseAllowTearing"),
	GD3D11UseAllowTearing,
	TEXT("Enable new dxgi flip mode with d3d11"),
	ECVF_RenderThreadSafe| ECVF_ReadOnly
);

uint32 FD3D11Viewport::GSwapChainFlags = DXGI_SWAP_CHAIN_FLAG_ALLOW_MODE_SWITCH;

FD3D11Viewport::FD3D11Viewport(FD3D11DynamicRHI* InD3DRHI, HWND InWindowHandle, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat InPreferredPixelFormat)
	: D3DRHI       (InD3DRHI)
	, WindowHandle (InWindowHandle)
	, PixelFormat  (InD3DRHI->GetDisplayFormat(InPreferredPixelFormat))
	, SizeX        (InSizeX)
	, SizeY        (InSizeY)
	, bIsFullscreen(bInIsFullscreen)
{
	check(IsInGameThread());
	D3DRHI->Viewports.Add(this);

	// Ensure that the D3D device has been created.
	D3DRHI->InitD3DDevice();

	TArray<IDXGISwapchainProvider*> DXGISwapchainProviderModules = IModularFeatures::Get().GetModularFeatureImplementations<IDXGISwapchainProvider>(IDXGISwapchainProvider::GetModularFeatureName());
	IDXGISwapchainProvider* DXGISwapchainProvider = nullptr;
	for (IDXGISwapchainProvider* ProviderModule : DXGISwapchainProviderModules)
	{
		if (ProviderModule->SupportsRHI(ERHIInterfaceType::D3D11))
		{
			DXGISwapchainProvider = ProviderModule;
			break;
		}
	}

	if (DXGISwapchainProvider)
	{
		static bool bCustomSwapchainLogged = false;
		if (!bCustomSwapchainLogged)
		{
			UE_LOGF(LogD3D11RHI, Log, "Found a custom swapchain provider: '%ls'.", DXGISwapchainProvider->GetProviderName());
			bCustomSwapchainLogged = true;
		}
	}

	// Create a backbuffer/swapchain for each viewport
	TRefCountPtr<IDXGIDevice> DXGIDevice;
	VERIFYD3D11RESULT_EX( D3DRHI->GetDevice()->QueryInterface(IID_PPV_ARGS(DXGIDevice.GetInitReference())), D3DRHI->GetDevice() );

	{
		GSwapEffect = DXGI_SWAP_EFFECT_DISCARD;
		GSwapScaling = DXGI_SCALING_STRETCH;
		GSwapChainFlags = DXGI_SWAP_CHAIN_FLAG_ALLOW_MODE_SWITCH;
		IDXGIFactory1* Factory1 = D3DRHI->GetFactory();
		TRefCountPtr<IDXGIFactory5> Factory5;

		if (GD3D11UseAllowTearing)
		{
			if (S_OK == Factory1->QueryInterface(IID_PPV_ARGS(Factory5.GetInitReference())))
			{
				UINT AllowTearing = 0;
				if (S_OK == Factory5->CheckFeatureSupport(DXGI_FEATURE_PRESENT_ALLOW_TEARING, &AllowTearing, sizeof(UINT)) && AllowTearing != 0)
				{
					GSwapEffect = DXGI_SWAP_EFFECT_FLIP_DISCARD;
					GSwapScaling = DXGI_SCALING_NONE;
					GSwapChainFlags |= DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING;
					bAllowTearing = true;
					GSwapChainBufferCount = 2;
				}
			}
		}
	}

	// If requested, keep a handle to a DXGIOutput so we can force that display on fullscreen swap
	uint32 DisplayIndex = 0;
	bForcedFullscreenDisplay = FParse::Value(FCommandLine::Get(), TEXT("FullscreenDisplay="), DisplayIndex);

	if (bForcedFullscreenDisplay)
	{
		TRefCountPtr<IDXGIAdapter> DXGIAdapter;
		DXGIDevice->GetAdapter(DXGIAdapter.GetInitReference());

		if (S_OK != DXGIAdapter->EnumOutputs(DisplayIndex, ForcedFullscreenOutput.GetInitReference()))
		{
			UE_LOGF(LogD3D11RHI, Log, "Failed to find requested output display (%i).", DisplayIndex);
			ForcedFullscreenOutput = nullptr;
			bForcedFullscreenDisplay = false;
		}
	}
	else
	{
		ForcedFullscreenOutput = nullptr;
	}

	DXGI_FORMAT SwapChainFormat = GetRenderTargetFormat(PixelFormat);

	// Skip swap chain creation in off-screen rendering mode
	bNeedSwapChain = !FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen"));
	if (bNeedSwapChain)
	{
		TRefCountPtr<IDXGIFactory2> Factory2;
		const bool bSupportsFactory2 = SUCCEEDED(D3DRHI->GetFactory()->QueryInterface(__uuidof(IDXGIFactory2), (void**)Factory2.GetInitReference()));

		// Create the swapchain.
		if (InD3DRHI->IsQuadBufferStereoEnabled() && bSupportsFactory2)
		{
			BOOL stereoEnabled = Factory2->IsWindowedStereoEnabled();
			if (stereoEnabled)
			{
				DXGI_SWAP_CHAIN_DESC1 SwapChainDesc1;
				FMemory::Memzero(&SwapChainDesc1, sizeof(DXGI_SWAP_CHAIN_DESC1));

				// Enable stereo 
				SwapChainDesc1.Stereo = true;
				// MSAA Sample count
				SwapChainDesc1.SampleDesc.Count = 1;
				SwapChainDesc1.SampleDesc.Quality = 0;

				SwapChainDesc1.Format = SwapChainFormat;
				SwapChainDesc1.BufferUsage = DXGI_USAGE_RENDER_TARGET_OUTPUT | DXGI_USAGE_SHADER_INPUT;
				// Double buffering required to create stereo swap chain
				SwapChainDesc1.BufferCount = 2;
				SwapChainDesc1.Scaling = DXGI_SCALING_NONE;
				SwapChainDesc1.SwapEffect = DXGI_SWAP_EFFECT_FLIP_DISCARD;
				SwapChainDesc1.Flags = GSwapChainFlags;

				TRefCountPtr<IDXGISwapChain1> SwapChain1;
				HRESULT CreateSwapChainForHwndResult = DXGISwapchainProvider ?
					DXGISwapchainProvider->CreateSwapChainForHwnd(Factory2, D3DRHI->GetDevice(), WindowHandle, &SwapChainDesc1, nullptr, nullptr, SwapChain1.GetInitReference()) :
					Factory2->CreateSwapChainForHwnd(D3DRHI->GetDevice(), WindowHandle, &SwapChainDesc1, nullptr, nullptr, SwapChain1.GetInitReference());

				VERIFYD3D11RESULT_EX(CreateSwapChainForHwndResult, D3DRHI->GetDevice());
				SwapChain = SwapChain1;
			}
			else
			{
				UE_LOGF(LogD3D11RHI, Log, "FD3D11Viewport::FD3D11Viewport was not able to create stereo SwapChain; Please enable stereo in driver settings.");
				InD3DRHI->DisableQuadBufferStereo();
			}
		}

		// Try and create a swapchain capable of being used on HDR monitors
		if ((SwapChain == nullptr) && InD3DRHI->bDXGISupportsHDR && bSupportsFactory2)
		{
			// Create the swapchain.
			DXGI_SWAP_CHAIN_DESC1 SwapChainDesc;
			FMemory::Memzero(&SwapChainDesc, sizeof(DXGI_SWAP_CHAIN_DESC1));
			SwapChainDesc.Width = SizeX;
			SwapChainDesc.Height = SizeY;
			SwapChainDesc.SampleDesc.Count = 1;
			SwapChainDesc.SampleDesc.Quality = 0;
			SwapChainDesc.Format = SwapChainFormat;
			SwapChainDesc.BufferUsage = DXGI_USAGE_RENDER_TARGET_OUTPUT | DXGI_USAGE_SHADER_INPUT;

			DXGI_SWAP_CHAIN_FULLSCREEN_DESC FSSwapChainDesc = {};
			FSSwapChainDesc.Windowed = !bIsFullscreen;

			// Needed for HDR
			BackBufferCount = 2;
			SwapChainDesc.SwapEffect = GSwapEffect;
			SwapChainDesc.BufferCount = BackBufferCount;
			SwapChainDesc.Flags = GSwapChainFlags;
			SwapChainDesc.Scaling = GSwapScaling;

			TRefCountPtr<IDXGISwapChain1> SwapChain1;
			HRESULT CreateSwapChainForHwndResult = DXGISwapchainProvider ?
				DXGISwapchainProvider->CreateSwapChainForHwnd(Factory2, D3DRHI->GetDevice(), WindowHandle, &SwapChainDesc, &FSSwapChainDesc, nullptr, SwapChain1.GetInitReference()) :
				Factory2->CreateSwapChainForHwnd(D3DRHI->GetDevice(), WindowHandle, &SwapChainDesc, &FSSwapChainDesc, nullptr, SwapChain1.GetInitReference());
			if(SUCCEEDED(CreateSwapChainForHwndResult))
			{
				SwapChain1->QueryInterface(IID_PPV_ARGS(SwapChain.GetInitReference()));

#if UE_INPUT_LATENCY_TRACKING
				FRHIFrameFlipStatTrackingRunnable::Create(SwapChain1);
#endif

				RECT WindowRect = {};
				GetWindowRect(WindowHandle, &WindowRect);

				FVector2D WindowTopLeft((float)WindowRect.left, (float)WindowRect.top);
				FVector2D WindowBottomRight((float)WindowRect.right, (float)WindowRect.bottom);
				bool bHDREnabled = false;
				EDisplayColorGamut LocalDisplayColorGamut = DisplayColorGamut;
				EDisplayOutputFormat LocalDisplayOutputFormat = DisplayOutputFormat;

				HDRGetMetaData(LocalDisplayOutputFormat, LocalDisplayColorGamut, bHDREnabled, WindowTopLeft, WindowBottomRight, (void*)WindowHandle);
				if (bHDREnabled)
				{
					DisplayOutputFormat = LocalDisplayOutputFormat;
					DisplayColorGamut = LocalDisplayColorGamut;
					EnableHDR();
				}
			}
			else
			{
				UE_LOGF(LogD3D11RHI, Log, "CreateSwapChainForHwnd failed with result '%ls' (0x%08X), falling back to legacy CreateSwapChain.",
					*GetD3D11ErrorString(CreateSwapChainForHwndResult, D3DRHI->GetDevice()),
					CreateSwapChainForHwndResult);
			}
		}

		if (SwapChain == nullptr)
		{
			BackBufferCount = GSwapChainBufferCount;

			// Create the swapchain.
			DXGI_SWAP_CHAIN_DESC SwapChainDesc;
			FMemory::Memzero(&SwapChainDesc, sizeof(DXGI_SWAP_CHAIN_DESC));

			SwapChainDesc.BufferDesc = SetupDXGI_MODE_DESC();
			// MSAA Sample count
			SwapChainDesc.SampleDesc.Count = 1;
			SwapChainDesc.SampleDesc.Quality = 0;
			SwapChainDesc.BufferUsage = DXGI_USAGE_RENDER_TARGET_OUTPUT | DXGI_USAGE_SHADER_INPUT;
			// 1:single buffering, 2:double buffering, 3:triple buffering
			SwapChainDesc.BufferCount = BackBufferCount;
			SwapChainDesc.OutputWindow = WindowHandle;
			SwapChainDesc.Windowed = !bIsFullscreen;
			// DXGI_SWAP_EFFECT_DISCARD / DXGI_SWAP_EFFECT_SEQUENTIAL
			SwapChainDesc.SwapEffect = GSwapEffect;
			SwapChainDesc.Flags = GSwapChainFlags;

			HRESULT CreateSwapChainResult = DXGISwapchainProvider ?
				DXGISwapchainProvider->CreateSwapChain(D3DRHI->GetFactory(), D3DRHI->GetDevice(), &SwapChainDesc, SwapChain.GetInitReference()) :
				D3DRHI->GetFactory()->CreateSwapChain(D3DRHI->GetDevice(), &SwapChainDesc, SwapChain.GetInitReference());
			if (CreateSwapChainResult == E_INVALIDARG)
			{
				const TCHAR* D3DFormatString = UE::DXGIUtilities::GetFormatString(SwapChainDesc.BufferDesc.Format);

				UE_LOGF(LogD3D11RHI, Error,
					"CreateSwapChain failed with E_INVALIDARG: \n" " Size:%ix%i Format:%ls(0x%08X) \n" " Windowed:%i SwapEffect:%i Flags: 0x%08X",
					SwapChainDesc.BufferDesc.Width, SwapChainDesc.BufferDesc.Height, D3DFormatString, SwapChainDesc.BufferDesc.Format,
					SwapChainDesc.Windowed, SwapChainDesc.SwapEffect, SwapChainDesc.Flags);

				{
					UINT FormatSupport = 0;
					HRESULT FormatSupportResult = D3DRHI->GetDevice()->CheckFormatSupport(SwapChainDesc.BufferDesc.Format, &FormatSupport);
					if (SUCCEEDED(FormatSupportResult))
					{
						UE_LOGF(LogD3D11RHI, Error, "CheckFormatSupport(%ls): 0x%08x", D3DFormatString, FormatSupport);
					}
					else
					{
						UE_LOGF(LogD3D11RHI, Error, "CheckFormatSupport(%ls) failed: 0x%08x", D3DFormatString, FormatSupportResult);
					}
				}
			}
			VERIFYD3D11RESULT_EX(CreateSwapChainResult, D3DRHI->GetDevice());
		}

		// Set the DXGI message hook to not change the window behind our back.
		D3DRHI->GetFactory()->MakeWindowAssociation(WindowHandle,DXGI_MWA_NO_WINDOW_CHANGES);
	}
	// Create a RHI surface to represent the viewport's back buffer.
	BackBuffer = GetSwapChainSurface(D3DRHI, PixelFormat, SizeX, SizeY, SwapChain);

	// Tell the window to redraw when they can.
	// @todo: For Slate viewports, it doesn't make sense to post WM_PAINT messages (we swallow those.)
	::PostMessageW( WindowHandle, WM_PAINT, 0, 0 );
}

static const FString GetDXGIColorSpaceString(DXGI_COLOR_SPACE_TYPE ColorSpace)
{
	switch (ColorSpace)
	{
	case DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P709:
		return TEXT("RGB_FULL_G22_NONE_P709");
	case DXGI_COLOR_SPACE_RGB_FULL_G10_NONE_P709:
		return TEXT("RGB_FULL_G10_NONE_P709");
	case DXGI_COLOR_SPACE_RGB_FULL_G2084_NONE_P2020:
		return TEXT("RGB_FULL_G2084_NONE_P2020");
	case DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P2020:
		return TEXT("RGB_FULL_G22_NONE_P2020");
	default:
		break;
	}

	return FString::FromInt(ColorSpace);
};

inline void EnsureColorSpace(IDXGISwapChain* SwapChain, DXGI_COLOR_SPACE_TYPE NewColorSpace)
{
	if (SwapChain == nullptr)
	{
		return;
	}

	TRefCountPtr<IDXGISwapChain3> swapChain3;
	if (FAILED(SwapChain->QueryInterface(IID_PPV_ARGS(swapChain3.GetInitReference()))))
	{
		return;
	}

	{
		UINT ColorSpaceSupport = 0;
		HRESULT hr = swapChain3->CheckColorSpaceSupport(NewColorSpace, &ColorSpaceSupport);
		FString ColorSpaceName = GetDXGIColorSpaceString(NewColorSpace);
		if (SUCCEEDED(hr) && (ColorSpaceSupport & DXGI_SWAP_CHAIN_COLOR_SPACE_SUPPORT_FLAG_PRESENT))
		{
			swapChain3->SetColorSpace1(NewColorSpace);
			UE_LOGF(LogD3D11RHI, Verbose, "Setting color space on swap chain: %ls", *ColorSpaceName);
		}
		else
		{
			UE_LOGF(LogD3D11RHI, Error, "Warning: unable to set color space %ls to the swapchain: verify EDisplayOutputFormat / swapchain format", *ColorSpaceName);
		}
	}
}

inline void EnsureColorSpace(IDXGISwapChain* SwapChain, EDisplayColorGamut DisplayGamut, EDisplayOutputFormat OutputDevice, EPixelFormat PixelFormat)
{
	DXGI_COLOR_SPACE_TYPE NewColorSpace = DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P709;	// sRGB;
	const bool bPrimaries2020 = (DisplayGamut == EDisplayColorGamut::Rec2020_D65);

	// See console variable r.HDR.Display.OutputDevice.
	switch (OutputDevice)
	{
		// Gamma 2.2
	case EDisplayOutputFormat::SDR_sRGB:
	case EDisplayOutputFormat::SDR_Rec709:
		NewColorSpace = bPrimaries2020 ? DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P2020 : DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P709;
		break;

		// Gamma ST.2084
	case EDisplayOutputFormat::HDR_ACES_1000nit_ST2084:
	case EDisplayOutputFormat::HDR_ACES_2000nit_ST2084:
		NewColorSpace = DXGI_COLOR_SPACE_RGB_FULL_G2084_NONE_P2020;
		ensure(PixelFormat == PF_A2B10G10R10);
		break;

		// Gamma 1.0 (Linear)
	case EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB:
	case EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB:
		// Linear. Still supports expanded color space with values >1.0f and <0.0f.
		// The actual range is determined by the pixel format (e.g. a UNORM format can only ever have 0-1).
		NewColorSpace = DXGI_COLOR_SPACE_RGB_FULL_G10_NONE_P709;
		ensure(PixelFormat == PF_FloatRGBA);
		break;
	}

	EnsureColorSpace(SwapChain, NewColorSpace);
}

// When a window has moved or resized we need to check whether it is on a HDR monitor or not. Set the correct color space of the monitor
void FD3D11Viewport::CheckHDRMonitorStatus()
{
}

void FD3D11Viewport::EnableHDR()
{
	if ( GRHISupportsHDROutput && IsHDREnabled() )
	{
		// Ensure we have the correct color space set.
		EnsureColorSpace(SwapChain, DisplayColorGamut, DisplayOutputFormat, PixelFormat);
	}
}

void FD3D11Viewport::ShutdownHDR()
{
	// Make sure to set the appropriate color space even if GRHISupportsHDROutput is false because we 
	// might have toggled HDR on and off in the windows settings
	if (PixelFormat == PF_FloatRGBA)
	{
		// Per DXGI spec, float-valued swapchains must contain linear data.
		// The swapchain color space must therefore be G10 instead of G22.
		EnsureColorSpace(SwapChain, DXGI_COLOR_SPACE_RGB_FULL_G10_NONE_P709);
	}
	else
	{
		// Ensure we have the correct color space set.
		EnsureColorSpace(SwapChain, EDisplayColorGamut::sRGB_D65, EDisplayOutputFormat::SDR_sRGB, PixelFormat);
	}
}

void FD3D11Viewport::ConditionalResetSwapChain(bool bIgnoreFocus)
{
	uint32 Valid = ValidState;
	if(0 != (Valid & VIEWPORT_INVALID))
	{
		if(0 != (Valid & VIEWPORT_FULLSCREEN_LOST))
		{
			FlushRenderingCommands();
			ValidState &= ~(VIEWPORT_FULLSCREEN_LOST);
			Resize(SizeX, SizeY, false, PixelFormat);
		}
		else
		{
			ResetSwapChainInternal(bIgnoreFocus);
		}
	}
}

void FD3D11Viewport::ResetSwapChainInternal(bool bIgnoreFocus)
{
	if (0 != (ValidState & VIEWPORT_INVALID))
	{
		// Check if the viewport's window is focused before resetting the swap chain's fullscreen state.
		HWND FocusWindow = ::GetFocus();
		const bool bIsFocused = FocusWindow == WindowHandle;
		const bool bIsIconic = !!::IsIconic(WindowHandle);
		if (bIgnoreFocus || (bIsFocused && !bIsIconic))
		{
			FlushRenderingCommands();

			// Explicit output selection in fullscreen only (commandline or HDR enabled)
			bool bNeedsForcedDisplay = bIsFullscreen && (bForcedFullscreenDisplay || PixelFormat == PF_FloatRGBA);
			HRESULT Result = SwapChain->SetFullscreenState(bIsFullscreen, bNeedsForcedDisplay ? ForcedFullscreenOutput : nullptr);

			if (SUCCEEDED(Result))
			{
				ValidState &= ~(VIEWPORT_INVALID);
			}
			else if (Result != DXGI_ERROR_NOT_CURRENTLY_AVAILABLE && Result != DXGI_STATUS_MODE_CHANGE_IN_PROGRESS)
			{
				UE_LOGF(LogD3D11RHI, Error, "IDXGISwapChain::SetFullscreenState returned %08x, unknown error status.", Result);
			}
		}
	}
}

void FD3D11DynamicRHI::RHIGetDisplaysInformation(FDisplayInformationArray& OutDisplayInformation)
{
	OutDisplayInformation.Append(DisplayList);
}

#include "Windows/HideWindowsPlatformTypes.h"
