// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11ConstantBuffer.h: Public D3D Constant Buffer definitions.
=============================================================================*/

#pragma once

#include "RenderResource.h"
#include "Stats/Stats.h"

class FD3D11DynamicRHI;

/**
Size of the default constant buffer.
Note: D3D11 allows for 64k of storage but increasing this has a negative impact on performance.
*/
#define MAX_GLOBAL_CONSTANT_BUFFER_BYTE_SIZE		(16*4096)
#define MIN_GLOBAL_CONSTANT_BUFFER_BYTE_SIZE		128

// !!! These offsets must match the cbuffer register definitions in Common.usf !!!
enum ED3D11ShaderOffsetBuffer
{
	/** Default constant buffer. */
	GLOBAL_CONSTANT_BUFFER_INDEX = 0,
	MAX_CONSTANT_BUFFER_SLOTS
};

/**
 * A D3D constant buffer
 */
class FD3D11ConstantBuffer : public FRenderResource, public FRefCountedObject
{
public:
	// New circular buffer system for faster constant uploads.  Avoids CopyResource and speeds things up considerably
	FD3D11ConstantBuffer(FD3D11DynamicRHI* InD3DRHI);
	virtual ~FD3D11ConstantBuffer();

	// FRenderResource interface.
	virtual void	InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void	ReleaseRHI() override;

	static inline constexpr uint32 GetMaxSize() { return (uint32)Align(MAX_GLOBAL_CONSTANT_BUFFER_BYTE_SIZE, 16); }

	/**
	* Updates a variable in the constant buffer.
	* @param Data - The data to copy into the constant buffer
	* @param Offset - The offset in the constant buffer to place the data at
	* @param InSize - The size of the data being copied
	*/
	void UpdateConstant(const uint8* Data, uint16 Offset, uint16 InSize)
	{
		// Check that the data we are shadowing fits in the allocated shadow data
		check((uint32)Offset + (uint32)InSize <= GetMaxSize());
		FMemory::Memcpy(ShadowData+Offset, Data, InSize);
		CurrentUpdateSize = FMath::Max((uint32)(Offset + InSize), CurrentUpdateSize);
	}

protected:
	FD3D11DynamicRHI* D3DRHI;
	uint8*	ShadowData = nullptr;

	/** Size of all constants that has been updated since the last call to Commit. */
	uint32	CurrentUpdateSize = 0;
};

DECLARE_CYCLE_STAT_EXTERN(TEXT("Global Constant buffer update time"),STAT_D3D11GlobalConstantBufferUpdateTime,STATGROUP_D3D11RHI, );

