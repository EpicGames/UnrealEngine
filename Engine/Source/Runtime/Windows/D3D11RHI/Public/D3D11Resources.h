// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Resources.h: D3D resource RHI definitions.
=============================================================================*/

#pragma once

#include "BoundShaderStateCache.h"
#include "ShaderCore.h"
#include "Windows/D3D11ThirdParty.h"

class FD3D11DynamicRHI;
typedef ID3D11DeviceContext FD3D11DeviceContext;

template <>
struct TTypeTraits<D3D11_INPUT_ELEMENT_DESC> : public TTypeTraitsBase<D3D11_INPUT_ELEMENT_DESC>
{
	enum { IsBytewiseComparable = true };
};

/** Convenience typedef: preallocated array of D3D11 input element descriptions. */
typedef TArray<D3D11_INPUT_ELEMENT_DESC,TFixedAllocator<MaxVertexElementCount> > FD3D11VertexElements;

/** This represents a vertex declaration that hasn't been combined with a specific shader to create a bound shader. */
class FD3D11VertexDeclaration : public FRHIVertexDeclaration
{
public:
	/** Elements of the vertex declaration. */
	FD3D11VertexElements VertexElements;

	uint16 StreamStrides[MaxVertexElementCount];

	/** Initialization constructor. */
	explicit FD3D11VertexDeclaration(const FD3D11VertexElements& InElements, const uint16* InStrides)
		: VertexElements(InElements)
	{
		FMemory::Memcpy(StreamStrides, InStrides, sizeof(StreamStrides));
	}
};

struct FD3D11ShaderData
{
	TArray<FShaderCodeVendorExtension>	VendorExtensions;
	bool								bShaderNeedsGlobalConstantBuffer;
	bool								bIsSm6Shader;
	uint16								UAVMask;
};

/** This represents a vertex shader that hasn't been combined with a specific declaration to create a bound shader. */
class FD3D11VertexShader : public FRHIVertexShader, public FD3D11ShaderData
{
public:
	enum { StaticFrequency = SF_Vertex };

	/** The vertex shader resource. */
	TRefCountPtr<ID3D11VertexShader> Resource;

	/** The vertex shader's bytecode, with custom data attached. */
	TArray<uint8> Code;

	// TEMP remove with removal of bound shader state
	int32 Offset;
};

class FD3D11GeometryShader : public FRHIGeometryShader, public FD3D11ShaderData
{
public:
	enum { StaticFrequency = SF_Geometry };

	/** The shader resource. */
	TRefCountPtr<ID3D11GeometryShader> Resource;
};

class FD3D11PixelShader : public FRHIPixelShader, public FD3D11ShaderData
{
public:
	enum { StaticFrequency = SF_Pixel };

	/** The shader resource. */
	TRefCountPtr<ID3D11PixelShader> Resource;
};

class FD3D11ComputeShader : public FRHIComputeShader, public FD3D11ShaderData
{
public:
	enum { StaticFrequency = SF_Compute };

	/** The shader resource. */
	TRefCountPtr<ID3D11ComputeShader> Resource;
};

/**
 * Combined shader state and vertex definition for rendering geometry. 
 * Each unique instance consists of a vertex decl, vertex shader, and pixel shader.
 */
class FD3D11BoundShaderState : public FRHIBoundShaderState
{
public:

	FCachedBoundShaderStateLink CacheLink;
	uint16 StreamStrides[MaxVertexElementCount];
	TRefCountPtr<ID3D11InputLayout> InputLayout;
	TRefCountPtr<ID3D11VertexShader> VertexShader;
	TRefCountPtr<ID3D11PixelShader> PixelShader;
	TRefCountPtr<ID3D11GeometryShader> GeometryShader;

	bool bShaderNeedsGlobalConstantBuffer[SF_NumStandardFrequencies];


	/** Initialization constructor. */
	FD3D11BoundShaderState(
		FRHIVertexDeclaration* InVertexDeclarationRHI,
		FRHIVertexShader* InVertexShaderRHI,
		FRHIPixelShader* InPixelShaderRHI,
		FRHIGeometryShader* InGeometryShaderRHI,
		ID3D11Device* Direct3DDevice
		);

	~FD3D11BoundShaderState();

	/**
	 * Get the shader for the given frequency.
	 */
	inline FD3D11VertexShader*   GetVertexShader() const   { return (FD3D11VertexShader*)CacheLink.GetVertexShader(); }
	inline FD3D11PixelShader*    GetPixelShader() const    { return (FD3D11PixelShader*)CacheLink.GetPixelShader(); }
	inline FD3D11GeometryShader* GetGeometryShader() const { return (FD3D11GeometryShader*)CacheLink.GetGeometryShader(); }
};

/** The base class of resources that may be bound as shader resources. */
class FD3D11ViewableResource
{
public:
	~FD3D11ViewableResource()
	{
		checkf(!HasLinkedViews(), TEXT("All linked views must have been removed before the underlying resource can be deleted."));
	}

	bool HasLinkedViews() const
	{
		return LinkedViews != nullptr;
	}

	void UpdateLinkedViews();

private:
	friend class FD3D11ShaderResourceView;
	friend class FD3D11UnorderedAccessView;
	class FD3D11View* LinkedViews = nullptr;
};

/** Texture base class. */
class FD3D11Texture final : public FRHITexture, public FD3D11ViewableResource
{
public:
	FD3D11Texture(const FRHITextureCreateDesc& InDesc);
	void FinalizeCreation(
		ID3D11Resource* InResource,
		ID3D11ShaderResourceView* InShaderResourceView,
		int32 InRTVArraySize,
		bool bInCreatedRTVsPerSlice,
		TConstArrayView<TRefCountPtr<ID3D11RenderTargetView>> InRenderTargetViews,
		TConstArrayView<TRefCountPtr<ID3D11DepthStencilView>> InDepthStencilViews
	);

	FD3D11Texture(
		const FRHITextureCreateDesc& InDesc,
		ID3D11Resource* InResource,
		ID3D11ShaderResourceView* InShaderResourceView,
		int32 InRTVArraySize,
		bool bInCreatedRTVsPerSlice,
		TConstArrayView<TRefCountPtr<ID3D11RenderTargetView>> InRenderTargetViews,
		TConstArrayView<TRefCountPtr<ID3D11DepthStencilView>> InDepthStencilViews)
		: FD3D11Texture(InDesc)
	{
		FinalizeCreation(InResource, InShaderResourceView, InRTVArraySize, bInCreatedRTVsPerSlice, InRenderTargetViews, InDepthStencilViews);
	}

	enum EAliasResourceParam { CreateAlias };
	D3D11RHI_API explicit FD3D11Texture(FD3D11Texture const& Other, const FString& Name, EAliasResourceParam);
	D3D11RHI_API void AliasResource(FD3D11Texture const& Other);

	D3D11RHI_API virtual ~FD3D11Texture();

	inline uint64 GetMemorySize() const
	{
		return RHICalcTexturePlatformSize(GetDesc()).Size;
	}

	// Accessors.
	inline ID3D11Resource* GetResource() const { return Resource; }
	inline ID3D11ShaderResourceView* GetShaderResourceView() const { return ShaderResourceView; }

	inline bool IsCubemap() const
	{
		FRHITextureDesc const& Desc = GetDesc();
		return Desc.Dimension == ETextureDimension::TextureCube || Desc.Dimension == ETextureDimension::TextureCubeArray;
	}

	inline ID3D11Texture2D* GetD3D11Texture2D() const
	{
		check(Resource);
		check(GetDesc().Dimension == ETextureDimension::Texture2D
		   || GetDesc().Dimension == ETextureDimension::Texture2DArray
		   || GetDesc().Dimension == ETextureDimension::TextureCube
		   || GetDesc().Dimension == ETextureDimension::TextureCubeArray);

		return static_cast<ID3D11Texture2D*>(Resource.GetReference());
	}

	inline ID3D11Texture3D* GetD3D11Texture3D() const
	{
		check(Resource);
		check(GetDesc().Dimension == ETextureDimension::Texture3D);

		return static_cast<ID3D11Texture3D*>(Resource.GetReference());
	}

	inline bool IsTexture3D() const
	{
		return GetDesc().Dimension == ETextureDimension::Texture3D;
	}

	virtual inline void* GetNativeResource() const override
	{
		return GetResource();
	}

	virtual inline void* GetNativeShaderResourceView() const override
	{
		return GetShaderResourceView();
	}

	virtual inline void* GetTextureBaseRHI() override
	{
		return this;
	}

	inline void SetIHVResourceHandle(void* InHandle)
	{
		IHVResourceHandle = InHandle;
	}

	inline void* GetIHVResourceHandle() const
	{
		return IHVResourceHandle;
	}

	/** 
	 * Get the render target view for the specified mip and array slice.
	 * An array slice of -1 is used to indicate that no array slice should be required. 
	 */
	inline ID3D11RenderTargetView* GetRenderTargetView(int32 MipIndex, int32 ArraySliceIndex) const
	{
		int32 ArrayIndex = MipIndex;

		if (bCreatedRTVsPerSlice)
		{
			check(ArraySliceIndex >= 0);
			ArrayIndex = MipIndex * RTVArraySize + ArraySliceIndex;
		}
		else 
		{
			// Catch attempts to use a specific slice without having created the texture to support it
			check(ArraySliceIndex == -1 || ArraySliceIndex == 0);
		}

		if ((uint32)ArrayIndex < (uint32)RenderTargetViews.Num())
		{
			return RenderTargetViews[ArrayIndex];
		}
		return 0;
	}

	inline ID3D11DepthStencilView* GetDepthStencilView(FExclusiveDepthStencil AccessType) const
	{ 
		return DepthStencilViews[AccessType.GetIndex()]; 
	}

#if RHI_ENABLE_RESOURCE_INFO
	virtual bool GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const override
	{
		OutResourceInfo = FRHIResourceInfo{};
		OutResourceInfo.Name = GetName();
		OutResourceInfo.Type = GetType();
		OutResourceInfo.VRamAllocation.AllocationSize = GetMemorySize();
		return true;
	}
#endif

	/**
	* Locks one of the texture's mip-maps.
	* @return A pointer to the specified texture data.
	*/
	FRHILockTextureResult Lock(FD3D11DynamicRHI* D3DRHI, const FRHILockTextureArgs& Arguments, bool bForceLockDeferred);

	/** Unlocks a previously locked mip-map. */
	void Unlock(FD3D11DynamicRHI* D3DRHI, const FRHILockTextureArgs& Arguments);

private:
	//Resource handle for use by IHVs for SLI and other purposes.
	void* IHVResourceHandle = nullptr;

	/** The texture resource. */
	TRefCountPtr<ID3D11Resource> Resource;

	/** A shader resource view of the texture. */
	TRefCountPtr<ID3D11ShaderResourceView> ShaderResourceView;
	
	/** A render targetable view of the texture. */
	TArray<TRefCountPtr<ID3D11RenderTargetView> > RenderTargetViews;

	/** A depth-stencil targetable view of the texture. */
	TRefCountPtr<ID3D11DepthStencilView> DepthStencilViews[FExclusiveDepthStencil::MaxIndex];

	int32 RTVArraySize = 0;

	uint8 bCreatedRTVsPerSlice : 1;
	uint8 bAlias : 1;
};

class FD3D11RenderQuery
{
private:
	TRefCountPtr<ID3D11Query> Resource;

	// Location the result is written to.
	uint64* Target = nullptr;

	// Linked list pointers. Used to build a list of "active" queries, i.e. queries that need data to be polled from the GPU.
	FD3D11RenderQuery** Prev = nullptr;
	FD3D11RenderQuery* Next = nullptr;

public:
	uint8 TOPCounter = 0;
	uint8 BOPCounter = 0;
	std::atomic<uint8> LastCachedBOPCounter = 0;

	enum class EType : uint8
	{
		Timestamp,
		Occlusion,
		Profiler,
		ProfilerEndFrame
	} const Type;

	FD3D11RenderQuery(EType Type);
	~FD3D11RenderQuery();

	bool CacheResult(class FD3D11DynamicRHI& RHI, bool bWait);

	void Begin(ID3D11DeviceContext* Context);
	void End(ID3D11DeviceContext* Context, uint64* Target);

	bool IsLinked() const { return Prev != nullptr; }

private:
	void Link();
	void Unlink();
};

/** D3D11 render query */
class FD3D11RenderQuery_RHI : public FRHIRenderQuery, public FD3D11RenderQuery
{
public:
	uint64 Result = 0;

	FD3D11RenderQuery_RHI(EType Type)
		: FD3D11RenderQuery(Type)
	{}
};

/** Forward declare the constants ring buffer. */
class FD3D11ConstantsRingBuffer;

/** A ring allocation from the constants ring buffer. */
struct FRingAllocation
{
	ID3D11Buffer* Buffer;
	void* DataPtr;
	uint32 Offset;
	uint32 Size;

	FRingAllocation() : Buffer(NULL) {}
	inline bool IsValid() const { return Buffer != NULL; }
};

/** Uniform buffer resource class. */
class FD3D11UniformBuffer : public FRHIUniformBuffer
{
public:

	/** The D3D11 constant buffer resource */
	TRefCountPtr<ID3D11Buffer> Resource;

	/** Allocation in the constants ring buffer if applicable. */
	FRingAllocation RingAllocation;

	/** Initialization constructor. */
	FD3D11UniformBuffer(class FD3D11DynamicRHI* InD3D11RHI, const FRHIUniformBufferLayout* InLayout, ID3D11Buffer* InResource,const FRingAllocation& InRingAllocation, bool bInAllocatedFromPool)
	: FRHIUniformBuffer(InLayout)
	, Resource(InResource)
	, RingAllocation(InRingAllocation)
	, D3D11RHI(InD3D11RHI)
	, bAllocatedFromPool(bInAllocatedFromPool)
	{}

	virtual ~FD3D11UniformBuffer();

	// Provides public non-const access to ResourceTable.
	// @todo refactor uniform buffers to perform updates as a member function, so this isn't necessary.
	TArray<TRefCountPtr<FRHIResource>>& GetResourceTable() { return ResourceTable; }

private:
	class FD3D11DynamicRHI* D3D11RHI;
	bool bAllocatedFromPool;
};

/** Buffer resource class. */
class FD3D11Buffer : public FRHIBuffer, public FD3D11ViewableResource
{
public:

	TRefCountPtr<ID3D11Buffer> Resource;

	FD3D11Buffer(ID3D11Buffer* InResource, const FRHIBufferCreateDesc& InCreateDesc)
		: FRHIBuffer(InCreateDesc)
		, Resource(InResource)
	{}

	// FRHIResource overrides
#if RHI_ENABLE_RESOURCE_INFO
	bool GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const override
	{
		OutResourceInfo = FRHIResourceInfo{};
		OutResourceInfo.Name = GetName();
		OutResourceInfo.Type = GetType();
		OutResourceInfo.VRamAllocation.AllocationSize = GetSize();
		return true;
	}
#endif

	virtual ~FD3D11Buffer();

	void TakeOwnership(FD3D11Buffer& Other);
	void ReleaseOwnership();

	ID3D11Resource* GetResource() const
	{
		return Resource.GetReference();
	}
};

class FD3D11StagingBuffer final : public FRHIStagingBuffer
{
	friend class FD3D11DynamicRHI;
public:
	FD3D11StagingBuffer()
		: FRHIStagingBuffer()
	{}

	~FD3D11StagingBuffer() override;

	void* Lock(uint32 Offset, uint32 NumBytes) override;
	void Unlock() override;
	uint64 GetGPUSizeBytes() const override { return ShadowBufferSize; }

private:
	FD3D11DeviceContext* Context;
	TRefCountPtr<ID3D11Buffer> StagedRead;
	uint32 ShadowBufferSize;
};

class FD3D11Sync
{
	// The graph event to trigger when the sync is resolved.
	FGraphEventRef Event;

	// The D3D11 API query object to poll / wait on.
	TRefCountPtr<ID3D11Query> Query;

	FD3D11Sync() = default;
	FD3D11Sync(FGraphEventRef&& Event, TRefCountPtr<ID3D11Query>&& Query)
		: Event(MoveTemp(Event))
		, Query(MoveTemp(Query))
	{
	}

	static TSpscQueue<FD3D11Sync> ActiveSyncs;

public:
	static void EnqueueSync(FGraphEventRef Event);

	static void PollSyncsUntil(FGraphEvent* Target);

	static void PollSyncs()
	{
		PollSyncsUntil(nullptr);
	}
};

class FD3D11GPUFence final : public FRHIGPUFence
{
private:
	FGraphEventRef Event;

public:
	FD3D11GPUFence(FName InName);

	void Clear() override;
	bool Poll() const override;
	void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const override;

	void WriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList);
};

namespace D3D11BufferStats
{
	void UpdateUniformBufferStats(ID3D11Buffer* Buffer, int64 BufferSize, bool bAllocating);
	void UpdateBufferStats(FD3D11Buffer& Buffer, bool bAllocating);
}

class FD3D11View : public TIntrusiveLinkedList<FD3D11View>
{
public:
	virtual ~FD3D11View()
	{
		Unlink();
	}

	virtual void UpdateView() = 0;
};

/** Shader resource view class. */
class FD3D11ShaderResourceView final : public FRHIShaderResourceView, public FD3D11View
{
public:
	TRefCountPtr<ID3D11ShaderResourceView> View;

	FD3D11ShaderResourceView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc);
	FD3D11ViewableResource* GetBaseResource() const;

	virtual void UpdateView() override;
};

/** Unordered access view class. */
class FD3D11UnorderedAccessView final : public FRHIUnorderedAccessView, public FD3D11View
{
public:
	TRefCountPtr<ID3D11UnorderedAccessView> View;

	FD3D11UnorderedAccessView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc);
	FD3D11ViewableResource* GetBaseResource() const;

	virtual void UpdateView() override;
};

template<class T>
struct TD3D11ResourceTraits
{
};
template<>
struct TD3D11ResourceTraits<FRHIVertexDeclaration>
{
	typedef FD3D11VertexDeclaration TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIVertexShader>
{
	typedef FD3D11VertexShader TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIGeometryShader>
{
	typedef FD3D11GeometryShader TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIPixelShader>
{
	typedef FD3D11PixelShader TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIComputeShader>
{
	typedef FD3D11ComputeShader TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIBoundShaderState>
{
	typedef FD3D11BoundShaderState TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIRenderQuery>
{
	typedef FD3D11RenderQuery_RHI TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIUniformBuffer>
{
	typedef FD3D11UniformBuffer TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIBuffer>
{
	typedef FD3D11Buffer TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIStagingBuffer>
{
	typedef FD3D11StagingBuffer TConcreteType;
};
// @todo-staging Implement D3D11 fences.
template<>
struct TD3D11ResourceTraits<FRHIGPUFence>
{
	typedef FD3D11GPUFence TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIShaderResourceView>
{
	typedef FD3D11ShaderResourceView TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIUnorderedAccessView>
{
	typedef FD3D11UnorderedAccessView TConcreteType;
};

