// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11State.h: D3D state definitions.
=============================================================================*/

#pragma once

#include "D3D11Resources.h"

class FD3D11SamplerState : public FRHISamplerState
{
public:

	TRefCountPtr<ID3D11SamplerState> Resource;
};

class FD3D11RasterizerState : public FRHIRasterizerState
{
public:

	TRefCountPtr<ID3D11RasterizerState> Resource;
};

class FD3D11DepthStencilState : public FRHIDepthStencilState
{
public:
	
	TRefCountPtr<ID3D11DepthStencilState> Resource;

	/* Describes the read/write state of the separate depth and stencil components of the DSV. */
	FExclusiveDepthStencil AccessType;
};

class FD3D11BlendState : public FRHIBlendState
{
public:

	TRefCountPtr<ID3D11BlendState> Resource;
};

template<>
struct TD3D11ResourceTraits<FRHISamplerState>
{
	typedef FD3D11SamplerState TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIRasterizerState>
{
	typedef FD3D11RasterizerState TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIDepthStencilState>
{
	typedef FD3D11DepthStencilState TConcreteType;
};
template<>
struct TD3D11ResourceTraits<FRHIBlendState>
{
	typedef FD3D11BlendState TConcreteType;
};
