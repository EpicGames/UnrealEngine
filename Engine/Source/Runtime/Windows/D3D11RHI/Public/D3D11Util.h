// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Util.h: D3D RHI utility definitions.
=============================================================================*/

#pragma once

#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "Templates/RefCounting.h"
#include "Windows/D3D11ThirdParty.h"
#include "Containers/StringFwd.h"

class FD3D11Buffer;
class FD3D11Texture;
class FD3D11UnorderedAccessView;
class FRHIBuffer;
class FRHITexture;

#define D3D11RHI_IMMEDIATE_CONTEXT	(GD3D11RHI->GetDeviceContext())
#define D3D11RHI_DEVICE				(GD3D11RHI->GetDevice())

void SetD3D11ObjectName(ID3D11DeviceChild* Object, FAnsiStringView Name);
void SetD3D11ObjectName(ID3D11DeviceChild* Object, FStringView Name);
void SetD3D11ResourceName(FD3D11Texture* Texture, FStringView Name);
void SetD3D11ResourceName(FD3D11Buffer* Buffer, FStringView Name);
void SetD3D11ResourceName(FD3D11UnorderedAccessView* UAV, FStringView Name);

/**
 * Checks that the given result isn't a failure.  If it is, the application does not exit and only logs an appropriate error message.
 * @param	Result - The result code to check.
 * @param	Code - The code which yielded the result.
 * @param	Filename - The filename of the source file containing Code.
 * @param	Line - The line number of Code within Filename.
 */
extern D3D11RHI_API void VerifyD3D11ResultNoExit(HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device);

/**
 * Checks that the given result isn't a failure.  If it is, the application exits with an appropriate error message.
 * @param	Result - The result code to check.
 * @param	Code - The code which yielded the result.
 * @param	Filename - The filename of the source file containing Code.
 * @param	Line - The line number of Code within Filename.
 */
extern D3D11RHI_API void VerifyD3D11Result(HRESULT Result,const ANSICHAR* Code,const ANSICHAR* Filename,uint32 Line, ID3D11Device* Device);

/**
 * Checks that the given result isn't a failure.  If it is, the application exits with an appropriate error message.
 * @param	Shader - The shader we are trying to create.
 * @param	Result - The result code to check.
 * @param	Code - The code which yielded the result.
 * @param	Filename - The filename of the source file containing Code.
 * @param	Line - The line number of Code within Filename.
 * @param	Device - The D3D device used to create the shader.
 */
extern D3D11RHI_API void VerifyD3D11ShaderResult(class FRHIShader* Shader, HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device);

/**
* Checks that the given result isn't a failure.  If it is, the application exits with an appropriate error message.
* @param	Result - The result code to check
* @param	Code - The code which yielded the result.
* @param	Filename - The filename of the source file containing Code.
* @param	Line - The line number of Code within Filename.	
*/
extern D3D11RHI_API void VerifyD3D11CreateTextureResult(HRESULT D3DResult, int32 UEFormat,const ANSICHAR* Code,const ANSICHAR* Filename,uint32 Line,
										 uint32 SizeX,uint32 SizeY,uint32 SizeZ,uint8 D3DFormat,uint32 NumMips,uint32 Flags, D3D11_USAGE Usage,
										 uint32 CPUAccessFlags, uint32 MiscFlags, uint32 SampleCount, uint32 SampleQuality,
										 const void* SubResPtr, uint32 SubResPitch, uint32 SubResSlicePitch, ID3D11Device* Device, const TCHAR* DebugName);

struct FD3D11ResizeViewportState
{
	uint32 SizeX;
	uint32 SizeY;
	DXGI_FORMAT Format;
	bool bIsFullscreen;
};

extern D3D11RHI_API void VerifyD3D11ResizeViewportResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line,
	const FD3D11ResizeViewportState& OldState, const FD3D11ResizeViewportState& NewState, ID3D11Device* Device);

extern D3D11RHI_API void VerifyD3D11CreateViewResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device, FRHITexture* Texture, const D3D11_UNORDERED_ACCESS_VIEW_DESC& Desc);
extern D3D11RHI_API void VerifyD3D11CreateViewResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D11Device* Device, FRHIBuffer* Buffer, const D3D11_UNORDERED_ACCESS_VIEW_DESC& Desc);

/**
 * A macro for using VERIFYD3D11RESULT that automatically passes in the code and filename/line.
 */
#define VERIFYD3D11RESULT_EX(x, Device)	{HRESULT hr = x; if (FAILED(hr)) { VerifyD3D11Result(hr,#x,__FILE__,__LINE__, Device); }}
#define VERIFYD3D11RESULT(x)			{HRESULT hr = x; if (FAILED(hr)) { VerifyD3D11Result(hr,#x,__FILE__,__LINE__, 0); }}
#define VERIFYD3D11RESULT_NOEXIT(x)		{HRESULT hr = x; if (FAILED(hr)) { VerifyD3D11ResultNoExit(hr,#x,__FILE__,__LINE__, 0); }}
#define VERIFYD3D11SHADERRESULT(Result, Shader, Device) {HRESULT hr = (Result); if (FAILED(hr)) { VerifyD3D11ShaderResult(Shader, hr, #Result,__FILE__,__LINE__, Device); }}
#define VERIFYD3D11RESULT_NOEXIT(x)		{HRESULT hr = x; if (FAILED(hr)) { VerifyD3D11ResultNoExit(hr,#x,__FILE__,__LINE__, 0); }}
#define VERIFYD3D11CREATETEXTURERESULT(x,UEFormat,SizeX,SizeY,SizeZ,Format,NumMips,Flags,Usage,CPUAccessFlags,MiscFlags,SampleCount,SampleQuality,SubResPtr,SubResPitch,SubResSlicePitch,Device,DebugName) {HRESULT hr = x; if (FAILED(hr)) { VerifyD3D11CreateTextureResult(hr, UEFormat,#x,__FILE__,__LINE__,SizeX,SizeY,SizeZ,Format,NumMips,Flags,Usage,CPUAccessFlags,MiscFlags,SampleCount,SampleQuality,SubResPtr,SubResPitch,SubResSlicePitch,Device,DebugName); }}
#define VERIFYD3D11RESIZEVIEWPORTRESULT(x, OldState, NewState, Device) { HRESULT hr = x; if (FAILED(hr)) { VerifyD3D11ResizeViewportResult(hr, #x, __FILE__, __LINE__, OldState, NewState, Device); }}
#define VERIFYD3D11CREATEVIEWRESULT(x, Device, Resource, Desc) {HRESULT hr = x; if (FAILED(hr)) { VerifyD3D11CreateViewResult(hr, #x, __FILE__, __LINE__, Device, Resource, Desc); }}

/**
 * Checks that a COM object has the expected number of references.
 */
extern D3D11RHI_API void VerifyComRefCount(IUnknown* Object,int32 ExpectedRefs,const TCHAR* Code,const TCHAR* Filename,int32 Line);
#define checkComRefCount(Obj,ExpectedRefs) VerifyComRefCount(Obj,ExpectedRefs,TEXT(#Obj),TEXT(__FILE__),__LINE__)

/** Returns a string for the provided error code, can include device removed information if the device is provided. */
FString GetD3D11ErrorString(HRESULT ErrorCode, ID3D11Device* Device);

/**
* Convert from ECubeFace to D3DCUBEMAP_FACES type
* @param Face - ECubeFace type to convert
* @return D3D cube face enum value
*/
inline uint32 GetD3D11CubeFace(ECubeFace Face)
{
	switch(Face)
	{
	case CubeFace_PosX:
	default:
		return 0;//D3DCUBEMAP_FACE_POSITIVE_X;
	case CubeFace_NegX:
		return 1;//D3DCUBEMAP_FACE_NEGATIVE_X;
	case CubeFace_PosY:
		return 2;//D3DCUBEMAP_FACE_POSITIVE_Y;
	case CubeFace_NegY:
		return 3;//D3DCUBEMAP_FACE_NEGATIVE_Y;
	case CubeFace_PosZ:
		return 4;//D3DCUBEMAP_FACE_POSITIVE_Z;
	case CubeFace_NegZ:
		return 5;//D3DCUBEMAP_FACE_NEGATIVE_Z;
	};
}

/**
 * Keeps track of Locks for D3D11 objects
 */
class FD3D11LockedKey
{
public:
	ID3D11Resource* SourceObject = nullptr;
	uint32 Subresource = 0;

	FD3D11LockedKey() = default;

	FD3D11LockedKey(ID3D11Resource* InSource, uint32 InSubresource = 0)
		: SourceObject(InSource)
		, Subresource (InSubresource)
	{}

	bool operator == (const FD3D11LockedKey& Other) const
	{
		return SourceObject == Other.SourceObject && Subresource == Other.Subresource;
	}

	bool operator != (const FD3D11LockedKey& Other) const
	{
		return SourceObject != Other.SourceObject || Subresource != Other.Subresource;
	}

	uint32 GetHash() const
	{
		return PointerHash(SourceObject);
	}

	/** Hashing function. */
	friend uint32 GetTypeHash(const FD3D11LockedKey& K)
	{
		return K.GetHash();
	}
};

/** Information about a D3D resource that is currently locked. */
struct FD3D11LockedData
{
	TRefCountPtr<ID3D11Resource> StagingResource;
	uint32 Pitch;
	uint32 DepthPitch;

	// constructor
	FD3D11LockedData()
		: bAllocDataWasUsed(false)
		, bLockDeferred(false)
	{
	}

	// 16 byte alignment for best performance  (can be 30x faster than unaligned)
	void AllocData(uint32 Size)
	{
		Data = (uint8*)FMemory::Malloc(Size, 16);
		bAllocDataWasUsed = true;
	}

	// Some driver might return aligned memory so we don't enforce the alignment
	void SetData(void* InData)
	{
		check(!bAllocDataWasUsed); Data = (uint8*)InData;
	}

	uint8* GetData() const
	{
		return Data;
	}

	// only call if AllocData() was used
	void FreeData()
	{
		check(bAllocDataWasUsed);
		FMemory::Free(Data);
		Data = 0;
	}

private:
	//
	uint8* Data;
	// then FreeData
	bool bAllocDataWasUsed;
public:
	// Whether the lock op is deferred
	bool bLockDeferred;
};

/**
 * Class for retrieving render targets currently bound to the device context.
 */
class FD3D11BoundRenderTargets
{
public:
	/** Initialization constructor: requires the device context. */
	explicit FD3D11BoundRenderTargets(ID3D11DeviceContext* InDeviceContext);

	/** Destructor. */
	~FD3D11BoundRenderTargets();

	/** Accessors. */
	inline int32 GetNumActiveTargets() const { return NumActiveTargets; }
	inline ID3D11RenderTargetView* GetRenderTargetView(int32 TargetIndex) { return RenderTargetViews[TargetIndex]; }
	inline ID3D11DepthStencilView* GetDepthStencilView() { return DepthStencilView; }

private:
	/** Active render target views. */
	ID3D11RenderTargetView* RenderTargetViews[MaxSimultaneousRenderTargets];
	/** Active depth stencil view. */
	ID3D11DepthStencilView* DepthStencilView;
	/** The number of active render targets. */
	int32 NumActiveTargets;
};

struct FD3D11RHIGenericCommandString
{
	static const TCHAR* TStr() { return TEXT("FD3D11RHIGenericCommand"); }
};

template <
	typename JobType,
	typename = TEnableIf<
	std::is_same_v<JobType, TFunction<void()>> ||
	std::is_same_v<JobType, TFunction<void()>&>>>
class TD3D11RHIGenericCommand final : public FRHICommand<TD3D11RHIGenericCommand<JobType>, FD3D11RHIGenericCommandString>
{
public:
	// InRHIJob is supposed to be called on RHIT (don't capture things that can become outdated here)
	TD3D11RHIGenericCommand(JobType&& InRHIJob)
		: RHIJob(Forward<JobType>(InRHIJob))
	{}

	void Execute(FRHICommandListBase& RHICmdList)
	{
		RHIJob();
	}

private:
	JobType RHIJob;
};

template<typename JobType>
inline void RunOnRHIThread(JobType&& InRHIJob)
{
	//check(IsInRenderingThread());
	typedef TD3D11RHIGenericCommand<JobType> CmdType;
	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();
	new (RHICmdList.AllocCommand<CmdType>()) CmdType(Forward<JobType>(InRHIJob));
}

inline bool ShouldNotEnqueueRHICommand()
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();
	return RHICmdList.Bypass() || (IsRunningRHIInSeparateThread() && IsInRHIThread()) || (!IsRunningRHIInSeparateThread() && IsInRenderingThread());
}
