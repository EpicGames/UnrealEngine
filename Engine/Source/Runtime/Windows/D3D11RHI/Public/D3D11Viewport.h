// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D11Viewport.h: D3D viewport RHI definitions.
=============================================================================*/

#pragma once

#include "D3D11Resources.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHIResources.h"
#include "Windows/D3D11ThirdParty.h"
#include "DXGIUtilities.h"

class FD3D11Viewport : public FRHIViewport
{
public:
	enum ED3DViewportValidFlags : uint32
	{
		VIEWPORT_INVALID = 0x1,
		VIEWPORT_FULLSCREEN_LOST = 0x2,

	};

	D3D11RHI_API FD3D11Viewport(class FD3D11DynamicRHI* InD3DRHI, HWND InWindowHandle, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat InPreferredPixelFormat);
	D3D11RHI_API ~FD3D11Viewport();

	D3D11RHI_API virtual void Resize(uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat PreferredPixelFormat);

	/**
	 * If the swap chain has been invalidated by DXGI, resets the swap chain to the expected state; otherwise, does nothing.
	 * Called once/frame by the game thread on all viewports.
	 * @param bIgnoreFocus - Whether the reset should happen regardless of whether the window is focused.
	 */
	D3D11RHI_API void ConditionalResetSwapChain(bool bIgnoreFocus);

	/**
	 * Called whenever the Viewport is moved to see if it has moved between HDR or LDR monitors
	 */
	D3D11RHI_API void CheckHDRMonitorStatus();


	/** Presents the swap chain. 
	 * Returns true if Present was done by Engine.
	 */
	D3D11RHI_API bool Present(IRHICommandContext& RHICmdContext, bool bLockToVsync);

	// Accessors.
	FIntPoint GetSizeXY() const { return FIntPoint(SizeX, SizeY); }
	FD3D11Texture* GetBackBuffer() const { return BackBuffer; }

	virtual void WaitForFrameEventCompletion() override;
	virtual void IssueFrameEvent() override;

	IDXGISwapChain* GetSwapChain() const { return SwapChain; }

	virtual void* GetNativeSwapChain() const override;
	virtual void* GetNativeBackBufferTexture() const override;
	virtual void* GetNativeBackBufferRT() const override;

	virtual void SetCustomPresent(FRHICustomPresent* InCustomPresent) override
	{
		CustomPresent = InCustomPresent;
	}
	virtual FRHICustomPresent* GetCustomPresent() const { return CustomPresent; }

	virtual void* GetNativeWindow(void** AddParam = nullptr) const override { return (void*)WindowHandle; }
	static D3D11RHI_API FD3D11Texture* GetSwapChainSurface(FD3D11DynamicRHI* D3DRHI, EPixelFormat PixelFormat, uint32 SizeX, uint32 SizeY, IDXGISwapChain* SwapChain);

	static DXGI_FORMAT GetRenderTargetFormat(EPixelFormat PixelFormat)
	{
		return UE::DXGIUtilities::GetSwapChainFormat(PixelFormat);
	}

protected:

	D3D11RHI_API void ResetSwapChainInternal(bool bIgnoreFocus);

	/** Gets the swap chain flags */
	D3D11RHI_API uint32 GetSwapChainFlags();

	/** Presents the frame synchronizing with DWM. */
	D3D11RHI_API void PresentWithVsyncDWM(IRHICommandContext& RHICmdContext);

	/**
	 * Presents the swap chain checking the return result. 
	 * Returns true if Present was done by Engine.
	 */
	D3D11RHI_API bool PresentChecked(IRHICommandContext& RHICmdContext, int32 SyncInterval);

	/** Enable HDR meta data transmission and set the necessary color space. */
	void EnableHDR();

	/** Disable HDR meta data transmission and set the necessary color space. */
	void ShutdownHDR();


	FD3D11DynamicRHI* const D3DRHI;
	HWND const WindowHandle;

	EPixelFormat PixelFormat;
	uint32 SizeX;
	uint32 SizeY;
	bool bIsFullscreen;

	uint64 LastFlipTime = 0;
	uint64 LastFrameComplete = 0;
	uint64 LastCompleteTime = 0;
	int32 SyncCounter = 0;
	bool bSyncedLastFrame = false;
	uint32 MaximumFrameLatency = 3;
	uint32 BackBufferCount = 0;
	uint32 PresentFailCount = 0;
	TAtomic<uint32> ValidState = 0;
	EDisplayColorGamut DisplayColorGamut = EDisplayColorGamut::sRGB_D65;
	EDisplayOutputFormat DisplayOutputFormat = EDisplayOutputFormat::SDR_sRGB;
	bool bAllowTearing = false;
	bool bResizeInProgress = false;

	static D3D11RHI_API uint32 GSwapChainFlags;

	TRefCountPtr<IDXGISwapChain> SwapChain;
	TRefCountPtr<FD3D11Texture> BackBuffer;

	// Support for selecting non-default output for display in fullscreen exclusive
	TRefCountPtr<IDXGIOutput>	ForcedFullscreenOutput;
	bool						bForcedFullscreenDisplay;

	// Whether to create swap chain and use swap chain's back buffer surface, 
	// or don't create swap chain and create an off-screen back buffer surface.
	// Currently used for pixel streaming plugin "windowless" mode to run in the cloud without on screen display.
	bool						bNeedSwapChain;

	/** An event used to track the GPU's progress. */
	FGraphEventRef FrameSyncEvent;

	FCustomPresentRHIRef CustomPresent;

	D3D11RHI_API DXGI_MODE_DESC SetupDXGI_MODE_DESC() const;

};

template<>
struct TD3D11ResourceTraits<FRHIViewport>
{
	typedef FD3D11Viewport TConcreteType;
};
