// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHI.h"

#if PLATFORM_WINDOWS
	#include "Windows/D3D11ThirdParty.h"
#else
	#include "D3D11ThirdParty.h"
#endif

typedef ID3D11DeviceContext FD3D11DeviceContext;
typedef ID3D11Device FD3D11Device;

struct ID3D11DynamicRHI : public FDynamicRHIPSOFallback
{
	virtual ERHIInterfaceType     GetInterfaceType() const override final { return ERHIInterfaceType::D3D11; }

	virtual ID3D11Device*         RHIGetDevice() const = 0;
	virtual ID3D11DeviceContext*  RHIGetDeviceContext() const = 0;
	virtual IDXGIAdapter*         RHIGetAdapter() const = 0;
	virtual IDXGISwapChain*       RHIGetSwapChain(FRHIViewport* InViewport) const = 0;
	virtual DXGI_FORMAT           RHIGetSwapChainFormat(EPixelFormat InFormat) const = 0;

	virtual FTextureRHIRef        RHICreateTexture2DFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* Resource) = 0;
	virtual FTextureRHIRef        RHICreateTexture2DArrayFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* Resource) = 0;
	virtual FTextureRHIRef        RHICreateTextureCubeFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D11Texture2D* Resource) = 0;

	virtual ID3D11Buffer*         RHIGetResource(FRHIBuffer* InBuffer) const = 0;
	virtual ID3D11Resource*       RHIGetResource(FRHITexture* InTexture) const = 0;
	virtual int64                 RHIGetResourceMemorySize(FRHITexture* InTexture) const = 0;

	virtual ID3D11RenderTargetView*   RHIGetRenderTargetView(FRHITexture* InTexture, int32 InMipIndex = 0, int32 InArraySliceIndex = -1) const = 0;
	virtual ID3D11ShaderResourceView* RHIGetShaderResourceView(FRHITexture* InTexture) const = 0;

	virtual void                  RHIVerifyResult(ID3D11Device* Device, HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line) const = 0;

	// Returns the RHI viewport, assuming only a single FRHIViewport instance exists. Otherwise, returns nullptr.
	virtual FRHIViewport* RHIGetSingletonViewport() const = 0;

	UE_DEPRECATED(5.8, "RHIRegisterWork is deprecated, and there is no replacement. Remove calls to this function.")
	void RHIRegisterWork(uint32 NumPrimitives)
	{
	}
};

inline bool IsRHID3D11()
{
	return GDynamicRHI != nullptr && GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D11;
}

inline ID3D11DynamicRHI* GetID3D11DynamicRHI()
{
	check(GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D11);
	return GetDynamicRHI<ID3D11DynamicRHI>();
}
