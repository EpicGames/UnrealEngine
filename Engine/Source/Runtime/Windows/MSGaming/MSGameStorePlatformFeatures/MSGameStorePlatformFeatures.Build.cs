// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatformGroups("Windows")]
public class MSGameStorePlatformFeatures : ModuleRules
{
	public MSGameStorePlatformFeatures( ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"Engine",
				"GRDK",
			});

		PublicDependencyModuleNames.AddRange(new string[]
			{
				"WindowsPlatformFeatures",
			});
	}
}
