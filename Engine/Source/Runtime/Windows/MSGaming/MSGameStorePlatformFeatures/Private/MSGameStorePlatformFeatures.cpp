// Copyright Epic Games, Inc. All Rights Reserved.

#include "MSGameStorePlatformFeatures.h"
#if WITH_GRDK
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <XGame.h>
#include <XAppCapture.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

void FMSGameStorePlatformFeaturesModule::SetScreenshotEnableState(bool bEnabled)
{
	if (bEnabled)
	{
		XAppCaptureEnableRecord();
	}
	else
	{
		XAppCaptureDisableRecord();
	}
}


FString FMSGameStorePlatformFeaturesModule::GetUniqueAppId()
{
	uint32_t TitleId = 0;
	if (SUCCEEDED(XGameGetXboxTitleId(&TitleId)))
	{
		return FString::Printf(TEXT("%X"), TitleId);
	}
	else
	{
		return FString();
	}
}
#endif //WITH_GRDK

IMPLEMENT_MODULE(FMSGameStorePlatformFeaturesModule, MSGameStorePlatformFeatures);
