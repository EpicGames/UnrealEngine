// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsPlatformFeatures.h"

class FMSGameStorePlatformFeaturesModule : public FWindowsPlatformFeaturesModule
{
public:
#if WITH_GRDK
	virtual FString GetUniqueAppId() override;
	virtual void SetScreenshotEnableState(bool bEnabled) override;
#endif
};

