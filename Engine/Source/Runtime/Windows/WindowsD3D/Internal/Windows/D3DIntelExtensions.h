// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef INTEL_EXTENSIONS
#define INTEL_EXTENSIONS 0
#endif

#if INTEL_EXTENSIONS
#include "Windows/WindowsD3D12ThirdParty.h"
#define INTC_IGDEXT_D3D12 1
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <igdext.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

WINDOWSD3D_API INTCExtensionAppInfo1 GetIntelApplicationInfo();
WINDOWSD3D_API void EnableIntelAppDiscovery(uint32 DeviceId);
#endif
