// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Templates/RefCounting.h"

struct ID3D11Device;
struct ID3D12Device;

struct FWindowsGPUInfo
{
	uint32 VendorId = 0;
	uint32 DeviceId = 0;
	uint64 DedicatedVideoMemory = 0;
	FString AdapterName;
};

class FWindowsD3D
{
public:
	static WINDOWSD3D_API FWindowsGPUInfo GetExpectedGPUInfo();
	static WINDOWSD3D_API FWindowsGPUInfo GetGPUInfoByDedicatedMemory();
	static WINDOWSD3D_API uint64 ChooseD3D11Adapter(uint64 DesiredAdapterLuid);
	static WINDOWSD3D_API uint64 ChooseD3D12Adapter(uint64 DesiredAdapterLuid);

	static WINDOWSD3D_API ID3D11Device* FindCachedD3D11AdapterDevice(uint64 AdapterLuid, int32& FeatureLevel);
	static WINDOWSD3D_API void CacheD3D11AdapterDevice(uint64 AdapterLuid, TRefCountPtr<ID3D11Device>&& Device, int32 FeatureLevel);
	static WINDOWSD3D_API ID3D12Device* FindCachedD3D12AdapterDevice(uint64 AdapterLuid);
	static WINDOWSD3D_API void CacheD3D12AdapterDevice(uint64 AdapterLuid, TRefCountPtr<ID3D12Device>&& Device);
	static WINDOWSD3D_API void ReleaseCachedAdapterDevices();
};
