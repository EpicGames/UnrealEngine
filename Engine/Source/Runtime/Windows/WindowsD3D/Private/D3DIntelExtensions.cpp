// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/D3DIntelExtensions.h"

#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"

#if INTEL_EXTENSIONS
INTCExtensionAppInfo1 GetIntelApplicationInfo()
{
	// CVar set to disable workload registration
	static TConsoleVariableData<int32>* CVarDisableEngineAndAppRegistration = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DisableEngineAndAppRegistration"));

	INTCExtensionAppInfo1 AppInfo{};

	if (!(CVarDisableEngineAndAppRegistration && CVarDisableEngineAndAppRegistration->GetValueOnAnyThread() != 0))
	{
		AppInfo.pApplicationName = FApp::HasProjectName() ? FApp::GetProjectName() : TEXT("");
		//AppInfo.ApplicationVersion = FApp::GetBuildVersion();		// Currently no support for version

		AppInfo.pEngineName = TEXT("Unreal Engine");
		AppInfo.EngineVersion.major = FEngineVersion::Current().GetMajor();
		AppInfo.EngineVersion.minor = FEngineVersion::Current().GetMinor();
		AppInfo.EngineVersion.patch = FEngineVersion::Current().GetPatch();
	}

	return AppInfo;
}

void EnableIntelAppDiscovery(uint32 DeviceId)
{
	if (SUCCEEDED(INTC_LoadExtensionsLibrary(false, 0x8086, DeviceId)))
	{
		// Fill in registration information for this workload (App name and Engine name)
		INTCExtensionAppInfo1 AppInfo = GetIntelApplicationInfo();

		// Intel Application Discovery - registering UE5 application info in the driver
		INTC_D3D12_SetApplicationInfo(&AppInfo);
	}
	else
	{
		UE_LOGF(LogWindows, Log, "Failed to load Intel Extensions Library (App Discovery)");
	}
}
#endif // INTEL_EXTENSIONS
