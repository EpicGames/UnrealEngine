// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsD3D.h"

#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Templates/RefCounting.h"
#include "Windows/D3D11ThirdParty.h"
#include "Windows/D3DIntelExtensions.h"
#include "Windows/WindowsD3D12ThirdParty.h"

DEFINE_LOG_CATEGORY_STATIC(LogD3D, Log, All);

static TMap<uint64, TPair<TRefCountPtr<ID3D11Device>, int32>> D3D11AdapterDevices;
static TMap<uint64, TRefCountPtr<ID3D12Device>> D3D12AdapterDevices;

enum class ED3DGpuVendorId : uint32
{
	Unknown = 0xffffffff,
	NotQueried = 0,

	Amd = 0x1002,
	ImgTec = 0x1010,
	Nvidia = 0x10DE,
	Arm = 0x13B5,
	Broadcom = 0x14E4,
	Qualcomm = 0x5143,
	Intel = 0x8086,
	Apple = 0x106B,
	Vivante = 0x7a05,
	VeriSilicon = 0x1EB1,
	SamsungAMD = 0x144D,
	Microsoft = 0x1414,

	Kazan = 0x10003,	// VkVendorId
	Codeplay = 0x10004,	// VkVendorId
	Mesa = 0x10005,	// VkVendorId
};

static ED3DGpuVendorId GetPreferredGpuVendor()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("preferAMD")))
	{
		return ED3DGpuVendorId::Amd;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("preferIntel")))
	{
		return ED3DGpuVendorId::Intel;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("preferNvidia")))
	{
		return ED3DGpuVendorId::Nvidia;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("preferMS")) || FParse::Param(FCommandLine::Get(), TEXT("preferMicrosoft")))
	{
		return ED3DGpuVendorId::Microsoft;
	}

	return ED3DGpuVendorId::Unknown;
}

static ED3DGpuVendorId ConvertToGpuVendorId(uint32 VendorId)
{
	switch ((ED3DGpuVendorId)VendorId)
	{
	case ED3DGpuVendorId::NotQueried:
		return ED3DGpuVendorId::NotQueried;

	case ED3DGpuVendorId::Amd:
	case ED3DGpuVendorId::Kazan:
	case ED3DGpuVendorId::Codeplay:
	case ED3DGpuVendorId::Mesa:
	case ED3DGpuVendorId::ImgTec:
	case ED3DGpuVendorId::Nvidia:
	case ED3DGpuVendorId::Arm:
	case ED3DGpuVendorId::Broadcom:
	case ED3DGpuVendorId::Qualcomm:
	case ED3DGpuVendorId::Intel:
	case ED3DGpuVendorId::Apple:
	case ED3DGpuVendorId::Vivante:
	case ED3DGpuVendorId::VeriSilicon:
	case ED3DGpuVendorId::SamsungAMD:
	case ED3DGpuVendorId::Microsoft:
		return (ED3DGpuVendorId)VendorId;

	case ED3DGpuVendorId::Unknown:
	default:
		return ED3DGpuVendorId::Unknown;
	}
}

static FWindowsGPUInfo GetSpecificGPUInfo(uint64 AdapterLuid)
{
	TRefCountPtr<IDXGIFactory1> DXGIFactory1;
	if (FAILED(CreateDXGIFactory1(IID_PPV_ARGS(DXGIFactory1.GetInitReference()))) || !DXGIFactory1)
	{
		return {};
	}

	DXGI_ADAPTER_DESC BestDesc = {};
	TRefCountPtr<IDXGIAdapter> TempAdapter;
	for (uint32 AdapterIndex = 0; DXGIFactory1->EnumAdapters(AdapterIndex, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
	{
		if (TempAdapter)
		{
			DXGI_ADAPTER_DESC Desc;
			if (SUCCEEDED(TempAdapter->GetDesc(&Desc)))
			{
				if (!FMemory::Memcmp(&AdapterLuid, &Desc.AdapterLuid, sizeof(LUID)))
				{
					BestDesc = Desc;
					break;
				}

				if (AdapterIndex == 0)
				{
					BestDesc = Desc;
				}
			}
		}
	}

	return FWindowsGPUInfo{ BestDesc.VendorId, BestDesc.DeviceId, BestDesc.DedicatedVideoMemory, BestDesc.Description };
}

// Make a best effort to anticipate the GPU adapter that will be chosen.
FWindowsGPUInfo FWindowsD3D::GetExpectedGPUInfo()
{
	// TODO: Respect IHeadMountedDisplayModule::GetGraphicsAdapterLuid without creating a circular reference.
	const uint64 DesiredAdapaterLuid = 0;

	uint64 Luid = ChooseD3D12Adapter(DesiredAdapaterLuid);
	if (!Luid)
	{
		Luid = ChooseD3D11Adapter(DesiredAdapaterLuid);
	}

	return Luid ? GetSpecificGPUInfo(Luid) : GetGPUInfoByDedicatedMemory();
}

FWindowsGPUInfo FWindowsD3D::GetGPUInfoByDedicatedMemory()
{
	TRefCountPtr<IDXGIFactory1> DXGIFactory1;
	if (CreateDXGIFactory1(IID_PPV_ARGS(DXGIFactory1.GetInitReference())) != S_OK || !DXGIFactory1)
	{
		return {};
	}

	DXGI_ADAPTER_DESC BestDesc = {};
	TRefCountPtr<IDXGIAdapter> TempAdapter;
	for (uint32 AdapterIndex = 0; DXGIFactory1->EnumAdapters(AdapterIndex, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
	{
		if (TempAdapter)
		{
			DXGI_ADAPTER_DESC Desc;
			if (SUCCEEDED(TempAdapter->GetDesc(&Desc)))
			{
				if (Desc.DedicatedVideoMemory > BestDesc.DedicatedVideoMemory || AdapterIndex == 0)
				{
					BestDesc = Desc;
				}
			}
		}
	}

	return FWindowsGPUInfo{ BestDesc.VendorId, BestDesc.DeviceId, BestDesc.DedicatedVideoMemory, BestDesc.Description };
}

static HRESULT EnumAdapters(int32 AdapterIndex, DXGI_GPU_PREFERENCE GpuPreference, IDXGIFactory1* DxgiFactory1, IDXGIFactory6* DxgiFactory6, IDXGIAdapter** TempAdapter)
{
	if (!DxgiFactory6 || GpuPreference == DXGI_GPU_PREFERENCE_UNSPECIFIED)
	{
		return DxgiFactory1->EnumAdapters(AdapterIndex, TempAdapter);
	}
	else
	{
		return DxgiFactory6->EnumAdapterByGpuPreference(AdapterIndex, GpuPreference, IID_PPV_ARGS(TempAdapter));
	}
}

/**
 * Attempts to create a D3D12 device for the adapter or find an existing one.
 * If creation is successful, true is returned.
 */
static bool SafeTestD3D12CreateDevice(IDXGIAdapter* Adapter, const DXGI_ADAPTER_DESC& AdapterDesc, bool& bOutUMA)
{
	bOutUMA = false;

	constexpr D3D_FEATURE_LEVEL MinFeatureLevel = D3D_FEATURE_LEVEL_11_0;

	const uint64 Luid = BitCast<uint64>(AdapterDesc.AdapterLuid);
	ID3D12Device* Device = FWindowsD3D::FindCachedD3D12AdapterDevice(Luid);
	if (!Device)
	{
		TRefCountPtr<ID3D12Device> DevicePtr;
		const HRESULT D3D12CreateDeviceResult = D3D12CreateDevice(Adapter, MinFeatureLevel, IID_PPV_ARGS(DevicePtr.GetInitReference()));
		if (FAILED(D3D12CreateDeviceResult))
		{
			UE_LOGF(LogD3D, Log, "D3D12CreateDevice failed with code 0x%08X", static_cast<int32>(D3D12CreateDeviceResult));
			return false;
		}

		Device = DevicePtr.GetReference();
		FWindowsD3D::CacheD3D12AdapterDevice(Luid, MoveTemp(DevicePtr));
	}

	D3D12_FEATURE_DATA_ARCHITECTURE data{};
	bOutUMA = SUCCEEDED(Device->CheckFeatureSupport(D3D12_FEATURE_ARCHITECTURE, &data, sizeof(D3D12_FEATURE_DATA_ARCHITECTURE))) && data.UMA;
	return true;
}

/**
 * Attempts to create a D3D11 device for the adapter using at most MaxFeatureLevel or find an existing one.
 * If creation is successful, true is returned.
 */
static bool SafeTestD3D11CreateDevice(IDXGIAdapter* Adapter, const DXGI_ADAPTER_DESC& AdapterDesc, D3D_FEATURE_LEVEL MinFeatureLevel, D3D_FEATURE_LEVEL MaxFeatureLevel, bool& bOutUMA)
{
	uint32 DeviceFlags = D3D11_CREATE_DEVICE_SINGLETHREADED;
	// Use a debug device if specified on the command line.
	if (FParse::Param(FCommandLine::Get(), TEXT("d3ddebug")) ||
		FParse::Param(FCommandLine::Get(), TEXT("d3debug")) ||
		FParse::Param(FCommandLine::Get(), TEXT("dxdebug")) ||
		FParse::Param(FCommandLine::Get(), TEXT("d3dlogwarnings")) ||
		FParse::Param(FCommandLine::Get(), TEXT("d3dbreakonwarning")) ||
		FParse::Param(FCommandLine::Get(), TEXT("d3dcontinueonerrors")))
	{
		DeviceFlags |= D3D11_CREATE_DEVICE_DEBUG;
	}

	// @MIXEDREALITY_CHANGE : BEGIN - Add BGRA flag for Windows Mixed Reality HMD's
	DeviceFlags |= D3D11_CREATE_DEVICE_BGRA_SUPPORT;
	// @MIXEDREALITY_CHANGE : END

	D3D_FEATURE_LEVEL RequestedFeatureLevels[] =
	{
		D3D_FEATURE_LEVEL_11_1,
		D3D_FEATURE_LEVEL_11_0,
	};

	// Trim to allowed feature levels
	int32 FirstAllowedFeatureLevel = 0;
	int32 NumAllowedFeatureLevels = UE_ARRAY_COUNT(RequestedFeatureLevels);
	int32 LastAllowedFeatureLevel = NumAllowedFeatureLevels - 1;

	while (FirstAllowedFeatureLevel < NumAllowedFeatureLevels)
	{
		if (RequestedFeatureLevels[FirstAllowedFeatureLevel] == MaxFeatureLevel)
		{
			break;
		}
		FirstAllowedFeatureLevel++;
	}

	while (LastAllowedFeatureLevel > 0)
	{
		if (RequestedFeatureLevels[LastAllowedFeatureLevel] >= MinFeatureLevel)
		{
			break;
		}
		LastAllowedFeatureLevel--;
	}

	NumAllowedFeatureLevels = LastAllowedFeatureLevel - FirstAllowedFeatureLevel + 1;
	if (MaxFeatureLevel < MinFeatureLevel || NumAllowedFeatureLevels <= 0)
	{
		return false;
	}

	const uint64 Luid = BitCast<uint64>(AdapterDesc.AdapterLuid);
	int32 Unused;
	ID3D11Device* D3DDevice = FWindowsD3D::FindCachedD3D11AdapterDevice(Luid, Unused);
	if (!D3DDevice)
	{
		// We don't want software renderer. Ideally we specify D3D_DRIVER_TYPE_HARDWARE on creation but
		// when we specify an adapter we need to specify D3D_DRIVER_TYPE_UNKNOWN (otherwise the call fails).
		// We cannot check the device type later (seems this is missing functionality in D3D).
		D3D_FEATURE_LEVEL DeviceFeatureLevel = (D3D_FEATURE_LEVEL)0;
		TRefCountPtr<ID3D11Device> DevicePtr;
		HRESULT Result = D3D11CreateDevice(
			Adapter,
			D3D_DRIVER_TYPE_UNKNOWN,
			nullptr,
			DeviceFlags,
			&RequestedFeatureLevels[FirstAllowedFeatureLevel],
			NumAllowedFeatureLevels,
			D3D11_SDK_VERSION,
			DevicePtr.GetInitReference(),
			&DeviceFeatureLevel,
			nullptr);

		if (FAILED(Result))
		{
			// Log any reason for failure to create test device. Extra debug help.
			UE_LOGF(LogD3D, Log, "D3D11CreateDevice failed with code 0x%08X", static_cast<uint32>(Result));

			// Fatal error on 0x887A002D
			if (DXGI_ERROR_SDK_COMPONENT_MISSING == Result)
			{
				UE_LOGF(LogD3D, Fatal, "-d3ddebug was used but optional Graphics Tools were not found. Install them through the Manage Optional Features in windows. See: https://docs.microsoft.com/en-us/windows/uwp/gaming/use-the-directx-runtime-and-visual-studio-graphics-diagnostic-features");
			}

			return false;
		}

		D3DDevice = DevicePtr.GetReference();
		FWindowsD3D::CacheD3D11AdapterDevice(Luid, MoveTemp(DevicePtr), DeviceFeatureLevel);
	}

	D3D11_FEATURE_DATA_D3D11_OPTIONS2 FeatureData = {};
	bOutUMA = SUCCEEDED(D3DDevice->CheckFeatureSupport(D3D11_FEATURE_D3D11_OPTIONS2, &FeatureData, sizeof(FeatureData))) && FeatureData.UnifiedMemoryArchitecture;

	return true;
}

static uint64 ConvertLuidToUint64(LUID Luid)
{
	return (uint64_t)Luid.LowPart | ((uint64_t)Luid.HighPart << 32);
}

uint64 FWindowsD3D::ChooseD3D11Adapter(uint64 DesiredAdapterLuid)
{
	static uint64 CachedAdapterLuid = 0;
	if (!DesiredAdapterLuid && CachedAdapterLuid)
	{
		return CachedAdapterLuid;
	}

	TRefCountPtr<IDXGIFactory1> DXGIFactory1;
	CreateDXGIFactory1(IID_PPV_ARGS(DXGIFactory1.GetInitReference()));
	if (!DXGIFactory1)
	{
		return 0;
	}

	TRefCountPtr<IDXGIFactory6> DXGIFactory6;
	DXGIFactory1->QueryInterface(IID_PPV_ARGS(DXGIFactory6.GetInitReference()));

	bool bAllowPerfHUD = true;

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	bAllowPerfHUD = false;
#endif

	// Non-static as it is used only a few times
	const TConsoleVariableData<int32>* const CVarGraphicsAdapter = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GraphicsAdapter"));
	int32 CVarExplicitAdapterValue = DesiredAdapterLuid == 0 ? (CVarGraphicsAdapter ? CVarGraphicsAdapter->GetValueOnGameThread() : -1) : -2;
	FParse::Value(FCommandLine::Get(), TEXT("graphicsadapter="), CVarExplicitAdapterValue);

	const bool bFavorDiscreteAdapter = CVarExplicitAdapterValue == -1;

	TRefCountPtr<IDXGIAdapter> TempAdapter;
	D3D_FEATURE_LEVEL MinAllowedFeatureLevel = D3D_FEATURE_LEVEL_11_0;
	D3D_FEATURE_LEVEL MaxAllowedFeatureLevel = D3D_FEATURE_LEVEL_11_1;

	TOptional<LUID> PreferredAdapterLuid;
	TOptional<LUID> FirstDiscreteAdapterLuid;
	TOptional<LUID> BestMemoryAdapterLuid;
	TOptional<LUID> FirstAdapterLuid;
	SIZE_T BestDedicatedVideoMemory = 0;

	int GpuPreferenceInt = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE;
	FParse::Value(FCommandLine::Get(), TEXT("-gpupreference="), GpuPreferenceInt);
	DXGI_GPU_PREFERENCE GpuPreference;
	switch (GpuPreferenceInt)
	{
	case 1: GpuPreference = DXGI_GPU_PREFERENCE_MINIMUM_POWER; break;
	case 2: GpuPreference = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE; break;
	default: GpuPreference = DXGI_GPU_PREFERENCE_UNSPECIFIED; break;
	}

	const ED3DGpuVendorId PreferredVendor = GetPreferredGpuVendor();
	const bool bAllowSoftwareFallback = FParse::Param(FCommandLine::Get(), TEXT("AllowSoftwareRendering"));

	// Enumerate the DXGIFactory's adapters.
	for (uint32 AdapterIndex = 0; EnumAdapters(AdapterIndex, GpuPreference, DXGIFactory1, DXGIFactory6, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
	{
		// Check that if adapter supports D3D11.
		if (TempAdapter)
		{
			DXGI_ADAPTER_DESC AdapterDesc{};
			bool bUMA;
			if (SUCCEEDED(TempAdapter->GetDesc(&AdapterDesc)) && SafeTestD3D11CreateDevice(TempAdapter, AdapterDesc, MinAllowedFeatureLevel, MaxAllowedFeatureLevel, bUMA))
			{
				const bool bIsWARP = ConvertToGpuVendorId(AdapterDesc.VendorId) == ED3DGpuVendorId::Microsoft;

				// PerfHUD is for performance profiling
				const bool bIsPerfHUD = !FCString::Stricmp(AdapterDesc.Description, TEXT("NVIDIA PerfHUD"));

				// Add special check to support HMDs, which do not have associated outputs.
				// To reject the software emulation, unless the cvar wants it.
				// https://msdn.microsoft.com/en-us/library/windows/desktop/bb205075(v=vs.85).aspx#WARP_new_for_Win8
				// Before we tested for no output devices but that failed where a laptop had a Intel (with output) and NVidia (with no output)
				const bool bSkipSoftwareAdapter = bIsWARP && !bAllowSoftwareFallback && CVarExplicitAdapterValue < 0 && DesiredAdapterLuid == 0;

				// we don't allow the PerfHUD adapter
				const bool bSkipPerfHUDAdapter = bIsPerfHUD && !bAllowPerfHUD;

				// the caller wants a specific adapter, not this one
				const bool bSkipDesiredAdapter = DesiredAdapterLuid != 0 && FMemory::Memcmp(&DesiredAdapterLuid, &AdapterDesc.AdapterLuid, sizeof(LUID)) != 0;

				// the user wants a specific adapter, not this one
				const bool bSkipExplicitAdapter = CVarExplicitAdapterValue >= 0 && AdapterIndex != CVarExplicitAdapterValue;

				const bool bSkipAdapter = bSkipSoftwareAdapter || bSkipPerfHUDAdapter || bSkipDesiredAdapter || bSkipExplicitAdapter || FParse::Param(FCommandLine::Get(), TEXT("ForceZeroAdapters"));

				const bool bIsIntegrated = bUMA;

				if (!bSkipAdapter)
				{
					if (PreferredVendor != ED3DGpuVendorId::Unknown && PreferredVendor == ConvertToGpuVendorId(AdapterDesc.VendorId) && !PreferredAdapterLuid.IsSet())
					{
						PreferredAdapterLuid = AdapterDesc.AdapterLuid;
					}

					if (!bIsWARP && !bIsIntegrated)
					{
						if (!FirstDiscreteAdapterLuid.IsSet())
						{
							FirstDiscreteAdapterLuid = AdapterDesc.AdapterLuid;
						}

						if (AdapterDesc.DedicatedVideoMemory > BestDedicatedVideoMemory)
						{
							BestMemoryAdapterLuid = AdapterDesc.AdapterLuid;
							BestDedicatedVideoMemory = AdapterDesc.DedicatedVideoMemory;
							if (PreferredVendor != ED3DGpuVendorId::Unknown && PreferredVendor == ConvertToGpuVendorId(AdapterDesc.VendorId))
							{
								// Choose the best option of the preferred IHV devices
								PreferredAdapterLuid = BestMemoryAdapterLuid;
							}
						}
					}

					if (!FirstAdapterLuid.IsSet())
					{
						FirstAdapterLuid = AdapterDesc.AdapterLuid;
					}
				}
			}
		}
	}

	uint64 Luid = 0;
	if (bFavorDiscreteAdapter)
	{
		if (PreferredAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(PreferredAdapterLuid.GetValue());
		}
		else if (BestMemoryAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(BestMemoryAdapterLuid.GetValue());
		}
		else if (FirstDiscreteAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(FirstDiscreteAdapterLuid.GetValue());
		}
		else if (FirstAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(FirstAdapterLuid.GetValue());
		}
	}
	else if (FirstAdapterLuid.IsSet())
	{
		Luid = ConvertLuidToUint64(FirstAdapterLuid.GetValue());
	}

	if (!DesiredAdapterLuid)
	{
		CachedAdapterLuid = Luid;
	}
	return Luid;
}

uint64 FWindowsD3D::ChooseD3D12Adapter(uint64 DesiredAdapterLuid)
{
	static uint64 CachedAdapterLuid = 0;
	if (!DesiredAdapterLuid && CachedAdapterLuid)
	{
		return CachedAdapterLuid;
	}

	TRefCountPtr<IDXGIFactory1> DXGIFactory1;
	CreateDXGIFactory1(IID_PPV_ARGS(DXGIFactory1.GetInitReference()));
	if (!DXGIFactory1)
	{
		return 0;
	}

	TRefCountPtr<IDXGIFactory6> DXGIFactory6;
	DXGIFactory1->QueryInterface(__uuidof(IDXGIFactory6), (void**)DXGIFactory6.GetInitReference());

	bool bAllowPerfHUD = true;

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	bAllowPerfHUD = false;
#endif

	// Non-static as it is used only a few times
	const TConsoleVariableData<int32>* const CVarGraphicsAdapter = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GraphicsAdapter"));
	int32 CVarExplicitAdapterValue = DesiredAdapterLuid == 0 ? (CVarGraphicsAdapter ? CVarGraphicsAdapter->GetValueOnGameThread() : -1) : -2;
	FParse::Value(FCommandLine::Get(), TEXT("graphicsadapter="), CVarExplicitAdapterValue);

	const bool bFavorDiscreteAdapter = CVarExplicitAdapterValue == -1;

	TRefCountPtr<IDXGIAdapter> TempAdapter;

	TOptional<LUID> PreferredAdapterLuid;
	TOptional<LUID> FirstDiscreteAdapterLuid;
	TOptional<LUID> BestMemoryAdapterLuid;
	TOptional<LUID> FirstAdapterLuid;
	SIZE_T BestDedicatedVideoMemory = 0;

	static bool bRequestedWARP = FParse::Param(FCommandLine::Get(), TEXT("warp"));
	static bool bAllowSoftwareRendering = FParse::Param(FCommandLine::Get(), TEXT("AllowSoftwareRendering"));

	int GpuPreferenceInt = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE;
	FParse::Value(FCommandLine::Get(), TEXT("-gpupreference="), GpuPreferenceInt);
	DXGI_GPU_PREFERENCE GpuPreference;
	switch (GpuPreferenceInt)
	{
	case 1: GpuPreference = DXGI_GPU_PREFERENCE_MINIMUM_POWER; break;
	case 2: GpuPreference = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE; break;
	default: GpuPreference = DXGI_GPU_PREFERENCE_UNSPECIFIED; break;
	}

	const ED3DGpuVendorId PreferredVendor = GetPreferredGpuVendor();

	// Enumerate the DXGIFactory's adapters.
	for (uint32 AdapterIndex = 0; EnumAdapters(AdapterIndex, GpuPreference, DXGIFactory1, DXGIFactory6, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
	{
		// Check that if adapter supports D3D12.
		if (TempAdapter)
		{
			DXGI_ADAPTER_DESC AdapterDesc{};
			if (FAILED(TempAdapter->GetDesc(&AdapterDesc)))
			{
				continue;
			}

#if INTEL_EXTENSIONS
			// Enable Intel App Discovery
			if (AdapterDesc.VendorId == (uint32)ED3DGpuVendorId::Intel)
			{
				// Intel's App information needs to be registered *before* device creation, so we have to do it here at the last second.
				// Even though it takes the device ID as an argument, we've been told by Intel that this isn't going to cause problems if multiple Intel devices are detected.
				EnableIntelAppDiscovery(AdapterDesc.DeviceId);
			}
#endif

			bool bUMA{};
			if (SafeTestD3D12CreateDevice(TempAdapter, AdapterDesc, bUMA))
			{
				const bool bIsWARP = ConvertToGpuVendorId(AdapterDesc.VendorId) == ED3DGpuVendorId::Microsoft;

				// If requested WARP, then reject all other adapters. If WARP not requested, then reject the WARP device if software rendering support is disallowed
				const bool bSkipWARP = (bRequestedWARP && !bIsWARP) || (!bRequestedWARP && bIsWARP && !bAllowSoftwareRendering);

				// PerfHUD is for performance profiling
				const bool bIsPerfHUD = !FCString::Stricmp(AdapterDesc.Description, TEXT("NVIDIA PerfHUD"));

				// we don't allow the PerfHUD adapter
				const bool bSkipPerfHUDAdapter = bIsPerfHUD && !bAllowPerfHUD;

				// the caller wants a specific adapter, not this one
				const bool bSkipDesiredAdapter = DesiredAdapterLuid != 0 && FMemory::Memcmp(&DesiredAdapterLuid, &AdapterDesc.AdapterLuid, sizeof(LUID)) != 0;

				// the user wants a specific adapter, not this one
				const bool bSkipExplicitAdapter = CVarExplicitAdapterValue >= 0 && AdapterIndex != CVarExplicitAdapterValue;

				const bool bSkipAdapter = bSkipWARP || bSkipPerfHUDAdapter || bSkipDesiredAdapter || bSkipExplicitAdapter || FParse::Param(FCommandLine::Get(), TEXT("ForceZeroAdapters"));

				const bool bIsIntegrated = bUMA;

				if (!bSkipAdapter)
				{
					if (PreferredVendor != ED3DGpuVendorId::Unknown && PreferredVendor == ConvertToGpuVendorId(AdapterDesc.VendorId) && !PreferredAdapterLuid.IsSet())
					{
						PreferredAdapterLuid = AdapterDesc.AdapterLuid;
					}

					if (!bIsWARP && !bIsIntegrated)
					{
						if (!FirstDiscreteAdapterLuid.IsSet())
						{
							FirstDiscreteAdapterLuid = AdapterDesc.AdapterLuid;
						}

						if (AdapterDesc.DedicatedVideoMemory > BestDedicatedVideoMemory)
						{
							BestMemoryAdapterLuid = AdapterDesc.AdapterLuid;
							BestDedicatedVideoMemory = AdapterDesc.DedicatedVideoMemory;
							if (PreferredVendor != ED3DGpuVendorId::Unknown && PreferredVendor == ConvertToGpuVendorId(AdapterDesc.VendorId))
							{
								// Choose the best option of the preferred IHV devices
								PreferredAdapterLuid = BestMemoryAdapterLuid;
							}
						}
					}

					if (!FirstAdapterLuid.IsSet())
					{
						FirstAdapterLuid = AdapterDesc.AdapterLuid;
					}
				}
			}
		}
	}

	uint64 Luid = 0;
	if (bFavorDiscreteAdapter)
	{
		// We assume Intel is integrated graphics (slower than discrete) than NVIDIA or AMD cards and rather take a different one
		if (PreferredAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(PreferredAdapterLuid.GetValue());
		}
		else if (BestMemoryAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(BestMemoryAdapterLuid.GetValue());
		}
		else if (FirstDiscreteAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(FirstDiscreteAdapterLuid.GetValue());
		}
		else if (FirstAdapterLuid.IsSet())
		{
			Luid = ConvertLuidToUint64(FirstAdapterLuid.GetValue());
		}
	}
	else if (FirstAdapterLuid.IsSet())
	{
		Luid = ConvertLuidToUint64(FirstAdapterLuid.GetValue());
	}

	if (!DesiredAdapterLuid)
	{
		CachedAdapterLuid = Luid;
	}
	return Luid;
}

ID3D11Device* FWindowsD3D::FindCachedD3D11AdapterDevice(uint64 AdapterLuid, int32& OutFeatureLevel)
{
	ID3D11Device* Device = nullptr;

	const TPair<TRefCountPtr<ID3D11Device>, int32>* const PairPtr = D3D11AdapterDevices.Find(AdapterLuid);
	if (PairPtr)
	{
		OutFeatureLevel = PairPtr->Value;
		Device = PairPtr->Key.GetReference();
	}

	return Device;
}

void FWindowsD3D::CacheD3D11AdapterDevice(uint64 AdapterLuid, TRefCountPtr<ID3D11Device>&& Device, int32 FeatureLevel)
{
	D3D11AdapterDevices.Emplace(AdapterLuid, TPair<TRefCountPtr<ID3D11Device>, int32>(MoveTemp(Device), FeatureLevel));
}

ID3D12Device* FWindowsD3D::FindCachedD3D12AdapterDevice(uint64 AdapterLuid)
{
	TRefCountPtr<ID3D12Device>* DevicePtr = D3D12AdapterDevices.Find(AdapterLuid);
	return DevicePtr ? DevicePtr->GetReference() : nullptr;
}

void FWindowsD3D::CacheD3D12AdapterDevice(uint64 AdapterLuid, TRefCountPtr<ID3D12Device>&& Device)
{
	D3D12AdapterDevices.Emplace(AdapterLuid, MoveTemp(Device));
}

void FWindowsD3D::ReleaseCachedAdapterDevices()
{
	D3D11AdapterDevices.Reset();
	D3D12AdapterDevices.Reset();
}
