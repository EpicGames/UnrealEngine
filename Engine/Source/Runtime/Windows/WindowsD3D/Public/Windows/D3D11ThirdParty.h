// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"

#pragma pack(push,16)

THIRD_PARTY_INCLUDES_START
	#include <d3d11_4.h>
	#include <dxgi1_6.h>
	#include <dxgidebug.h>
THIRD_PARTY_INCLUDES_END

#undef DrawText

#pragma pack(pop)

#include "Microsoft/HideMicrosoftPlatformTypes.h"
