// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WindowsD3D : ModuleRules
{
	public WindowsD3D(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core"
			}
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12", "IntelExtensionsFramework");
	}
}
