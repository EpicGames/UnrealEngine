// Copyright Epic Games, Inc. All Rights Reserved.

#include "HighlightRecorder.h"
#include "WmfMp4Writer.h"

#include "Misc/Paths.h"

#include "Engine/GameEngine.h"
#include "RenderingThread.h"
#include "Rendering/SlateRenderer.h"
#include "Framework/Application/SlateApplication.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "HAL/PlatformTime.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/Thread.h"
#include "UnrealEngine.h"
#include "VideoRecordingSystem.h"

DEFINE_LOG_CATEGORY(WMF);
DEFINE_LOG_CATEGORY(HighlightRecorder);

//////////////////////////////////////////////////////////////////////////
// console commands for testing

FAutoConsoleCommand HighlightRecorderStart(TEXT("HighlightRecorder.Start"), TEXT("Starts recording of highlight clip, optional parameter: max duration (float, 30 seconds by default)"), FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&FHighlightRecorder::Start));
FAutoConsoleCommand HighlightRecorderStop(TEXT("HighlightRecorder.Stop"), TEXT("Stops recording of highlight clip"), FConsoleCommandDelegate::CreateStatic(&FHighlightRecorder::StopCmd));
FAutoConsoleCommand HighlightRecorderPause(TEXT("HighlightRecorder.Pause"), TEXT("Pauses recording of highlight clip"), FConsoleCommandDelegate::CreateStatic(&FHighlightRecorder::PauseCmd));
FAutoConsoleCommand HighlightRecorderResume(TEXT("HighlightRecorder.Resume"), TEXT("Resumes recording of highlight clip"), FConsoleCommandDelegate::CreateStatic(&FHighlightRecorder::ResumeCmd));
FAutoConsoleCommand HighlightRecorderSave(TEXT("HighlightRecorder.Save"), TEXT("Saves highlight clip, optional parameters: filename (\"test.mp4\" by default) and max duration (float, secs, duration of ring buffer by default)"), FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&FHighlightRecorder::SaveCmd));

//////////////////////////////////////////////////////////////////////////

FHighlightRecorder* FHighlightRecorder::Singleton = nullptr;

CSV_DECLARE_CATEGORY_EXTERN(WindowsVideoRecordingSystem);

//////////////////////////////////////////////////////////////////////////

FHighlightRecorder::FHighlightRecorder()
{
	check(Singleton == nullptr);
}

FHighlightRecorder::~FHighlightRecorder()
{
	Stop();
	UE_LOGF(HighlightRecorder, Log, "destroyed");
}

bool FHighlightRecorder::Start(double RingBufferDurationSecs)
{
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, HighlightRecorder_Start);

	if (State != EState::Stopped)
	{
		UE_LOGF(HighlightRecorder, Error, "cannot start recording, invalid state: %d", static_cast<int32>(State.Load()));
		return false;
	}

	RingBuffer.Reset();
	RingBuffer.SetMaxDuration(FTimespan::FromSeconds(RingBufferDurationSecs));

	RecordingStartTime = FTimespan::FromSeconds(FPlatformTime::Seconds());
	PauseTimestamp = 0;
	TotalPausedDuration = 0;
	NumPushedFrames = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!FGameplayMediaEncoder::Get()->RegisterListener(this))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return false;
	}

	State = EState::Recording;

	UE_LOGF(HighlightRecorder, Log, "recording started, ring buffer %.2f secs", RingBufferDurationSecs);

	return true;
}

bool FHighlightRecorder::Pause(bool bPause)
{
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, HighlightRecorder_Pause);

	if (State == EState::Stopped)
	{
		UE_LOGF(HighlightRecorder, Error, "cannot pause/resume recording, recording is stopped");
		return false;
	}

	if (bPause && PauseTimestamp == 0)
	{
		PauseTimestamp = GetRecordingTime();
		State = EState::Paused;
		UE_LOGF(HighlightRecorder, Log, "paused");
	}
	//allow unpause to occur if we are actually paused or if we happened to pause on the same frame as recording start
	else if (!bPause && (PauseTimestamp != 0 || PauseTimestamp == GetRecordingTime()))
	{
		FTimespan LastPausedDuration = GetRecordingTime() - PauseTimestamp;
		TotalPausedDuration += LastPausedDuration;
		PauseTimestamp = 0;
		FPlatformMisc::MemoryBarrier();
		State = EState::Recording;
		UE_LOGF(HighlightRecorder, Log, "resumed after %.3f s", LastPausedDuration.GetTotalSeconds());
	}

	return true;
}

void FHighlightRecorder::Stop()
{
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, HighlightRecorder_Stop);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGameplayMediaEncoder::Get()->UnregisterListener(this);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	State = EState::Stopped;

	if (BackgroundSaving)
	{
		BackgroundSaving->Join();
		BackgroundSaving.Reset();
	}

	UE_LOGF(HighlightRecorder, Log, "recording stopped");
}

FTimespan FHighlightRecorder::GetRecordingTime() const
{
	return FTimespan::FromSeconds(FPlatformTime::Seconds()) - RecordingStartTime - TotalPausedDuration;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FHighlightRecorder::OnMediaSample(const AVEncoder::FMediaPacket& InSample)
{
	// We might be paused, so don't do anything
	if (State != EState::Recording)
	{
		return;
	}

	// Only start pushing video frames once we receive a key frame
	if (NumPushedFrames == 0 && InSample.Type == AVEncoder::EPacketType::Video)
	{
		if (!InSample.IsVideoKeyFrame())
		{
			return;
		}

		++NumPushedFrames;
	}

	AVEncoder::FMediaPacket SampleCopy = InSample;


	AVEncoder::FMediaPacket A = InSample;
	AVEncoder::FMediaPacket B = MoveTemp(A);
	AVEncoder::FMediaPacket C(AVEncoder::EPacketType::Video);
	C = MoveTemp(B);
	AVEncoder::FMediaPacket D(AVEncoder::EPacketType::Video);
	D = C;

	if (TotalPausedDuration != 0)
	{
		SampleCopy.Timestamp = SampleCopy.Timestamp  - TotalPausedDuration;
	}

	RingBuffer.Push(MoveTemp(SampleCopy));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FHighlightRecorder::SaveHighlight(const TCHAR* Filename, FDoneCallback InDoneCallback, double MaxDurationSecs)
{
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, HighlightRecorder_Save);

	if (State == EState::Stopped)
	{
		UE_LOGF(HighlightRecorder, Error, "cannot save clip when recording is stopped");
		return false;
	}

	if (bSaving)
	{
		UE_LOGF(HighlightRecorder, Error, "saving is busy with the previous clip");
		return false;
	}

	UE_LOGF(HighlightRecorder, Log, "start saving to %ls, max duration %.3f", Filename, MaxDurationSecs);

	auto& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FString FullFilename = PlatformFile.ConvertToAbsolutePathForExternalAppForWrite(*(FPaths::VideoCaptureDir() + Filename));

	bSaving = true;
	DoneCallback = MoveTemp(InDoneCallback);

	{
		CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, HighlightRecorder_SaveThreadCreation);
		if (BackgroundSaving && BackgroundSaving->IsJoinable())
		{
			BackgroundSaving->Join();
		}

		BackgroundSaving.Reset(new FThread(TEXT("Highlight Saving"), [this, FullFilename, MaxDurationSecs]()
		{
			SaveHighlightInBackground(FullFilename, MaxDurationSecs);
		}));
	}

	return true;
}


// the bool result is solely for convenience (CHECK_HR) and is ignored as it's a thread function and
// nobody checks it's result. Actual result is notified by the callback.
bool FHighlightRecorder::SaveHighlightInBackground(const FString& Filename, double MaxDurationSecs)
{
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, HighlightRecorder_SaveInBackground);

	double T0 = FPlatformTime::Seconds();

	bool bRes = SaveHighlightInBackgroundImpl(Filename, MaxDurationSecs);

	double PassedSecs = FPlatformTime::Seconds() - T0;
	UE_LOGF(HighlightRecorder, Log, "saving to %ls %ls, took %.3f secs", *Filename, bRes ? TEXT("succeeded") : TEXT("failed"), PassedSecs);

	DoneCallback(bRes, Filename);
	bSaving = false;

	return bRes;
}


bool FHighlightRecorder::SaveHighlightInBackgroundImpl(const FString& Filename, double MaxDurationSecs)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<AVEncoder::FMediaPacket> Samples = RingBuffer.GetCopy();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	

	if (Samples.Num()==0)
	{
		UE_LOGF(HighlightRecorder, Error, "no samples to save to .mp4");
		return false;
	}

	int FirstSampleIndex;
	FTimespan StartTime;
	if (!GetSavingStart(Samples, FTimespan::FromSeconds(MaxDurationSecs), FirstSampleIndex, StartTime))
	{
		return false;
	}

	// Check if we have audio, so that if we don't, we don't create an audio track
	bool bHasAudio = false;
	for (int Idx = FirstSampleIndex; Idx != Samples.Num(); ++Idx)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Samples[Idx].Type == AVEncoder::EPacketType::Audio)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			bHasAudio = true;
			break;
		}
	}

	if (!InitialiseMp4Writer(Filename, bHasAudio))
	{
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	checkf(Samples[FirstSampleIndex].IsVideoKeyFrame(), TEXT("t %.3f d %.3f"), Samples[FirstSampleIndex].Timestamp.GetTotalSeconds(), Samples[FirstSampleIndex].Duration.GetTotalSeconds());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (FirstSampleIndex == Samples.Num())
	{
		UE_LOGF(HighlightRecorder, Error, "no samples to save to .mp4");
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_LOGF(HighlightRecorder, Verbose, "writting %d samples to .mp4, %.3f s, starting from %.3f s, index %d",
		Samples.Num() - FirstSampleIndex,
		(Samples.Last().Timestamp - StartTime + Samples.Last().Duration).GetTotalSeconds(),
		StartTime.GetTotalSeconds(),
		FirstSampleIndex);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// get samples starting from `StartTime` and push them into Mp4Writer
	for (int Idx = FirstSampleIndex; Idx != Samples.Num(); ++Idx)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AVEncoder::FMediaPacket& Sample = Samples[Idx];
		Sample.Timestamp = Sample.Timestamp - StartTime;
		if (!Mp4Writer->Write(Sample, (Sample.Type==AVEncoder::EPacketType::Audio) ? AudioStreamIndex : VideoStreamIndex))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			return false;
		}
	}

	if (!Mp4Writer->Finalize())
	{
		return false;
	}

	return true;
}


bool FHighlightRecorder::InitialiseMp4Writer(const FString& FullFilename, bool bHasAudio)
{
	// create target directory if it does not exist
	FString DirName = FPaths::GetPath(FullFilename);
	auto& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*DirName))
	{
		bool bRes = PlatformFile.CreateDirectory(*DirName);
		if (!bRes)
		{
			UE_LOGF(HighlightRecorder, Error, "Can't create directory %ls", *DirName);
			return false;
		}
	}

	Mp4Writer.Reset(new FWmfMp4Writer);

	if (!Mp4Writer->Initialize(*FullFilename))
	{
		return false;
	}

	if (bHasAudio)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TPair<FString, AVEncoder::FAudioConfig> AudioConfig = TPair<FString, AVEncoder::FAudioConfig>( FGameplayMediaEncoder::Get()->GetAudioConfig().Codec, FGameplayMediaEncoder::Get()->GetAudioConfig());		
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (AudioConfig.Key == "")
		{
			UE_LOGF(HighlightRecorder, Error, "Could not get audio config");
			return false;
		}
		
		TOptional<DWORD> Res = Mp4Writer->CreateAudioStream(AudioConfig.Key, AudioConfig.Value);
		if (Res.IsSet())
		{
			AudioStreamIndex = Res.GetValue();
		}
		else
		{
			return false;
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TPair<FString, AVEncoder::FVideoConfig> VideoConfig = TPair<FString, AVEncoder::FVideoConfig>( FGameplayMediaEncoder::Get()->GetVideoConfig().Codec, FGameplayMediaEncoder::Get()->GetVideoConfig());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (VideoConfig.Key == "")
	{
		UE_LOGF(HighlightRecorder, Error, "Could not get video config");
		return false;
	}

	TOptional<DWORD> Res = Mp4Writer->CreateVideoStream(VideoConfig.Key, VideoConfig.Value);
	if (Res.IsSet())
	{
		VideoStreamIndex = Res.GetValue();
	}
	else
	{
		return false;
	}

	if (!Mp4Writer->Start())
	{
		return false;
	}

	return true;
}

// finds index and timestamp of the first sample that should be written to .mp4
PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FHighlightRecorder::GetSavingStart(const TArray<AVEncoder::FMediaPacket>& Samples, FTimespan MaxDuration, int& StartIndex, FTimespan& StartTime) const
// the first sample in .mp4 file should have timestamp 0 and all other timestamps should be relative to the
// first one
// 1) if `MaxDurationSecs` > actual ring buffer duration (last sample timestamp - first) -> we need to save all
// samples from the ring buffer. saving start time = first sample timestamp
// 2) if `MaxDurationSecs` < actual ring buffer duration -> we need to start from the first video key-frame with
// timestamp > than ("cur time" - "max duration to save")
{
	// convert max duration to absolute time
	StartTime = GetRecordingTime() - MaxDuration;

	if (Samples.Num() == 0)
	{
		StartIndex = 0;
		UE_LOGF(HighlightRecorder, Error, "No samples to write to .mp4, max duration: %.3f", MaxDuration.GetTotalSeconds());
		return false;
	}

	FTimespan FirstTimestamp = Samples[0].Timestamp;

	if (FirstTimestamp > StartTime)
	{
		StartTime = FirstTimestamp;
		StartIndex = 0;
		return true;
	}

	int i = 0;
	bool bFound = false;
	for (; i != Samples.Num(); ++i)
	{
		FTimespan Time = Samples[i].Timestamp;
		if (Time >= StartTime && Samples[i].IsVideoKeyFrame())
		{
			// correct StartTime to match timestamp of the first sample to be written
			StartTime = Time;
			bFound = true;
			break;
		}
	}

	if (!bFound)
	{
		UE_LOGF(HighlightRecorder, Error, "No samples found to write to .mp4, max duration: %.3f", MaxDuration.GetTotalSeconds());
		return false;
	}

	StartIndex = i;

	return true;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS