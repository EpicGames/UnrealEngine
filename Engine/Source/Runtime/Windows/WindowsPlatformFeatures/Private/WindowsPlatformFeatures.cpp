// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsPlatformFeatures.h"
#include "WmfPrivate.h"
#include "WindowsVideoRecordingSystem.h"
#include "SaveGameSystem.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"

IMPLEMENT_MODULE(FWindowsPlatformFeaturesModule, WindowsPlatformFeatures);

FWindowsPlatformFeaturesModule::FWindowsPlatformFeaturesModule()
{
}

IVideoRecordingSystem* FWindowsPlatformFeaturesModule::GetVideoRecordingSystem()
{
	if (!VideoRecordingSystem)
	{
		VideoRecordingSystem = MakeShared<FWindowsVideoRecordingSystem>();
	}
	
	return VideoRecordingSystem.Get();
}

void FWindowsPlatformFeaturesModule::RegisterVideoRecordingSystem(TSharedPtr<IVideoRecordingSystem> InVideoRecordingSystem)
{
	VideoRecordingSystem = InVideoRecordingSystem;
}

ISaveGameSystem* FWindowsPlatformFeaturesModule::GetSaveGameSystem()
{
	static ISaveGameSystem* SaveGameSystem = nullptr;
	static bool bIniChecked = false;
	if (!SaveGameSystem || !bIniChecked)
	{
		ISaveGameSystemModule* SaveGameSystemModule = nullptr;
		if (!GEngineIni.IsEmpty())
		{
			FString SaveGameModule;
			GConfig->GetString(TEXT("PlatformFeatures"), TEXT("SaveGameSystemModule"), SaveGameModule, GEngineIni);

			if (!SaveGameModule.IsEmpty())
			{
				SaveGameSystemModule = FModuleManager::LoadModulePtr<ISaveGameSystemModule>(*SaveGameModule);
				if (SaveGameSystemModule != nullptr)
				{
					// Attempt to grab the save game system
					SaveGameSystem = SaveGameSystemModule->GetSaveGameSystem();
				}
			}
			bIniChecked = true;
		}

		if (SaveGameSystem == nullptr)
		{
			// Placeholder/default instance
			SaveGameSystem = IPlatformFeaturesModule::GetSaveGameSystem();
		}
	}

	return SaveGameSystem;
}


void FWindowsPlatformFeaturesModule::StartupModule()
{
	FModuleManager::Get().LoadModule(TEXT("GameplayMediaEncoder"));
}
