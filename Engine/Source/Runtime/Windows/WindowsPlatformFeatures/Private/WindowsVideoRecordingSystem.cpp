// Copyright Epic Games, Inc. All Rights Reserved.
#include "WindowsVideoRecordingSystem.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Engine/GameEngine.h"
#include "RenderingThread.h"
#include "Rendering/SlateRenderer.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/GameModeBase.h"
#include "Features/IModularFeatures.h"
#include "PlatformFeatures.h"

#include "ProfilingDebugging/CsvProfiler.h"

#include "HighlightRecorder.h"

#include <dwmapi.h>

DEFINE_VIDEOSYSTEMRECORDING_STATS
DEFINE_LOG_CATEGORY(WindowsVideoRecordingSystem);
CSV_DEFINE_CATEGORY(WindowsVideoRecordingSystem, true);

/**
 * This internal helper class handles disabling of the Windows built-in screenshot (PrtScr key) and clip
 * recorder (via Win-G key menu on Windows 10)
 */
class FWindowsVideoRecordingSystem::FWindowsScreenRecording
{
public:
	bool Disable()
	{
		return DisableInternal();
	}

	bool Enable()
	{
		return EnableInternal();
	}

private:
	struct FWindowHandle
	{
		FWindowHandle() : WindowsHandle(0), Affinity(0), bIsSupported(false), DisableCount(0) { }
		HWND			WindowsHandle;
		DWORD			Affinity;
		BOOL			bIsSupported;
		volatile int32	DisableCount;
	};

	bool DisableInternal()
	{
		if (GetCurrentlyActiveHandle())
		{
			check(CurrentHandle.IsValid());
			if (FPlatformAtomics::InterlockedIncrement(&CurrentHandle->DisableCount) == 1)
			{
				if (SetWindowDisplayAffinity(CurrentHandle->WindowsHandle, WDA_MONITOR))
				{
					return true;
				}
				else
				{
					//DWORD lerr = GetLastError();
					return false;
				}
			}
			return true;
		}
		return false;
	}

	bool EnableInternal()
	{
		if (GetCurrentlyActiveHandle())
		{
			check(CurrentHandle.IsValid());
			// Check that the disable count is valid. Should we have either lost the original handle or the user
			// is calling enable without having called disable first (which is an error but we let slide) we
			// should not drop the disable count.
			if (FPlatformAtomics::AtomicRead_Relaxed(&CurrentHandle->DisableCount) > 0)
			{
				if (FPlatformAtomics::InterlockedDecrement(&CurrentHandle->DisableCount) == 0)
				{
					if (SetWindowDisplayAffinity(CurrentHandle->WindowsHandle, CurrentHandle->Affinity))
					{
						return true;
					}
					else
					{
						//DWORD lerr = GetLastError();
						return false;
					}
				}
			}
			// Let's return true even if we weren't disabled on a new handle.
			return true;
		}
		return false;
	}

	TUniquePtr<FWindowHandle> CreateCurrentHandle()
	{
		TUniquePtr<FWindowHandle>	NewHandle(new FWindowHandle);
		HRESULT						res;
		res = DwmIsCompositionEnabled(&NewHandle->bIsSupported);
		if (res == S_OK)
		{
			if (NewHandle->bIsSupported)
			{
				if ((NewHandle->WindowsHandle = GetWindowHandleInternal()) != 0)
				{
					if ((NewHandle->bIsSupported = GetWindowDisplayAffinity(NewHandle->WindowsHandle, &NewHandle->Affinity)) != false)
					{
						return MoveTemp(NewHandle);
					}
				}
			}
		}
		return nullptr;
	}
	HWND GetWindowHandleInternal()
	{
		//return (HWND)GEngine->GameViewport->GetWindow()->GetNativeWindow()->GetOSWindowHandle();
		return GetActiveWindow();
	}

	bool GetCurrentlyActiveHandle()
	{
		TUniquePtr<FWindowHandle> Now = CreateCurrentHandle();
		if (Now.IsValid())
		{
			if (CurrentHandle.IsValid() && CurrentHandle->WindowsHandle != Now->WindowsHandle)
			{
				// In case the window handle has changed for any reason we do not try to restore the
				// state we got when we created our internal handle. In all likelihood we do not own
				// the window any more and should not be messing with its handle.
				CurrentHandle.Reset();
			}
			if (!CurrentHandle.IsValid())
			{
				CurrentHandle = MoveTemp(Now);
			}
			return true;
		}
		else
		{
			// Likewise if we can't get a valid handle right now we drop whatever we were using before.
			CurrentHandle.Reset();
		}
		return false;
	}

	TUniquePtr<FWindowHandle>	CurrentHandle;
};



PRAGMA_DISABLE_DEPRECATION_WARNINGS
FWindowsVideoRecordingSystem::FWindowsVideoRecordingSystem()
{
	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);

	EnableRecording(true);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FWindowsVideoRecordingSystem::~FWindowsVideoRecordingSystem()
{
	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);

	EnableRecording(false);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FWindowsVideoRecordingSystem::EnableRecording(bool bEnableRecording)
{
	SCOPE_CYCLE_COUNTER(STAT_VideoRecordingSystem_EnableRecording);
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, EnableRecording);

	if (!FApp::CanEverRender())
	{
		UE_LOGF(WindowsVideoRecordingSystem, Warning, "Can't enable recording because this App can't render.");
		return;
	}

	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);
	if (bEnableRecording && !Recorder)
	{
		Recorder.Reset(new FHighlightRecorder());
	}
	else if (!bEnableRecording && Recorder)
	{
		Recorder.Reset();
	}


	// Also disable/enable the Windows screenshot and clip recording ability.
	if (!ScreenshotAndRecorderHandler.IsValid())
	{
		ScreenshotAndRecorderHandler.Reset(new FWindowsScreenRecording);
	}
	if (bEnableRecording)
	{
		ScreenshotAndRecorderHandler->Enable();
	}
	else
	{
		ScreenshotAndRecorderHandler->Disable();
	}
}

bool FWindowsVideoRecordingSystem::IsEnabled() const
{
	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);
	return Recorder.Get()==nullptr ? false : true;
}

void FWindowsVideoRecordingSystem::NextRecording()
{
	if (Parameters.bAutoContinue)
	{
		CurrentFilename = FString::Printf(TEXT("%s_%" UINT64_FMT ".mp4"), *BaseFilename, ++RecordingIndex);
	}
	else
	{
		CurrentFilename = BaseFilename + TEXT(".mp4");
	}
}

bool FWindowsVideoRecordingSystem::NewRecording(const TCHAR* DestinationFileName, FVideoRecordingParameters InParameters)
{
	SCOPE_CYCLE_COUNTER(STAT_VideoRecordingSystem_NewRecording);
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, NewRecording);

	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);

	if (!IsEnabled())
	{
		UE_LOGF(WindowsVideoRecordingSystem, Warning, "%ls : can't open a new recording. Recording is disabled", __FUNCTIONW__);
		return false;
	}

	if (RecordState != EVideoRecordingState::None)
	{
		UE_LOGF(WindowsVideoRecordingSystem, Warning, "FWindowsVideoRecordingSystem::NewRecording: can't open a new recording, one is already in progress.");
		return false;
	}

	Parameters = InParameters;

	RecordingIndex = 0;
	BaseFilename = "recording";
	CyclesBeforePausing = 0;
	CurrentStartRecordingCycles = 0;

	if (DestinationFileName != nullptr)
	{
		BaseFilename = DestinationFileName;
	}

	NextRecording();

	if (Recorder->GetState() == FHighlightRecorder::EState::Stopped)
	{
		// Call Start to initialize the internals, and pause right away
		if (Recorder->Start(double(Parameters.RecordingLengthSeconds)) == false)
		{
			// this could be if running with -nullrhi, and the Pause below will crash
			return false;
		}
	}
	bool bPauseRet = Recorder->Pause(true);
	check(bPauseRet);
	RecordState = EVideoRecordingState::Paused;

	if (Parameters.bAutoStart)
	{
		StartRecording();
	}

	return true;
}

void FWindowsVideoRecordingSystem::StartRecording()
{
	SCOPE_CYCLE_COUNTER(STAT_VideoRecordingSystem_StartRecording);
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, StartRecording);

	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);

	if (RecordState != EVideoRecordingState::Paused)
	{
		UE_LOGF(WindowsVideoRecordingSystem, Warning, "%ls: can't start recording, invalid state", __FUNCTIONW__);
		return;
	}

	UE_LOGF(WindowsVideoRecordingSystem, Log, "%ls: starting a recording", __FUNCTIONW__);

	if (Recorder->GetState() == FHighlightRecorder::EState::Stopped)
	{
		Recorder->Start(double(Parameters.RecordingLengthSeconds));
	}
	if (Recorder->GetState() == FHighlightRecorder::EState::Paused)
	{
		bool bPauseRet = Recorder->Pause(false);
		check(bPauseRet);
	}

	RecordState = EVideoRecordingState::Recording;
	CurrentStartRecordingCycles = FPlatformTime::Cycles64();
}

void FWindowsVideoRecordingSystem::PauseRecording()
{
	SCOPE_CYCLE_COUNTER(STAT_VideoRecordingSystem_PauseRecording);
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, PauseRecording);

	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);

	if (RecordState != EVideoRecordingState::Recording)
	{
		UE_LOGF(WindowsVideoRecordingSystem, Warning, "%ls: can't pause recording, invalid state.", __FUNCTIONW__);
		return;
	}

	UE_LOGF(WindowsVideoRecordingSystem, Log, "%ls: pausing a recording", __FUNCTIONW__);

	CyclesBeforePausing += FPlatformTime::Cycles64() - CurrentStartRecordingCycles;

	bool bPauseRet = Recorder->Pause(true);
	check(bPauseRet);

	RecordState = EVideoRecordingState::Paused;
}

uint64 FWindowsVideoRecordingSystem::GetMinimumRecordingSeconds() const
{
	return 6;
}

uint64 FWindowsVideoRecordingSystem::GetMaximumRecordingSeconds() const
{
	return 900;
}

float FWindowsVideoRecordingSystem::GetCurrentRecordingSeconds() const
{
	float Ret = float((FPlatformTime::Cycles64() - CurrentStartRecordingCycles + CyclesBeforePausing) * FPlatformTime::GetSecondsPerCycle());
	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls: reporting %f", __FUNCTIONW__, Ret);
	return Ret;
}

// #RVF : Make use of Comment ?
void FWindowsVideoRecordingSystem::FinalizeRecording(const bool bSaveRecording, const FText& Title, const FText& Comment, const bool bStopAutoContinue/* = true*/)
{
	SCOPE_CYCLE_COUNTER(STAT_VideoRecordingSystem_FinalizeRecording);
	CSV_SCOPED_TIMING_STAT(WindowsVideoRecordingSystem, FinalizeRecording);

	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);

	if (RecordState == EVideoRecordingState::None)
	{
		UE_LOGF(WindowsVideoRecordingSystem, Warning, "%ls: can't finalize recording, invalid state.", __FUNCTIONW__);
		return;
	}

	if (bSaveRecording)
	{
		bool bRet = Recorder->SaveHighlight(*CurrentFilename, [this, bStopAutoContinue](bool bRes, const FString& InFullPathToFile)
		{
			// Execute the Finalize event on the GameThread
			FGraphEventRef FinalizeEvent = FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
				FSimpleDelegateGraphTask::FDelegate::CreateRaw(this, &FWindowsVideoRecordingSystem::FinalizeCallbackOnGameThread,
					bRes, Parameters.bAutoContinue && !bStopAutoContinue, InFullPathToFile, true),
				TStatId(), nullptr, ENamedThreads::GameThread);
		}, double(Parameters.RecordingLengthSeconds));

		if (!bRet)
		{
			UE_LOGF(WindowsVideoRecordingSystem, Warning, "%ls: can't finalize recording.", __FUNCTIONW__);
			return;
		}

		RecordState = EVideoRecordingState::Finalizing;
	}
	else
	{
		// Drop this recording
		FinalizeCallbackOnGameThread(false, Parameters.bAutoContinue && !bStopAutoContinue, CurrentFilename, false);
	}

}

void FWindowsVideoRecordingSystem::FinalizeCallbackOnGameThread(bool bSaved, bool bAutoContinue, FString Path, bool bBroadcast)
{
	UE_LOGF(WindowsVideoRecordingSystem, Verbose, "%ls", __FUNCTIONW__);
	RecordState = EVideoRecordingState::None;

	if (bAutoContinue)
	{
		NextRecording();
		RecordState = EVideoRecordingState::Recording;
	}
	else
	{
		Recorder->Stop();
	}

	if (bBroadcast)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OnVideoRecordingFinalized.Broadcast(bSaved, Path);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
}

EVideoRecordingState FWindowsVideoRecordingSystem::GetRecordingState() const
{
	return RecordState;
}
