// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoRecordingSystem.h"

#include "RHI.h"
#include "RHIResources.h"

#include "HAL/ThreadSafeBool.h"
#include "CoreMinimal.h"
#include "AudioMixerDevice.h"
#include "RHI.h"
#include "RHIResources.h"
#include "PipelineStateCache.h"

DECLARE_LOG_CATEGORY_EXTERN(WindowsVideoRecordingSystem, VeryVerbose, VeryVerbose);

class FHighlightRecorder;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FWindowsVideoRecordingSystem : public IVideoRecordingSystem
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	FWindowsVideoRecordingSystem();
	~FWindowsVideoRecordingSystem() override;

	// IVideoRecordingSystem interface
	void EnableRecording(const bool bEnableRecording) override;
	bool IsEnabled() const override;
	bool NewRecording(const TCHAR* DestinationFileName, FVideoRecordingParameters Parameters = FVideoRecordingParameters()) override;
	void StartRecording() override;
	void PauseRecording() override;
	virtual uint64 GetMinimumRecordingSeconds() const override;
	virtual uint64 GetMaximumRecordingSeconds() const override;
	float GetCurrentRecordingSeconds() const override;
	void FinalizeRecording(const bool bSaveRecording, const FText& Title, const FText& Comment, const bool bStopAutoContinue = true) override;
	EVideoRecordingState GetRecordingState() const override;

private:
	void NextRecording();
	void FinalizeCallbackOnGameThread(bool bSaved, bool bAutoContinue, FString Path, bool bBroadcast);

	TAtomic<EVideoRecordingState> RecordState{ EVideoRecordingState::None };

	FVideoRecordingParameters Parameters;

	FString BaseFilename;
	FString CurrentFilename;
	uint64 RecordingIndex = 0;
	uint64 CurrentStartRecordingCycles = 0;
	uint64 CyclesBeforePausing = 0;

	// If this is nullptr, then it means video recording is not enabled
	TUniquePtr<FHighlightRecorder> Recorder;

	bool bAudioFormatChecked = false;

	class FWindowsScreenRecording;
	TUniquePtr<FWindowsScreenRecording>	ScreenshotAndRecorderHandler;
};

