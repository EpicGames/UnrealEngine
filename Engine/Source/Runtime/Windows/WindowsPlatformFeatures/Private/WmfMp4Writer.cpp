// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMp4Writer.h"

#include "IMFSampleWrapper.h"

#if WMFMEDIA_SUPPORTED_PLATFORM
	#pragma comment(lib, "mfplat")
	#pragma comment(lib, "mfuuid")
	#pragma comment(lib, "Mfreadwrite")
#endif

DECLARE_LOG_CATEGORY_EXTERN(MP4, Log, VeryVerbose);

DEFINE_LOG_CATEGORY(MP4);

bool FWmfMp4Writer::Initialize(const TCHAR* Filename)
{
	CHECK_HR(MFCreateSinkWriterFromURL(Filename, nullptr, nullptr, Writer.GetInitReference()));
	UE_LOGF(WMF, Verbose, "Initialised Mp4Writer for %ls", Filename);
	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TOptional<DWORD> FWmfMp4Writer::CreateAudioStream(const FString& Codec, const AVEncoder::FAudioConfig& Config)
{
	GUID Format;
	int AudioBitsPerSample = 0;
	if (Codec=="aac")
	{
		Format = MFAudioFormat_AAC;
		AudioBitsPerSample = 16;
	}
#if 0
	else if (Codec=="mp3")
	{
		Format = MFAudioFormat_MP3;
		AudioBitsPerSample = 16;
	}
#endif
	else
	{
		UE_LOGF(WMF, Error, "FWmfMp4Writer doesn't support codec %ls", *Codec);
		return {};
	}

	TRefCountPtr<IMFMediaType> MediaType;
	CHECK_HR_DEFAULT(MFCreateMediaType(MediaType.GetInitReference()));
	CHECK_HR_DEFAULT(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio));
	CHECK_HR_DEFAULT(MediaType->SetGUID(MF_MT_SUBTYPE, Format));
	CHECK_HR_DEFAULT(MediaType->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, AudioBitsPerSample));
	CHECK_HR_DEFAULT(MediaType->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, Config.Samplerate));
	CHECK_HR_DEFAULT(MediaType->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, Config.NumChannels));
	CHECK_HR_DEFAULT(MediaType->SetUINT32(MF_MT_AUDIO_AVG_BYTES_PER_SECOND, Config.Bitrate));

	DWORD StreamIndex = 0;
	CHECK_HR_DEFAULT(Writer->AddStream(MediaType, &StreamIndex));
	// no transcoding here so input type is the same as output type
	CHECK_HR_DEFAULT(Writer->SetInputMediaType(StreamIndex, MediaType, nullptr));
	return TOptional<DWORD>(StreamIndex);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TOptional<DWORD> FWmfMp4Writer::CreateVideoStream(const FString& Codec, const AVEncoder::FVideoConfig& Config)
{
	GUID Format;
	if (Codec == "h264")
	{
		Format = MFVideoFormat_H264;
	}
#if 0
	else if (Codec == "h265")
	{
		Format = MFVideoFormat_H265;
	}
#endif
	else
	{
		UE_LOGF(WMF, Error, "FWmfMp4Writer doesn't support codec %ls", *Codec);
		return {};
	}

	TRefCountPtr<IMFMediaType> MediaType;
	CHECK_HR_DEFAULT(MFCreateMediaType(MediaType.GetInitReference()));
	CHECK_HR_DEFAULT(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video));
	CHECK_HR_DEFAULT(MediaType->SetGUID(MF_MT_SUBTYPE, Format));
	CHECK_HR_DEFAULT(MediaType->SetUINT32(MF_MT_AVG_BITRATE, Config.Bitrate));
	CHECK_HR_DEFAULT(MFSetAttributeRatio(MediaType, MF_MT_FRAME_RATE, Config.Framerate, 1));
	CHECK_HR_DEFAULT(MFSetAttributeSize(MediaType, MF_MT_FRAME_SIZE, Config.Width, Config.Height));
	CHECK_HR_DEFAULT(MFSetAttributeRatio(MediaType, MF_MT_PIXEL_ASPECT_RATIO, 1, 1));
	CHECK_HR_DEFAULT(MediaType->SetUINT32(MF_MT_INTERLACE_MODE, MFVideoInterlace_Progressive));

	DWORD StreamIndex = 0;
	CHECK_HR_DEFAULT(Writer->AddStream(MediaType, &StreamIndex));
	// no transcoding here so input type is the same as output type
	CHECK_HR_DEFAULT(Writer->SetInputMediaType(StreamIndex, MediaType, nullptr));
	return TOptional<DWORD>(StreamIndex);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FWmfMp4Writer::Start()
{
	CHECK_HR(Writer->BeginWriting());
	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FWmfMp4Writer::Write(const AVEncoder::FMediaPacket& InSample, DWORD StreamIndex)
{
	AVEncoder::FIMFSampleWrapper Sample { InSample.Type };

	if (!Sample.CreateSample())
	{
		return false;
	}

	TRefCountPtr<IMFMediaBuffer> WmfBuffer;
	CHECK_HR_DEFAULT(MFCreateAlignedMemoryBuffer(InSample.Data.Num(), MF_1_BYTE_ALIGNMENT, WmfBuffer.GetInitReference()));
	uint8* Dst = nullptr;
	CHECK_HR_DEFAULT(WmfBuffer->Lock(&Dst, nullptr, nullptr));
	FMemory::Memcpy(Dst, InSample.Data.GetData(), InSample.Data.Num());
	CHECK_HR_DEFAULT(WmfBuffer->Unlock());

	WmfBuffer->SetCurrentLength(InSample.Data.Num());
	Sample.GetSample()->AddBuffer(WmfBuffer);
	Sample.SetTime(InSample.Timestamp);
	Sample.SetDuration(InSample.Duration);
	if(InSample.IsVideoKeyFrame() || InSample.Type == AVEncoder::EPacketType::Audio)
	{
		Sample.SetIsKeyFrame();
	}

	CHECK_HR(Writer->WriteSample(StreamIndex, const_cast<IMFSample*>(Sample.GetSample())));

	UE_LOGF(MP4, VeryVerbose, "stream #%d: time %.3f, duration %.3f%ls", StreamIndex, Sample.GetTime().GetTotalSeconds(), Sample.GetDuration().GetTotalSeconds(), InSample.IsVideoKeyFrame() ? TEXT(", key-frame") : TEXT(""));

	return true;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FWmfMp4Writer::Finalize()
{
	CHECK_HR(Writer->Finalize());
	UE_LOGF(WMF, VeryVerbose, "Closed .mp4");

	return true;
}
