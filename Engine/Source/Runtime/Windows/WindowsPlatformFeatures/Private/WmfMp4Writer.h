// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfPrivate.h"

#include "Math/NumericLimits.h"
#include "MediaPacket.h"
#include "Misc/Optional.h"
#include "Templates/RefCounting.h"

class FWmfMp4Writer final
{
public:
	bool Initialize(const TCHAR* Filename);

	/**
	 * Create an audio stream and return the its index on success
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TOptional<DWORD> CreateAudioStream(const FString& Codec, const AVEncoder::FAudioConfig& Config);

	/**
	 * Create a video stream and return the its index on success
	 */
	TOptional<DWORD> CreateVideoStream(const FString& Codec, const AVEncoder::FVideoConfig& Config);

	bool Write(const AVEncoder::FMediaPacket& InSample, DWORD StreamIndex);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool Start();
	bool Finalize();

private:
	TRefCountPtr<IMFSinkWriter> Writer;
};

