// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "CoreMinimal.h"

#define WMFMEDIA_SUPPORTED_PLATFORM (PLATFORM_WINDOWS && !UE_SERVER)

DECLARE_LOG_CATEGORY_EXTERN(WMF, Log, VeryVerbose);

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	//#include <d3d11.h>
	#include <mftransform.h>
	#include <mfapi.h>
	#include <mferror.h>
	#include <mfidl.h>
	#include <mfreadwrite.h>
	#include <Codecapi.h>
	#include <shlwapi.h>
THIRD_PARTY_INCLUDES_END

inline const FString GetComErrorDescription(HRESULT Res)
{
	const uint32 BufSize = 4096;
	WIDECHAR buffer[4096];
	if (::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM,
		nullptr,
		Res,
		MAKELANGID(LANG_NEUTRAL, SUBLANG_NEUTRAL),
		buffer,
		sizeof(buffer) / sizeof(*buffer),
		nullptr))
	{
		return buffer;
	}
	else
	{
		return TEXT("[cannot find error description]");
	}
}

#include "Windows/HideWindowsPlatformTypes.h"

// macro to deal with COM calls inside a function that returns `false` on error
#define CHECK_HR(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(WMF, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return false;\
		}\
	}

// macro to deal with COM calls inside COM method (that returns HRESULT)
#define CHECK_HR_COM(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(WMF, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return Res;\
		}\
	}

// macro to deal with COM calls inside COM method (that simply returns)
#define CHECK_HR_VOID(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(WMF, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return;\
		}\
	}

// macro to deal with COM calls inside a function and return `{}` on error
// This has the advantage of being able to return any type that can be initialized with `{}`
#define CHECK_HR_DEFAULT(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(WMF, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return {};\
		}\
	}

