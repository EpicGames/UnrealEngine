// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfRingBuffer.h"
#include "Misc/ScopeLock.h"

DECLARE_LOG_CATEGORY_EXTERN(WmfRingBuffer, Log, VeryVerbose);
DEFINE_LOG_CATEGORY(WmfRingBuffer);


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FWmfRingBuffer::Push(AVEncoder::FMediaPacket&& Sample)
{
	check(MaxDuration != 0);

	FScopeLock Lock(&Mutex);

	// start always from video sample for simplicity of `Cleanup` impl
	if (Samples.Num() == 0 && Sample.Type == AVEncoder::EPacketType::Audio)
	{
		// drop
	}
	else if (Samples.Num() == 0)
	{
		Samples.Add(MoveTemp(Sample));
	}
	else
		// video encoding takes longer than audio thus video encoded samples arrive later and need to be placed
		// inside the container. Video also is not strictly timestamp-ordered (maybe due to B-frames)
	{
		FTimespan TimestampToInsert = Sample.Timestamp;

		bool bIsSampleVideoKeyFrame = Sample.IsVideoKeyFrame();

		int i = Samples.Num() - 1;
		while (true)
		{
			if (Samples[i].Type == Sample.Type // don't change order of same stream cos video is not strictly timestamp-ordered (B-Frames)
				|| TimestampToInsert >= Samples[i].Timestamp)
			{
				Samples.Insert(MoveTemp(Sample), i + 1);
				break;
			}

			if (i == 0)
			{
				if (Sample.Type == AVEncoder::EPacketType::Video)
				{
					Samples.Insert(MoveTemp(Sample), 0);
				}
				// else: the first sample in the buffer always should be video key-frame, just drop audio sample
				// this can happen during debugging due to big gaps in timestamps

				break;
			}

			--i;
		}

		// cleanup only on receiving key-frame and only when ring-buffer if full
		if (!bCleanupPaused && bIsSampleVideoKeyFrame)
		{
			while (GetDuration() > GetMaxDuration())
			{
				Cleanup();
			}
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FWmfRingBuffer::Cleanup()
// removes a/v samples grouped by the oldest key-frame period
{
	FScopeLock Lock(&Mutex);

	check(Samples.Num() != 0);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(Samples[0].IsVideoKeyFrame());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// find the second key-frame
	int i = 1; // skip the first sample which is a keyframe
	bool bFound = false;
	for (; i != Samples.Num(); ++i)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Samples[i].IsVideoKeyFrame())
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			bFound = true;
			break;
		}
	}
	check(bFound);

	// remove key-frame period
	Samples.RemoveAt(0, i, EAllowShrinking::No);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(Samples[0].IsVideoKeyFrame());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_LOGF(WmfRingBuffer, VeryVerbose, "%d samples, %.3f s, %.3f - %.3f:%.3f",
		Samples.Num(),
		GetDuration().GetTotalSeconds(),
		Samples[0].Timestamp.GetTotalSeconds(),
		Samples.Last().Timestamp.GetTotalSeconds(),
		Samples.Last().Duration.GetTotalSeconds());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FWmfRingBuffer::PauseCleanup(bool bPause)
{
	bCleanupPaused = bPause;
}

FTimespan FWmfRingBuffer::GetDuration() const
{
	if (Samples.Num() == 0)
	{
		return 0;
	}

	// A duration of a video is not "LastFrameCaptureTime - FirstFrameCaptureTime" , but 
	// (LastFrameCaptureTime - FirstFrameCaptureTime) + LastFrameDuration.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Samples.Last().Timestamp + Samples.Last().Duration - Samples[0].Timestamp;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TArray<AVEncoder::FMediaPacket> FWmfRingBuffer::GetCopy()
{
	FScopeLock Lock(&Mutex);

	TArray<AVEncoder::FMediaPacket> Copy;
	// cloning is required because during saving a copy of ring buffer we modify samples timestamps.
	// using samples references shared between ring buffer and its copy being saved would cause ring
	// buffer timestamp logic corruption
	Copy = Samples;

	return Copy;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FWmfRingBuffer::Reset()
{
	FScopeLock Lock(&Mutex);
	Samples.Empty();
}
