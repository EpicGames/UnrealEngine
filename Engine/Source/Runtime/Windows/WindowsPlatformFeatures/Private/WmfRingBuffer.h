// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfPrivate.h"

#include "HAL/ThreadSafeBool.h"
#include "MediaPacket.h"

class FWmfRingBuffer
{
public:
	FTimespan GetMaxDuration() const
	{
		return MaxDuration;
	}

	void SetMaxDuration(FTimespan InMaxDuration)
	{
		MaxDuration = InMaxDuration;
	}

	FTimespan GetDuration() const;

	void PauseCleanup(bool bPause);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void Push(AVEncoder::FMediaPacket&& Sample);

	TArray<AVEncoder::FMediaPacket> GetCopy();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void Reset();

private:
	void Cleanup();

private:
	FTimespan MaxDuration = 0;
	FCriticalSection Mutex;
	FThreadSafeBool bCleanupPaused = false;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<AVEncoder::FMediaPacket> Samples;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

