// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PlatformFeatures.h"

class FWindowsPlatformFeaturesModule : public IPlatformFeaturesModule
{
public:

	/** Creates a new instance of the audio device implemented by the module. */
	WINDOWSPLATFORMFEATURES_API FWindowsPlatformFeaturesModule();

	WINDOWSPLATFORMFEATURES_API virtual IVideoRecordingSystem* GetVideoRecordingSystem() override;
	WINDOWSPLATFORMFEATURES_API virtual class ISaveGameSystem* GetSaveGameSystem() override;

	WINDOWSPLATFORMFEATURES_API void RegisterVideoRecordingSystem(TSharedPtr<IVideoRecordingSystem> VideoRecordingSystem);

private:
	WINDOWSPLATFORMFEATURES_API void StartupModule() override;

private:
	TSharedPtr<IVideoRecordingSystem> VideoRecordingSystem;
};


