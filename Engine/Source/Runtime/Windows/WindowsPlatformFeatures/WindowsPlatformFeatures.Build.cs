// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

[SupportedPlatformGroups("Windows")]
public class WindowsPlatformFeatures : ModuleRules
{
	public WindowsPlatformFeatures(ReadOnlyTargetRules Target) : base(Target)
	{
		// NOTE: General rule is not to access the private folder of another module,
		// but to use the ISubmixBufferListener interface, we  need to include some private headers
		PrivateIncludePaths.Add(System.IO.Path.Combine(Directory.GetCurrentDirectory(), "./Runtime/AudioMixer/Private"));

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"Engine",
				"AudioMixer",
				"AVEncoder",
				"GameplayMediaEncoder",
			}
		);

        PublicDelayLoadDLLs.Add("mfplat.dll");
        PublicDelayLoadDLLs.Add("mfuuid.dll");
        PublicDelayLoadDLLs.Add("Mfreadwrite.dll");

    }
}
